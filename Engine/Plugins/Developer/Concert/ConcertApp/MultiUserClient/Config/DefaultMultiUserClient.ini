[/Script/MultiUserClient.MultiUserReplicationSettings]
ReplicationEditorSettings=(DefaultPropertySelection=(("/Script/Engine.SceneComponent", (DefaultSelectedProperties=("RelativeLocation","RelativeLocation.X","RelativeLocation.Y","RelativeLocation.Z","RelativeRotation","RelativeRotation.Yaw","RelativeRotation.Pitch","RelativeRotation.Roll"))),("/Script/CinematicCamera.CineCameraComponent", ()),("/Script/Engine.CameraComponent", ())),DefaultAddedSubobjectRules=(SubobjectMatchingRules=(("/Script/Engine.Actor", (IncludeClasses=("/Script/Engine.StaticMeshComponent","/Script/Engine.CameraComponent","/Script/CinematicCamera.CineCameraComponent"))))))
FrequencyRules=(DefaultObjectFrequencySettings=(ReplicationMode=SpecifiedRate,ReplicationRate=30),FrequencyRuleStack=((ReplicationSettings=(ReplicationRate=60),SubobjectMatchingRules=(SubobjectMatchingRules=(("/Script/Engine.Actor", (IncludeClasses=("/Script/CinematicCamera.CineCameraComponent"))))))))

[CoreRedirects]
; 5.4 -> 5.5
+ClassRedirects=(OldName="/Script/MultiUserClient.MultiUserReplicationClientPreset",NewName="/Script/MultiUserClient.MultiUserReplicationClientContent")