{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Multi-User Editing",
	"Description": "Allow collaborative multi-users sessions in the Editor",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MultiUserClient",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MultiUserClientLibrary",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MultiUserReplicationEditor",
			"Type": "Editor",
			"LoadingPhase": "PostDefault"
		}
	],
	"Plugins": [
		{
			"Name": "ConcertClientSharedSlate",
			"Enabled": true
		},
		{
			"Name": "ConcertReplicationScripting",
			"Enabled": true
		},
        {
          "Name": "ConcertSharedSlate",
          "Enabled": true
        },
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "MultiUserTakes",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": false
}