// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MultiUserClient : ModuleRules
	{
		public MultiUserClient(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"ApplicationCore",
					"ContentBrowser",
					"Core",
					"DesktopPlatform",
					"EditorStyle",
					"Engine",
					"FileSandboxUI",
					"InputCore",
					"JsonUtilities",
					"MessageLog",
					"Projects",
					"Slate",
					"SlateCore",
					"SourceControl",
					"ToolMenus",
					"ToolWidgets",
					
					// Concert
					"Concert",
					"ConcertClient",
					"ConcertClientSharedSlate",
					"ConcertReplicationScripting",
					"ConcertSharedSlate",
					"ConcertSyncClient",
					"ConcertSyncCore",
					"ConcertTransport", 
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange( 
					new string[]
					{ 
						"EditorFramework",
						"MultiUserReplicationEditor",
						"PropertyEditor",
						"Settings",
						"Sequencer",
						"ToolMenus",
						"UnrealEd",
						"WorldBrowser",
						"WorkspaceMenuStructure",
					}
				);
			}
		}
	}
}
