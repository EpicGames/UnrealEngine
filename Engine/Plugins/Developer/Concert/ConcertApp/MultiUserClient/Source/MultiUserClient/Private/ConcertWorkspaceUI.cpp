// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertWorkspaceUI.h"

#include "ConcertMessages.h"
#include "IConcertSyncClientModule.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "IConcertSession.h"
#include "IConcertClientSequencerManager.h"
#include "ConcertFrontendStyle.h"
#include "ConcertLogGlobal.h"
#include "ContentBrowserMenuContexts.h"

#include "Misc/MessageDialog.h"

#include "Algo/Transform.h"
#include "Modules/ModuleManager.h"
#include "ContentBrowserModule.h"
#include "UObject/Package.h"
#include "ISourceControlState.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#include "Styling/AppStyle.h"
#include "Logging/MessageLog.h"
#include "Misc/AsyncTaskNotification.h"
#include "ISequencerModule.h"
#include "ISequencer.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Notifications/NotificationManager.h"

#include "Session/History/SSessionHistoryWrapper.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/ClientSessionHistoryController.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SConcertSandboxPersistWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"

LLM_DEFINE_TAG(Concert_ConcertWorkspaceUI);
#define LOCTEXT_NAMESPACE "ConcertFrontend"

static const FName ConcertHistoryTabName(TEXT("ConcertHistory"));

//-----------------------------------------------------------------------------
// Widgets to display icons on top of the content browser assets to show when
// an asset is locked or modified by somebody else.
//-----------------------------------------------------------------------------

template <>
struct TWidgetTypeTraits<class SConcertWorkspaceLockStateIndicator>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Controls the appearance of the Concert workspace lock state icon. The lock state icon is displayed
 * on an asset in the editor content browser seen when a Concert user saves an assets or explicitly locks
 * it. The color of the lock depends who owns the lock. The lock can be held by the local client or by another
 * client connected to the Concert session.
 */
class SConcertWorkspaceLockStateIndicator : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertWorkspaceLockStateIndicator) {}
		SLATE_ARGUMENT(FName, AssetPath)
	SLATE_END_ARGS();

	/**
	 * Construct this widget.
	 * @param InArgs Slate arguments
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FConcertWorkspaceUI> InWorkspaceFrontend)
	{
		WorkspaceFrontend = MoveTemp(InWorkspaceFrontend);
		AssetPath = InArgs._AssetPath;
		SetVisibility(MakeAttributeSP(this, &SConcertWorkspaceLockStateIndicator::GetVisibility));

		ChildSlot
		[
			SNew(SOverlay)

			+SOverlay::Slot() // A colored plane (avatar color) that leaks through the lock image transparent parts (if any).
			[
				SNew(SImage)
				.ColorAndOpacity(this, &SConcertWorkspaceLockStateIndicator::GetBackgroundColor)
				.Image(LockBackgroundBrush)
			]
			+SOverlay::Slot() // The lock image on top.
			[
				SNew(SImage)
				.Image(this, &SConcertWorkspaceLockStateIndicator::GetImageBrush)
			]
		];
	}

	/** Cache the indicator brushes for access. */
	static void CacheIndicatorBrushes()
	{
		if (MyLockBrush == nullptr)
		{
			LockBackgroundBrush = FConcertFrontendStyle::Get()->GetBrush(TEXT("Concert.LockBackground"));
			MyLockBrush = FConcertFrontendStyle::Get()->GetBrush(TEXT("Concert.MyLock"));
			OtherLockBrush = FConcertFrontendStyle::Get()->GetBrush(TEXT("Concert.OtherLock"));
		}
	}

private:
	EVisibility GetVisibility() const
	{
		// If the asset is locked, make the icon visible, collapsed/hidden otherwise.
		return WorkspaceFrontend->GetResourceLockId(AssetPath).IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FSlateColor GetBackgroundColor() const
	{
		return WorkspaceFrontend->GetUserAvatarColor(WorkspaceFrontend->GetResourceLockId(AssetPath));
	}

	const FSlateBrush* GetImageBrush() const
	{
		FGuid LockId = WorkspaceFrontend->GetResourceLockId(AssetPath);
		if (!LockId.IsValid())
		{
			return nullptr; // The asset is not locked, don't show any icon.
		}
		else if (LockId == WorkspaceFrontend->GetWorkspaceLockId())
		{
			return MyLockBrush; // The asset is locked by this workspace user.
		}
		else
		{
			return OtherLockBrush; // The asset is locked by another user.
		}
	}

	// Brushes used to render the lock icon.
	static const FSlateBrush* MyLockBrush;
	static const FSlateBrush* OtherLockBrush;
	static const FSlateBrush* LockBackgroundBrush;

	/** Asset path for this indicator widget.*/
	FName AssetPath;

	/** Holds pointer to the workspace front-end. */
	TSharedPtr<FConcertWorkspaceUI> WorkspaceFrontend;
};

const FSlateBrush* SConcertWorkspaceLockStateIndicator::MyLockBrush = nullptr;
const FSlateBrush* SConcertWorkspaceLockStateIndicator::OtherLockBrush = nullptr;
const FSlateBrush* SConcertWorkspaceLockStateIndicator::LockBackgroundBrush = nullptr;

/**
 * Displays a tooltip when moving the mouse over the 'lock' icon displayed on asset locked
 * through Concert.
 */
class SConcertWorkspaceLockStateTooltip : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertWorkspaceLockStateTooltip) {}
		SLATE_ARGUMENT(FName, AssetPath)
	SLATE_END_ARGS();

	/**
	 * Construct this widget.
	 * @param InArgs Slate arguments
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FConcertWorkspaceUI> InWorkspaceFrontend)
	{
		WorkspaceFrontend = MoveTemp(InWorkspaceFrontend);
		AssetPath = InArgs._AssetPath;
		SetVisibility(MakeAttributeSP(this, &SConcertWorkspaceLockStateTooltip::GetTooltipVisibility));

		ChildSlot
		[
			SNew(STextBlock)
			.Text(this, &SConcertWorkspaceLockStateTooltip::GetTooltipText)
			.ColorAndOpacity(this, &SConcertWorkspaceLockStateTooltip::GetLockColor)
		];
	}

private:
	EVisibility GetTooltipVisibility() const
	{
		return WorkspaceFrontend->GetResourceLockId(AssetPath).IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText GetTooltipText() const
	{
		FGuid LockId = WorkspaceFrontend->GetResourceLockId(AssetPath);
		if (!LockId.IsValid())
		{
			return FText::GetEmpty(); // Not locked.
		}
		else if (LockId == WorkspaceFrontend->GetWorkspaceLockId())
		{
			return LOCTEXT("MyLock_Tooltip", "Locked by you"); // Locked by this client.
		}
		else
		{
			return FText::Format(LOCTEXT("OtherLock_Tooltip", "Locked by: {0}"), WorkspaceFrontend->GetUserDescriptionText(LockId)); // Locked by another client.
		}
	}

	FSlateColor GetLockColor() const
	{
		FGuid LockId = WorkspaceFrontend->GetResourceLockId(AssetPath);
		if (!LockId.IsValid())
		{
			return FLinearColor(); // Not locked.
		}
		else if (LockId == WorkspaceFrontend->GetWorkspaceLockId())
		{
			return FConcertFrontendStyle::Get()->GetColor("Concert.Color.LocalUser"); // Locked by this client.
		}
		else
		{
			return FConcertFrontendStyle::Get()->GetColor("Concert.Color.OtherUser"); // Locked by another client.
		}
	}

	/** Asset path for this indicator widget.*/
	FName AssetPath;

	/** Holds pointer to the workspace front-end. */
	TSharedPtr<FConcertWorkspaceUI> WorkspaceFrontend;
};


template<>
struct TWidgetTypeTraits<class SConcertWorkspaceModifiedByOtherIndicator>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Controls the appearance of the workspace 'modified by other' icon. The icon is displayed on an
 * asset in the editor content browser when the a client different from this workspace client has
 * live transaction(s) on the asset. The indicator is cleared when all live transactions from other
 * clients are cleared, usually when the asset is saved to disk.
 */
class SConcertWorkspaceModifiedByOtherIndicator : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertWorkspaceModifiedByOtherIndicator) {}
		SLATE_ARGUMENT(FName, AssetPath)
	SLATE_END_ARGS();

	/**
	 * Construct this widget.
	 * @param InArgs Slate arguments
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FConcertWorkspaceUI> InWorkspaceFrontend)
	{
		WorkspaceFrontend = MoveTemp(InWorkspaceFrontend);
		AssetPath = InArgs._AssetPath;
		SetVisibility(MakeAttributeSP(this, &SConcertWorkspaceModifiedByOtherIndicator::GetVisibility));

		ChildSlot
		[
			SNew(SImage)
			.Image(this, &SConcertWorkspaceModifiedByOtherIndicator::GetImageBrush)
		];
	}

	/** Caches the indicator brushes for access. */
	static void CacheIndicatorBrush()
	{
		if (ModifiedByOtherBrush == nullptr)
		{
			ModifiedByOtherBrush = FConcertFrontendStyle::Get()->GetBrush(TEXT("Concert.ModifiedByOther"));
		}
	}

private:
	EVisibility GetVisibility() const
	{
		return WorkspaceFrontend->IsAssetModifiedByOtherClients(AssetPath) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	const FSlateBrush* GetImageBrush() const
	{
		return WorkspaceFrontend->IsAssetModifiedByOtherClients(AssetPath) ? ModifiedByOtherBrush : nullptr;
	}

	/* Brush indicating that the asset has been modified by another user. */
	static const FSlateBrush* ModifiedByOtherBrush;

	/** Asset path for this indicator widget.*/
	FName AssetPath;

	/** Holds pointer to the workspace front-end. */
	TSharedPtr<FConcertWorkspaceUI> WorkspaceFrontend;
};

const FSlateBrush* SConcertWorkspaceModifiedByOtherIndicator::ModifiedByOtherBrush = nullptr;

/**
 * Displays a tooltip when moving the mouse over the 'modified by...' icon displayed on asset
 * modified, through Concert, by any client other than the client workspace.
 */
class SConcertWorkspaceModifiedByOtherTooltip : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertWorkspaceModifiedByOtherTooltip) {}
		SLATE_ARGUMENT(FName, AssetPath)
	SLATE_END_ARGS();

	/**
	 * Construct this widget.
	 * @param InArgs Slate arguments
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FConcertWorkspaceUI> InWorkspaceFrontend)
	{
		WorkspaceFrontend = MoveTemp(InWorkspaceFrontend);
		AssetPath = InArgs._AssetPath;
		SetVisibility(MakeAttributeSP(this, &SConcertWorkspaceModifiedByOtherTooltip::GetVisibility));

		ChildSlot
		[
			SNew(STextBlock)
			.Text(this, &SConcertWorkspaceModifiedByOtherTooltip::GetTooltip)
			.ColorAndOpacity(this, &SConcertWorkspaceModifiedByOtherTooltip::GetToolTipColor)
		];
	}

private:
	EVisibility GetVisibility() const
	{
		return WorkspaceFrontend->IsAssetModifiedByOtherClients(AssetPath) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText GetTooltip() const
	{
		// NOTE: We expect this function to be called only when visible, so we already know the resource was modified by someone.
		TArray<FConcertClientInfo> ModifiedBy;
		int32 ModifyByOtherCount = 0;
		WorkspaceFrontend->IsAssetModifiedByOtherClients(AssetPath, &ModifyByOtherCount, &ModifiedBy, 1); // Read up to 1 user
		return ModifyByOtherCount == 1 ?
			FText::Format(LOCTEXT("ConcertModifiedByUser_Tooltip", "Modified by {0}"), WorkspaceFrontend->GetUserDescriptionText(ModifiedBy[0])) :
			FText::Format(LOCTEXT("ConcertModifiedByNumUsers_Tooltip", "Modified by {0} other users"), ModifyByOtherCount);
	}

	FSlateColor GetToolTipColor() const
	{
		return WorkspaceFrontend->IsAssetModifiedByOtherClients(AssetPath) ? FConcertFrontendStyle::Get()->GetColor("Concert.Color.OtherUser") : FLinearColor();
	}

	/** Asset path for this indicator widget.*/
	FName AssetPath;

	/** Holds pointer to the workspace front-end. */
	TSharedPtr<FConcertWorkspaceUI> WorkspaceFrontend;
};


/**
 * Displays a tooltip when moving the mouse over the 'lock' icon displayed on asset locked
 * through Concert.
 */
class SConcertWorkspaceSequencerToolbarExtension : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertWorkspaceSequencerToolbarExtension) {}
	SLATE_END_ARGS();

	/**
	 * Construct this widget.
	 * @param InArgs Slate arguments
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FConcertWorkspaceUI> InWorkspaceFrontend)
	{
		WorkspaceFrontend = MoveTemp(InWorkspaceFrontend);
		SetVisibility(MakeAttributeSP(this, &SConcertWorkspaceSequencerToolbarExtension::GetVisibility));

		FToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None, TSharedPtr<FExtender>(), true);

		ToolbarBuilder.BeginSection("Concert Sequencer");
		{
			// Toggle playback sync
			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						if (SyncClientPin && SyncClientPin->GetSequencerManager())
						{
							IConcertClientSequencerManager* SequencerManager = SyncClientPin->GetSequencerManager();
							SequencerManager->SetSequencerPlaybackSync(!SequencerManager->IsSequencerPlaybackSyncEnabled());
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						return SyncClientPin.IsValid() && SyncClientPin->GetSequencerManager() && SyncClientPin->GetSequencerManager()->IsSequencerPlaybackSyncEnabled();
					})
				),
				NAME_None,
				LOCTEXT("TogglePlaybackSyncLabel", "Playback Sync"),
				LOCTEXT("TogglePlaybackSyncTooltip", "Toggle Multi-User Playback Sync. If the option is enabled, playback and scrubbing of Sequencer will be synchronized across all users in a Multi-Users session if they have that sequence open and this option enabled."),
				FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Sequencer.SyncTimeline", "Concert.Sequencer.SyncTimeline.Small"),
				EUserInterfaceActionType::ToggleButton
			);

			// Toggle unrelated timeline sync
			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						if (SyncClientPin && SyncClientPin->GetSequencerManager())
						{
							IConcertClientSequencerManager* SequencerManager = SyncClientPin->GetSequencerManager();
							SequencerManager->SetUnrelatedSequencerTimelineSync(!SequencerManager->IsUnrelatedSequencerTimelineSyncEnabled());
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						return SyncClientPin.IsValid() && SyncClientPin->GetSequencerManager() && SyncClientPin->GetSequencerManager()->IsUnrelatedSequencerTimelineSyncEnabled();
					})
				),
				NAME_None,
				LOCTEXT("ToggleUnrelatedTimelineSyncLabel", "Unrelated Timeline Sync"),
				LOCTEXT("ToggleUnrelatedTimelineSyncTooltip", "Toggle Multi-User Unrelated Timeline Sync. If the option is enabled, playback and scrubbing of Sequencer will be synchronized when receiving any other user time sync from any sequence."),
				FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Sequencer.SyncUnrelated", "Concert.Sequencer.SyncUnrelated.Small"),
				EUserInterfaceActionType::ToggleButton
			);

			// Toggle remote open
			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						if (SyncClientPin && SyncClientPin->GetSequencerManager())
						{
							IConcertClientSequencerManager* SequencerManager = SyncClientPin->GetSequencerManager();
							SequencerManager->SetSequencerRemoteOpen(!SequencerManager->IsSequencerRemoteOpenEnabled());
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						return SyncClientPin.IsValid() && SyncClientPin->GetSequencerManager() && SyncClientPin->GetSequencerManager()->IsSequencerRemoteOpenEnabled();
					})
				),
				NAME_None,
				LOCTEXT("ToggleRemoteOpenLabel", "Remote Open"),
				LOCTEXT("ToggleRemoteOpenTooltip", "Toggle Multi-User Remote Open. If the option is enabled, opening a sequence will open the same sequence on all users in the Multi-User session that also have this option enabled."),
				FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Sequencer.SyncSequence", "Concert.Sequencer.SyncSequence.Small"),
				EUserInterfaceActionType::ToggleButton
			);

			// Toggle remote close
			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						if (SyncClientPin && SyncClientPin->GetSequencerManager())
						{
							IConcertClientSequencerManager* SequencerManager = SyncClientPin->GetSequencerManager();
							SequencerManager->SetSequencerRemoteClose(!SequencerManager->IsSequencerRemoteCloseEnabled());
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([]()
					{
						TSharedPtr<IConcertSyncClient> SyncClientPin = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						return SyncClientPin.IsValid() && SyncClientPin->GetSequencerManager() && SyncClientPin->GetSequencerManager()->IsSequencerRemoteCloseEnabled();
					})
				),
				NAME_None,
				LOCTEXT("ToggleRemoteCloseLabel", "Remote Close"),
				LOCTEXT("ToggleRemoteCloseTooltip", "Toggle Multi-User Remote Close. If the option is enabled, any open Sequencers for a sequence will be closed when another user in the Multi-User session closes that sequence."),
				FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Sequencer.SyncSequence", "Concert.Sequencer.SyncSequence.Small"),
				EUserInterfaceActionType::ToggleButton
			);
		}
		ToolbarBuilder.EndSection();

		ChildSlot
		[
			ToolbarBuilder.MakeWidget()
		];
	}

private:
	EVisibility GetVisibility() const
	{
		return WorkspaceFrontend->ClientWorkspace.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/** Holds pointer to the workspace front-end. */
	TSharedPtr<FConcertWorkspaceUI> WorkspaceFrontend;
};

//------------------------------------------------------------------------------
// FConcertWorkspaceUI implementation.
//------------------------------------------------------------------------------

FName FConcertWorkspaceUI::ConcertSourceControlMenuOwnerName = TEXT("ConcertSourceControlMenu");
FName FConcertWorkspaceUI::ConcertPersistSessionChangesMenuEntryName = TEXT("ConcertPersistSessionChanges");

FName FConcertWorkspaceUI::AssetActionSectionNameToExtend = TEXT("ContentBrowser.AssetContextMenu");
FName FConcertWorkspaceUI::ConcertAssetActionsMenuOwnerName = TEXT("ConcertAssetActionsMenu");
FName FConcertWorkspaceUI::ConcertAssetActionsSectionName = TEXT("ConcertAssetActionsSection");
FName FConcertWorkspaceUI::ConcertAssetActionsDynamicEntryName = TEXT("MultiUserActions");
FName FConcertWorkspaceUI::ConcertAssetActionsSubMenuName = TEXT("MultiUserActionsSubMenu");

FConcertWorkspaceUI::FConcertWorkspaceUI()
{
	// Extend ContentBrowser Asset Icon
	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser")))
	{
		// Caches the icon brushes if not already cached.
		SConcertWorkspaceLockStateIndicator::CacheIndicatorBrushes();
		SConcertWorkspaceModifiedByOtherIndicator::CacheIndicatorBrush();

		// The 'lock' state icon displayed on top of the asset in the editor content browser.
		ContentBrowserAssetExtraStateDelegateHandles.Emplace(ContentBrowserModule->AddAssetViewExtraStateGenerator(FAssetViewExtraStateGenerator(
			FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FConcertWorkspaceUI::OnGenerateAssetViewLockStateIcons),
			FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FConcertWorkspaceUI::OnGenerateAssetViewLockStateTooltip)
		)));

		// The 'Modified by other' icon displayed on top of the asset in the editor content browser.
		ContentBrowserAssetExtraStateDelegateHandles.Emplace(ContentBrowserModule->AddAssetViewExtraStateGenerator(FAssetViewExtraStateGenerator(
			FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FConcertWorkspaceUI::OnGenerateAssetViewModifiedByOtherIcon),
			FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FConcertWorkspaceUI::OnGenerateAssetViewModifiedByOtherTooltip)
		)));
	}

	// Extend Sequencer toolbars
	if (ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		OnPreSequencerInitHandle = SequencerModulePtr->RegisterOnPreSequencerInit(FOnPreSequencerInit::FDelegate::CreateRaw(this, &FConcertWorkspaceUI::OnPreSequencerInit));
	}

	AssetHistoryLayout = FTabManager::NewLayout("ConcertAssetHistory_Layout")
		->AddArea
		(
			FTabManager::NewArea(700, 700)
			->SetOrientation(EOrientation::Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(ConcertHistoryTabName, ETabState::ClosedTab)
			)
		);
}

FConcertWorkspaceUI::~FConcertWorkspaceUI()
{
	// Remove Content Browser Asset Icon extensions
	FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser"));

	if (ContentBrowserModule)
	{
		for (const FDelegateHandle& DelegateHandle : ContentBrowserAssetExtraStateDelegateHandles)
		{
			if (DelegateHandle.IsValid())
			{
				ContentBrowserModule->RemoveAssetViewExtraStateGenerator(DelegateHandle);
			}
		}
	}

	// Remove Sequencer preinit hooks for toolbar extenders
	if (ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerModulePtr->UnregisterOnPreSequencerInit(OnPreSequencerInitHandle);
	}
}

void FConcertWorkspaceUI::InstallWorkspaceExtensions(TWeakPtr<IConcertClientWorkspace> InClientWorkspace, TWeakPtr<IConcertSyncClient> InSyncClient)
{
	UninstallWorspaceExtensions();
	ClientWorkspace = InClientWorkspace;
	SyncClient = InSyncClient;

	ExtendToolbarWithSourceControlMenu();
	ExtendAssetContextMenuOptions();

	// Register for the "MarkPackageDirty" callback to catch packages that have been modified so we can acquire lock or warn
	UPackage::PackageMarkedDirtyEvent.AddRaw(this, &FConcertWorkspaceUI::OnMarkPackageDirty);
	FCoreDelegates::OnEndFrame.AddRaw(this, &FConcertWorkspaceUI::OnEndFrame);
}

void FConcertWorkspaceUI::UninstallWorspaceExtensions()
{
	// Remove Content Browser extensions
	FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>(TEXT("ContentBrowser"));
	if (ContentBrowserAssetExtenderDelegateHandle.IsValid() && ContentBrowserModule)
	{
		ContentBrowserModule->GetAllAssetViewContextMenuExtenders().RemoveAll([DelegateHandle = ContentBrowserAssetExtenderDelegateHandle](const FContentBrowserMenuExtender_SelectedAssets& Delegate) { return Delegate.GetHandle() == DelegateHandle; });
		ContentBrowserAssetExtenderDelegateHandle.Reset();
	}

	UToolMenus::Get()->UnregisterOwnerByName(ConcertSourceControlMenuOwnerName);
	UToolMenus::Get()->UnregisterOwnerByName(ConcertAssetActionsMenuOwnerName);

	// Remove package dirty hook
	UPackage::PackageMarkedDirtyEvent.RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	ClientWorkspace.Reset();
}

bool FConcertWorkspaceUI::HasSessionChanges() const
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	return ClientWorkspacePin.IsValid() && ClientWorkspacePin->HasSessionChanges();
}

namespace UE::ConcertWorkspaceUI::Private
{
bool ShouldContinueWithSendReceiveState(EConcertSendReceiveState SendReceiveState)
{
	if (SendReceiveState == EConcertSendReceiveState::Default)
	{
		return true;
	}

	FText WarningMsg;
	if (SendReceiveState == EConcertSendReceiveState::SendOnly)
	{
		WarningMsg = LOCTEXT("PersistInNonDefaultStateSendOnly", "You are about to persist a session in send only mode.  Changes from other clients may not be included.  Continue?");
	}
	else if (SendReceiveState == EConcertSendReceiveState::ReceiveOnly)
	{
		WarningMsg = LOCTEXT("PersistInNonDefaultStateReceiveOnly", "You are about to persist a session in receive only mode.  Your changes have not been sent to other clients.  Continue?");
	}
	else
	{
		checkNoEntry();
	}
	EAppReturnType::Type RetType = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, WarningMsg);
	
	return RetType == EAppReturnType::Yes;
}
}

bool FConcertWorkspaceUI::PromptPersistSessionChanges()
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	TArray<TSharedPtr<FConcertPersistItem>> PersistItems;
	if (ClientWorkspacePin.IsValid())
	{
		IConcertClientSession& Session = ClientWorkspacePin->GetSession();
		if (!UE::ConcertWorkspaceUI::Private::ShouldContinueWithSendReceiveState(Session.GetSendReceiveState()))
		{
			return false;
		}

		// Get source control status of packages 
		TArray<FString> PackageFilenames;
		TArray<FName> PackageNames;
		TArray<FSourceControlStateRef> States;
		TArray<FName> CandidatePackages = ClientWorkspacePin->GatherSessionChanges();
		for (FName PackageName : CandidatePackages)
		{
			if (TOptional<FString> PackagePath = ClientWorkspacePin->GetValidPackageSessionPath(PackageName))
			{
				PackageFilenames.Add(PackagePath.GetValue());
				PackageNames.Add(PackageName);
			}
		}

		ECommandResult::Type Result = ISourceControlModule::Get().GetProvider().GetState(PackageFilenames, States, EStateCacheUsage::ForceUpdate);
		// The dummy Multi-User source control provider always succeed and always return proxy states.
		ensure(Result == ECommandResult::Succeeded);

		// Create the list of persist items from the package names and source control state
		for (int32 Index = 0; Index < PackageNames.Num(); ++Index)
		{
			PersistItems.Add(MakeShared<FConcertPersistItem>(PackageNames[Index], States[Index]));
		}
	}

	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(LOCTEXT("PersistSubmitWindowTitle", "Persist & Submit Files"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(600, 600))
		.SupportsMaximize(true)
		.SupportsMinimize(false);

	TSharedRef<SConcertSandboxPersistWidget> PersistWidget =
		SNew(SConcertSandboxPersistWidget)
		.ParentWindow(NewWindow)
		.Items(PersistItems);

	NewWindow->SetContent(
		PersistWidget
	);
	FSlateApplication::Get().AddModalWindow(NewWindow, nullptr);

	// if canceled, just exit
	if (!PersistWidget->IsDialogConfirmed())
	{
		return false; // Canceled.
	}
	FConcertPersistCommand PersistCmd = PersistWidget->GetPersistCommand();

	// Prepare the operation notification
	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.LogCategory = &LogConcert;
	NotificationConfig.TitleText = LOCTEXT("PersistingChanges", "Persisting Session Changes");
	FAsyncTaskNotification Notification(NotificationConfig);
	FText NotificationSub;

	FPersistResult PersistResult = ClientWorkspacePin->PersistSessionChanges(
		{PersistCmd.PackagesToPersist, PersistCmd.bShouldMakeFilesWritable});

	bool bSuccess = PersistResult.PersistStatus == EPersistStatus::Success;
	if (bSuccess)
	{
		bSuccess = SubmitChangelist(PersistCmd, NotificationSub);
	}
	else
	{
		NotificationSub = FText::Format(
			LOCTEXT("FailedPersistNotification",
					"Failed to persist session files. Reported {0} {0}|plural(one=error,other=errors)."), PersistResult.FailureReasons.Num());
		FMessageLog ConcertLog("Concert");
		for (const FText& Failure : PersistResult.FailureReasons)
		{
			ConcertLog.Error(Failure);
		}
	}

	Notification.SetProgressText(LOCTEXT("SeeMessageLog", "See Message Log"));
	Notification.SetComplete(bSuccess ?
							 LOCTEXT("PersistChangeSuccessHeader", "Successfully Persisted Session Changes")
							 : LOCTEXT("PersistChangeFailedHeader", "Failed to Persist Session Changes"), NotificationSub, bSuccess);

	return true; // Persisting.
}

bool FConcertWorkspaceUI::SubmitChangelist(const FConcertPersistCommand& PersistCommand, FText& OperationMessage)
{
	if (!PersistCommand.bShouldSubmit || PersistCommand.FilesToPersist.Num() == 0)
	{
		OperationMessage = LOCTEXT("PersistChangeSuccess", "Succesfully persisted session files");
		return true;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Revert any unchanged files first
	SourceControlHelpers::RevertUnchangedFiles(SourceControlProvider, PersistCommand.FilesToPersist);

	// Re-update the cache state with the modified flag
	TSharedPtr<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOp = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOp->SetUpdateModifiedState(true);
	SourceControlProvider.Execute(UpdateStatusOp.ToSharedRef(), PersistCommand.FilesToPersist);

	// Build the submit list, skipping unchanged files.
	TArray<FString> FilesToSubmit;
	FilesToSubmit.Reserve(PersistCommand.FilesToPersist.Num());
	for (const FString& File : PersistCommand.FilesToPersist)
	{
		FSourceControlStatePtr FileState = SourceControlProvider.GetState(File, EStateCacheUsage::Use);
		if (FileState.IsValid() &&
			(FileState->IsAdded() ||
				FileState->IsDeleted() ||
				FileState->IsModified() ||
				(SourceControlProvider.UsesCheckout() && FileState->IsCheckedOut())
			))
		{
			FilesToSubmit.Add(File);
		}
	}

	// Check in files
	bool bCheckinSuccess = false;
	if (FilesToSubmit.Num() > 0)
	{
		TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
		CheckInOperation->SetDescription(PersistCommand.ChangelistDescription);

		bCheckinSuccess = SourceControlProvider.Execute(CheckInOperation, FilesToSubmit) == ECommandResult::Succeeded;
		if (bCheckinSuccess)
		{
			OperationMessage = CheckInOperation->GetSuccessMessage();
		}
		else
		{
			OperationMessage = LOCTEXT("SourceControlSubmitFailed", "Failed to check in persisted files!");
		}
	}
	else
	{
		OperationMessage = LOCTEXT("SourceControlNoSubmitFail", "No file to submit after persisting!");
	}
	return bCheckinSuccess;
}

FText FConcertWorkspaceUI::GetUserDescriptionText(const FGuid& ClientId) const
{
	if (TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin())
	{
		FConcertSessionClientInfo ClientSessionInfo;
		if (ClientWorkspacePin->GetSession().FindSessionClient(ClientId, ClientSessionInfo))
		{
			return GetUserDescriptionText(ClientSessionInfo.ClientInfo);
		}
	}
	return FText();
}

FText FConcertWorkspaceUI::GetUserDescriptionText(const FConcertClientInfo& ClientInfo) const
{
	return (ClientInfo.DisplayName != ClientInfo.UserName) ?
		FText::Format(LOCTEXT("ConcertUserDisplayNameOnDevice", "'{0}' ({1}) on {2}"), FText::FromString(ClientInfo.DisplayName), FText::FromString(ClientInfo.UserName), FText::FromString(ClientInfo.DeviceName)) :
		FText::Format(LOCTEXT("ConcertUserNameOnDevice", "'{0}' on {1}"), FText::FromString(ClientInfo.DisplayName), FText::FromString(ClientInfo.DeviceName));
}

FLinearColor FConcertWorkspaceUI::GetUserAvatarColor(const FGuid& ClientId) const
{
	if (TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin())
	{
		if (ClientWorkspacePin->GetSession().GetLocalClientInfo().InstanceInfo.InstanceId == ClientId) // This client?
		{
			return ClientWorkspacePin->GetSession().GetLocalClientInfo().AvatarColor;
		}

		FConcertSessionClientInfo ClientSessionInfo;
		if (ClientWorkspacePin->GetSession().FindSessionClient(ClientId, ClientSessionInfo))
		{
			return ClientSessionInfo.ClientInfo.AvatarColor;
		}
	}

	return FLinearColor::Transparent;
}

FGuid FConcertWorkspaceUI::GetWorkspaceLockId() const
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	if (ClientWorkspacePin.IsValid())
	{
		return ClientWorkspacePin->GetWorkspaceLockId();
	}
	return FGuid();
}

FGuid FConcertWorkspaceUI::GetResourceLockId(const FName InResourceName) const
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	if (ClientWorkspacePin.IsValid())
	{
		return ClientWorkspacePin->GetResourceLockId(InResourceName);
	}
	return FGuid();
}

bool FConcertWorkspaceUI::CanLockResources(TArray<FName> InResourceNames) const
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	return ClientWorkspacePin.IsValid() && ClientWorkspacePin->AreResourcesLockedBy(InResourceNames, FGuid());
}

bool FConcertWorkspaceUI::CanUnlockResources(TArray<FName> InResourceNames) const
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	return ClientWorkspacePin.IsValid() && ClientWorkspacePin->AreResourcesLockedBy(InResourceNames, ClientWorkspacePin->GetWorkspaceLockId());
}

void FConcertWorkspaceUI::ExecuteLockResources(TArray<FName> InResourceNames)
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	if (ClientWorkspacePin.IsValid())
	{
		ClientWorkspacePin->LockResources(MoveTemp(InResourceNames));
	}
}

void FConcertWorkspaceUI::ExecuteUnlockResources(TArray<FName> InResourceNames)
{
	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	if (ClientWorkspacePin.IsValid())
	{
		ClientWorkspacePin->UnlockResources(MoveTemp(InResourceNames));
	}
}

void FConcertWorkspaceUI::ExecuteViewHistory(TArray<FName> InResourceNames)
{
	if (const TSharedPtr<IConcertSyncClient> ClientPin = SyncClient.Pin())
	{
		FGlobalTabmanager::Get()->RestoreFrom(AssetHistoryLayout.ToSharedRef(), nullptr);
		
		const TSharedRef<IConcertSyncClient> ClientRef = ClientPin.ToSharedRef();
		for (const FName& ResourceName : InResourceNames)
		{
			FGlobalTabmanager::Get()->InsertNewDocumentTab(ConcertHistoryTabName, FTabManager::ESearchPreference::PreferLiveTab, CreateHistoryTab(ResourceName, ClientRef));
		}
	}
	
}

bool FConcertWorkspaceUI::IsAssetModifiedByOtherClients(const FName& AssetName, int32* OutOtherClientsWithModifNum, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo, int32 OtherClientsWithModifMaxFetchNum) const
{
	if (TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin())
	{
		return ClientWorkspacePin->IsAssetModifiedByOtherClients(AssetName, OutOtherClientsWithModifNum, OutOtherClientsWithModifInfo, OtherClientsWithModifMaxFetchNum);
	}

	return false;
}

void FConcertWorkspaceUI::OnEndFrame()
{
	PerFramePackageDirtyList.Reset();
}

void FConcertWorkspaceUI::OnMarkPackageDirty(UPackage* InPackage, bool /*bWasDirty*/)
{
	if (PerFramePackageDirtyList.Contains(InPackage->GetFName()))
	{
		// In a take recorder situation, a record can include a large number of dirty events (especially with
		// a large number of properties).  This means that this delegate would get invoked at least once
		// per property per frame.  We reduce this overhead by storing the checked package and prevent
		// the relatively expensive call below.
		//
		return;
	}

	TSharedPtr<IConcertClientWorkspace> ClientWorkspacePin = ClientWorkspace.Pin();
	if (ClientWorkspacePin.IsValid()
		&& !ClientWorkspacePin->ShouldIgnorePackageDirtyEvent(InPackage)
		&& !ClientWorkspacePin->HasLiveTransactionSupport(InPackage))
	{
		FGuid ResourceLockId = ClientWorkspacePin->GetResourceLockId(InPackage->GetFName());

		// Automatically acquire the lock on the asset if it doesn't support live transactions
		if (!ResourceLockId.IsValid())
		{
			ClientWorkspacePin->LockResources({ InPackage->GetFName() });
		}
		// If the resource is locked by another client, warn the user
		else if (ResourceLockId != ClientWorkspacePin->GetWorkspaceLockId())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Asset"), FText::FromName(InPackage->GetFName()));
			Args.Add(TEXT("User"), GetUserDescriptionText(ResourceLockId));
			FText ErrorText = FText::Format(LOCTEXT("EditLockedNotification", "{Asset} is currently locked by {User}"), Args);

			if (!EditLockedNotificationWeakPtr.IsValid())
			{
				FNotificationInfo ErrorNotification(ErrorText);
				ErrorNotification.bFireAndForget = true;
				ErrorNotification.bUseLargeFont = false;
				ErrorNotification.ExpireDuration = 6.0f;
				EditLockedNotificationWeakPtr = FSlateNotificationManager::Get().AddNotification(ErrorNotification);
			}
			else
			{
				EditLockedNotificationWeakPtr.Pin()->SetText(ErrorText);
				EditLockedNotificationWeakPtr.Pin()->ExpireAndFadeout();
			}
		}
	}
	PerFramePackageDirtyList.Add(InPackage->GetFName());
}

void FConcertWorkspaceUI::ExtendToolbarWithSourceControlMenu()
{
	const FToolMenuOwnerScoped SourceControlMenuOwner(ConcertSourceControlMenuOwnerName);
	// Setup Concert Source Control Extension
	UToolMenu* const SourceControlMenu = UToolMenus::Get()->ExtendMenu("StatusBar.ToolBar.SourceControl");
	FToolMenuSection& Section = SourceControlMenu->FindOrAddSection("SourceControlMenu");

	Section.AddMenuEntry(
		ConcertPersistSessionChangesMenuEntryName,
		LOCTEXT("ConcertWVPersist", "Persist Session Changes..."),
		LOCTEXT("ConcertWVPersistTooltip", "Persist the session changes and prepare the files for revision control submission."),
		FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Persist"),
		FUIAction(FExecuteAction::CreateLambda([Weak = TWeakPtr<FConcertWorkspaceUI>(AsShared())]()
		{
			if (TSharedPtr<FConcertWorkspaceUI> PinThis = Weak.Pin())
			{
				PinThis->PromptPersistSessionChanges(); // Required to adapt the function signature.
			}
		}))
	);
}

void FConcertWorkspaceUI::ExtendAssetContextMenuOptions()
{
	const FToolMenuOwnerScoped SourceControlMenuOwner(ConcertAssetActionsMenuOwnerName);
	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(AssetActionSectionNameToExtend);
	FToolMenuSection& Section = Menu->AddSection(ConcertAssetActionsSectionName, LOCTEXT("MultiUserSectionLabel", "Multi-User"));
	Section.AddDynamicEntry(ConcertAssetActionsDynamicEntryName, FNewToolMenuSectionDelegate::CreateLambda([WeakThis = TWeakPtr<FConcertWorkspaceUI>(AsShared())](FToolMenuSection& InSection)
	{
		const TSharedPtr<FConcertWorkspaceUI> PinThis = WeakThis.Pin();
		UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
		if (!Context || !Context->bCanBeModified || Context->SelectedAssets.Num() == 0 || !ensure(PinThis))
		{
			return;
		}

		TArray<FName> TransformedAssets;
		Algo::Transform(Context->SelectedAssets, TransformedAssets, [](const FAssetData& AssetData)
		{
			return AssetData.PackageName;
		});

		InSection.AddSubMenu(
			ConcertAssetActionsSubMenuName,
			LOCTEXT("Concert_ContextMenu", "Multi-User Actions"),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(PinThis.Get(), &FConcertWorkspaceUI::GenerateConcertAssetContextMenu, MoveTemp(TransformedAssets)),
			false,
			FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.MultiUser")
			);
	}));
}

void FConcertWorkspaceUI::GenerateConcertAssetContextMenu(FMenuBuilder& MenuBuilder, TArray<FName> AssetObjectPaths)
{
	MenuBuilder.BeginSection("AssetConcertActions", LOCTEXT("AssetConcertActionsMenuHeading", "Multi-User"));

	// Lock Resource Action
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ConcertWVLock", "Lock Asset(s)"),
			LOCTEXT("ConcertWVLockTooltip", "Lock the asset(s) for editing."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertWorkspaceUI::ExecuteLockResources, AssetObjectPaths),
				FCanExecuteAction::CreateSP(this, &FConcertWorkspaceUI::CanLockResources, AssetObjectPaths)
			)
		);
	}
	
	// Unlock Resource Action
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ConcertWVUnlock", "Unlock Asset(s)"),
			LOCTEXT("ConcertWVUnlockTooltip", "Unlock the asset(s)."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertWorkspaceUI::ExecuteUnlockResources, AssetObjectPaths),
				FCanExecuteAction::CreateSP(this, &FConcertWorkspaceUI::CanUnlockResources, AssetObjectPaths)
			)
		);
	}

	// Lookup history for the asset
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ConcertWVHistory", "Asset history..."),
			LOCTEXT("ConcertWVHistoryToolTip", "View the asset's session history."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FConcertWorkspaceUI::ExecuteViewHistory, AssetObjectPaths)
			)
		);
	}

	MenuBuilder.EndSection();
}

TSharedRef<SWidget> FConcertWorkspaceUI::OnGenerateAssetViewLockStateIcons(const FAssetData& AssetData)
{	
	LLM_SCOPE_BYTAG(Concert_ConcertWorkspaceUI);
	return SNew(SConcertWorkspaceLockStateIndicator, AsShared())
		.AssetPath(AssetData.PackageName);
}

TSharedRef<SWidget> FConcertWorkspaceUI::OnGenerateAssetViewLockStateTooltip(const FAssetData& AssetData)
{
	LLM_SCOPE_BYTAG(Concert_ConcertWorkspaceUI);
	return SNew(SConcertWorkspaceLockStateTooltip, AsShared())
		.AssetPath(AssetData.PackageName);
}

TSharedRef<SWidget> FConcertWorkspaceUI::OnGenerateAssetViewModifiedByOtherIcon(const FAssetData& AssetData)
{
	LLM_SCOPE_BYTAG(Concert_ConcertWorkspaceUI);
	return SNew(SConcertWorkspaceModifiedByOtherIndicator, AsShared())
		.AssetPath(AssetData.PackageName);
}

TSharedRef<SWidget> FConcertWorkspaceUI::OnGenerateAssetViewModifiedByOtherTooltip(const FAssetData& AssetData)
{
	LLM_SCOPE_BYTAG(Concert_ConcertWorkspaceUI);
	return SNew(SConcertWorkspaceModifiedByOtherTooltip, AsShared())
		.AssetPath(AssetData.PackageName);
}

void FConcertWorkspaceUI::OnPreSequencerInit(TSharedRef<ISequencer> InSequencer, TSharedRef<ISequencerObjectChangeListener> InObjectChangeListener, const FSequencerInitParams& InitParams)
{
	if (InitParams.bEditWithinLevelEditor)
	{
		TWeakPtr<FConcertWorkspaceUI> Weak = AsShared();
		TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();

		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		TSharedPtr<FExtensibilityManager> ExtensionManager = SequencerModule.GetToolBarExtensibilityManager();

		if (ToolbarExtender.IsValid())
		{
			ExtensionManager->RemoveExtender(ToolbarExtender);
			ToolbarExtender.Reset();
		}

		ToolbarExtender = MakeShareable(new FExtender);
		ToolbarExtender->AddToolBarExtension(
			"CurveEditor",
			EExtensionHook::Before,
			CommandList,
			FToolBarExtensionDelegate::CreateLambda([Weak](FToolBarBuilder& ToolbarBuilder)
			{
				if (TSharedPtr<FConcertWorkspaceUI> PinThis = Weak.Pin())
				{
					ToolbarBuilder.AddWidget(SNew(SConcertWorkspaceSequencerToolbarExtension, PinThis));
				}
			})
		);

		ExtensionManager->AddExtender(ToolbarExtender);
	}
}

TSharedRef<SDockTab> FConcertWorkspaceUI::CreateHistoryTab(const FName& ResourceName, const TSharedRef<IConcertSyncClient>& SyncClient)
{
	const TSharedRef<FClientSessionHistoryController> SessionHistoryController =
		MakeShared<FClientSessionHistoryController>(SyncClient, ResourceName);
	return SNew(SDockTab)
		.TabRole(ETabRole::DocumentTab)
		.ContentPadding(FMargin(3.0f))
		.Label(FText::FromName(ResourceName))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(3.0f)
				.AutoWidth()
				[
					SNew(SImage)
					// Todo: Find another icon for the history tab.
					.Image( FAppStyle::GetBrush( "LevelEditor.Tabs.Details" ) )
				]

				+SHorizontalBox::Slot()
				.Padding(3.0f)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("AssetsHistory", "{0}'s history."), FText::FromString(ResourceName.ToString())))
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.ShadowOffset(FVector2D(1.0f, 1.0f))
				]
			]

			+SVerticalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(4.0f, 2.0f))
				[
					SNew(SSessionHistoryWrapper, SessionHistoryController)
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
