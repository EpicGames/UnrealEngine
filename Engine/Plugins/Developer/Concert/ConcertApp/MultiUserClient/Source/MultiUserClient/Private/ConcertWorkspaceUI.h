// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/TabManager.h"

class IConcertClientWorkspace;
class IConcertSyncClient;
struct FConcertPersistCommand;
struct FConcertClientInfo;

struct FAssetData;
class SWidget;
class FExtender;
class FMenuBuilder;
class SDockTab;
class SNotificationItem;
class ISequencer;
class ISequencerObjectChangeListener;
struct FSequencerInitParams;

/**
 * Concert Client Workspace UI
 */
class FConcertWorkspaceUI : public TSharedFromThis<FConcertWorkspaceUI>
{
	/** Name of menu extensions going into source control (bottom-right) */
	static FName ConcertSourceControlMenuOwnerName;
	/** Name for menu entry in source control that will persist changes */
	static FName ConcertPersistSessionChangesMenuEntryName;

	/** Name of UE section in asset context */
	static FName AssetActionSectionNameToExtend;
	/** Name of menu extensions going to asset context menu */
	static FName ConcertAssetActionsMenuOwnerName;
	/** Name of the new section in the asset action menu*/
	static FName ConcertAssetActionsSectionName;
	/** Name of entry added to asset context menu */
	static FName ConcertAssetActionsDynamicEntryName;
	/** Name of submenu added by ConcertAssetActionsDynamicEntryName */
	static FName ConcertAssetActionsSubMenuName;

public:
	FConcertWorkspaceUI();
	~FConcertWorkspaceUI();

	/** Install UI extensions for the workspace UI. */
	void InstallWorkspaceExtensions(TWeakPtr<IConcertClientWorkspace> InClientWorkspace, TWeakPtr<IConcertSyncClient> InSyncClient);

	/** Uninstall UI extensions for the workspace UI. */
	void UninstallWorspaceExtensions();

	/** Return true if the session contains changes yet to be persisted locally. */
	bool HasSessionChanges() const;

	/**
	 * Prompt User on which workspace file changes should be persisted and prepared for source control submission.
	 * @return true if the user accepted to persist, false if it was canceled.
	 */
	bool PromptPersistSessionChanges();

private:
	friend class SConcertWorkspaceLockStateIndicator;
	friend class SConcertWorkspaceLockStateTooltip;
	friend class SConcertWorkspaceModifiedByOtherIndicator;
	friend class SConcertWorkspaceModifiedByOtherTooltip;
	friend class SConcertWorkspaceSequencerToolbarExtension;

	/** On end frame callback. */
	void OnEndFrame();

	/** Check out and optionally submit files to source control. */
	bool SubmitChangelist(const FConcertPersistCommand& PersistCommand, FText& OperationMessage);

	/** Get a text description for the specified client that can be displayed in UI. */
	FText GetUserDescriptionText(const FGuid& ClientId) const;
	FText GetUserDescriptionText(const FConcertClientInfo& ClientInfo) const;

	/** Returns the avatar color corresponding to the specified client Id (including this client). */
	FLinearColor GetUserAvatarColor(const FGuid& ClientId) const;

	/** Get the local workspace's lock id. */
	FGuid GetWorkspaceLockId() const;

	/** Get the id of the client who owns the lock on a given resource. */
	FGuid GetResourceLockId(const FName InResourceName) const;
	
	/**
	 * @return whether a list of resources can be locked.
	 */
	bool CanLockResources(TArray<FName> InResourceNames) const;

	/**
	 * @return whether a list of resources can be unlocked.
	 */
	bool CanUnlockResources(TArray<FName> InResourceNames) const;

	/** Lock a list of resources. */
	void ExecuteLockResources(TArray<FName> InResourceNames);

	/**
	 * Unlock a list of resources.
	 */
	void ExecuteUnlockResources(TArray<FName> InResourceNames);

	/**
	 * View the history of the specified recourses.
	 */
	void ExecuteViewHistory(TArray<FName> InResourceNames);

	/**
	 * Returns true if the specified asset was modified by another user than the one associated to this workspace and optionally return the information about the last client who modified the resource.
	 * @param[in] AssetName The name of the asset to look up.
	 * @param[out] OutOtherClientsWithModifNum If not null, will contain how many other client(s) have modified the specified asset.
	 * @param[out] OutOtherClientsWithModifInfo If not null, will contain the other client who modified the asset, up to OtherClientsWithModifMaxFetchNum.
	 * @param[in] OtherClientsWithModifMaxFetchNum The maximum number of client info to store in OutOtherClientsWithModifInfo if the latter is not null.
	 */
	bool IsAssetModifiedByOtherClients(const FName& AssetName, int32* OutOtherClientsWithModifNum = nullptr, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo = nullptr, int32 OtherClientsWithModifMaxFetchNum = 0) const;

	/** Delegate called when a package is marked dirty. */
	void OnMarkPackageDirty(UPackage* InPackage, bool bDirty);

	/** Extends the toolbar with MU source control options */
	void ExtendToolbarWithSourceControlMenu();
	/** Extends the content browser asset context menu with MU options. */
	void ExtendAssetContextMenuOptions();
	/** Called to generate concert asset context menu. */
	void GenerateConcertAssetContextMenu(FMenuBuilder& MenuBuilder, TArray<FName> AssetObjectPaths);

	/** Delegate to generate an extra lock state indicator on content browser assets. */
	TSharedRef<SWidget> OnGenerateAssetViewLockStateIcons(const FAssetData& AssetData);

	/** Delegate to generate extra lock state tooltip on content browser assets. */
	TSharedRef<SWidget> OnGenerateAssetViewLockStateTooltip(const FAssetData& AssetData);

	/** Delegate to generate an extra "modified by other" icon on content browser assets. */
	TSharedRef<SWidget> OnGenerateAssetViewModifiedByOtherIcon(const FAssetData& AssetData);

	/** Delegate to generate the "modified by..." tooltip  on content browser assets. */
	TSharedRef<SWidget> OnGenerateAssetViewModifiedByOtherTooltip(const FAssetData& AssetData);

	/**
	 * Handle the initialization hooks for a new sequencer instance
	 *
	 * @param InSequencer	The sequencer that has just been created. Should not hold persistent shared references.
	 * @param InitParams	The initialization parameters the sequencer is going to be initialized with.
	 */
	void OnPreSequencerInit(TSharedRef<ISequencer> InSequencer, TSharedRef<ISequencerObjectChangeListener> InObjectChangeListener, const FSequencerInitParams& InitParams);

	/** Create an asset history tab filtered with a resource name. */
	static TSharedRef<SDockTab> CreateHistoryTab(const FName& ResourceName, const TSharedRef<IConcertSyncClient>& SyncClient);

	/** Workspace this is a view of. */
	TWeakPtr<IConcertClientWorkspace> ClientWorkspace;

	/** Ongoing workspace lock notification, if any. */
	TWeakPtr<SNotificationItem> EditLockedNotificationWeakPtr;

	/** The sync client for this view. */
	TWeakPtr<IConcertSyncClient> SyncClient;

	/** Delegate handle for context menu extension. */
	FDelegateHandle ContentBrowserAssetExtenderDelegateHandle;

	/* Delegate handles for asset lock state indicator extensions.  */
	TArray<FDelegateHandle> ContentBrowserAssetExtraStateDelegateHandles;

	/** Delegate handle for source control menu extension. */
	FDelegateHandle SourceControlExtensionDelegateHandle;

	/** Delegate handle for the global pre sequencer init event registered with the sequencer module */
	FDelegateHandle OnPreSequencerInitHandle;

	/** Asset history layout for asset history window tabs. */
	TSharedPtr<FTabManager::FLayout> AssetHistoryLayout;

	/** Toolbar extender for sequencer toolbar */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Set to track dirty packages per frame. */
	TSet<FName> PerFramePackageDirtyList;
};
