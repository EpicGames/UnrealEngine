// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameVisibilityColumnToggler.h"

#if WITH_EDITOR

#include "IConcertSyncClient.h"

#include "ILevelEditor.h"
#include "IWorldHierarchy.h"
#include "LevelEditor.h"
#include "WorldHierarchyColumns.h"

namespace UE::MultiUserClient
{
	FGameVisibilityColumnToggler::FGameVisibilityColumnToggler(TSharedRef<IConcertSyncClient> InMultiUserClient)
		: MultiUserClient(InMultiUserClient)
	{
		MultiUserClient->OnSyncSessionStartup().AddRaw(this, &FGameVisibilityColumnToggler::OnStartSession);
		MultiUserClient->OnSyncSessionShutdown().AddRaw(this, &FGameVisibilityColumnToggler::OnStopSession);
	}

	FGameVisibilityColumnToggler::~FGameVisibilityColumnToggler()
	{
		MultiUserClient->OnSyncSessionStartup().RemoveAll(this);
		MultiUserClient->OnSyncSessionShutdown().RemoveAll(this);
	}

	void FGameVisibilityColumnToggler::OnStartSession(const IConcertSyncClient*)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
		if (const TSharedPtr<WorldHierarchy::IWorldHierarchy> LevelsTab = LevelEditor->GetWorldHierarchy().Pin())
		{
			bHideVisibilityColumnOnSessionLeave = !LevelsTab->IsColumnVisible(WorldHierarchy::HierarchyColumns::ColumnID_GameVisibility);
			LevelsTab->SetColumnVisible(WorldHierarchy::HierarchyColumns::ColumnID_GameVisibility, true);
		}
	}

	void FGameVisibilityColumnToggler::OnStopSession(const IConcertSyncClient*)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
		if (LevelEditor.IsValid())
		{
			if (const TSharedPtr<WorldHierarchy::IWorldHierarchy> LevelsTab = LevelEditor->GetWorldHierarchy().Pin();
				LevelsTab && bHideVisibilityColumnOnSessionLeave)
			{
				LevelsTab->SetColumnVisible(WorldHierarchy::HierarchyColumns::ColumnID_GameVisibility, false);
			}
		}
	}
}

#endif
