// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;

namespace UE::MultiUserClient
{
	/**
	 * Toggles the GameVisibility column in the Levels tab when joining a MU session.
	 * 
	 * This is done to improve the UX of VP users: it allows them to show & hide levels on connected clients launched with -game,
	 * such as nDisplay nodes.
	 */
	class FGameVisibilityColumnToggler : FNoncopyable
	{
	public:
		
		FGameVisibilityColumnToggler(TSharedRef<IConcertSyncClient> InMultiUserClient);
		~FGameVisibilityColumnToggler();
		
	private:

		/** The client to which we listen for session changes to*/
		TSharedRef<IConcertSyncClient> MultiUserClient;

		/** This is false if the visibility column had been shown when the client joined the session. */
		bool bHideVisibilityColumnOnSessionLeave = true;

		void OnStartSession(const IConcertSyncClient* Client);
		void OnStopSession(const IConcertSyncClient* Client);
	};
}

#endif

