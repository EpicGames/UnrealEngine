// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMultiUserClientModule.h"

#include "ConcertClientSettings.h"
#include "ConcertLogGlobal.h"
#include "ConcertFrontendStyle.h"
#include "ConcertSyncSettings.h"
#include "ConcertWorkspaceData.h"
#include "ConcertWorkspaceUI.h"
#include "ConcertUtil.h"
#include "IConcertClient.h"
#include "IConcertModule.h"
#include "IConcertSession.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"
#include "MultiUserClientUtils.h"
#include "MultiUserReplicationStyle.h"
#include "Replication/MultiUserReplicationManager.h"
#include "Settings/MultiUserReplicationSettings.h"

#include "Misc/App.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopedSlowTask.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"

#include "ISourceControlState.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"

#include "MessageLogModule.h"
#include "Logging/MessageLog.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"

#include "EditorStyleSet.h"
#include "Delegates/IDelegateInstance.h"
#include "Interfaces/IEditorStyleModule.h"
#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Misc/GameVisibilityColumnToggler.h"
#include "Replication/Misc/DetailRowExtensionHandler.h"
#include "Sandbox/MultiUserSandboxIntegration.h"


#include "Widgets/SConcertBrowser.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SEditableTextBox.h"

#if WITH_EDITOR
	#include "FileHelpers.h"
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
	
	#include "LevelEditor.h"
	#include "ToolMenuEntry.h"
	#include "ToolMenus.h"
	#include "WorkspaceMenuStructure.h"
	#include "WorkspaceMenuStructureModule.h"
#endif

#if PLATFORM_MAC || PLATFORM_LINUX // See KillProcess() function below.
	#include <csignal> // for SIGTERM
	#include <sys/types.h> // for ::kill()
#endif


static const FName ConcertBrowserTabName("ConcertBrowser");
static const TCHAR MultiUserServerAppName[] = TEXT("UnrealMultiUserServer");
static const TCHAR MultiUserSlateServerAppName[] = TEXT("UnrealMultiUserSlateServer");

#define LOCTEXT_NAMESPACE "MultiUserClient"

LLM_DEFINE_TAG(Concert_MultiUserClient);

namespace MultiUserClientUtil
{

void KillProcess(uint32 ProcessID)
{
#if PLATFORM_MAC || PLATFORM_LINUX
	// NOTE: On Mac OS and Linux, the OpenProcess() function is not implemented properly. On Mac NSTask should be converted to POSIX and on Linux the
	//       POSIX implementation is incomplete. For the moment, we were recommended to use the POSIX API directly.
	::kill(static_cast<pid_t>(ProcessID), SIGTERM);
#else
	FProcHandle ProcHandle = FPlatformProcess::OpenProcess(ProcessID);
	FPlatformProcess::TerminateProc(ProcHandle);
	FPlatformProcess::CloseProc(ProcHandle);
#endif
}

// Validation task error codes;
constexpr uint32 GenericWorkspaceValidationErrorCode = 100;
constexpr uint32 SourceControlValidationGenericErrorCode = 110;
constexpr uint32 SourceControlValidationCancelErrorCode = 111;
constexpr uint32 SourceControlValidationErrorCode = 112;
constexpr uint32 DirtyPackageValidationErrorCode = 113;
}

/**
 * Common connection task used to validate the workspace
 */
class FConcertClientConnectionValidationTask : public IConcertClientConnectionTask
{
public:
	FConcertClientConnectionValidationTask(const UConcertClientConfig* InClientConfig)
		: ClientConfig(InClientConfig)
		, SharedState(MakeShared<FSharedAsyncState, ESPMode::ThreadSafe>())
	{}

	virtual void Abort() override
	{
		Cancel();
		SharedState.Reset(); // Always abandon the result
	}

	virtual void Tick(EConcertConnectionTaskAction TaskAction) override
	{
		// InvalidRequest is used here to signify a prompt
		if (GetStatus() == EConcertResponseCode::InvalidRequest)
		{
			if (TaskAction == EConcertConnectionTaskAction::Continue)
			{
				SharedState->Result = EConcertResponseCode::Success;
			}
			else if (TaskAction == EConcertConnectionTaskAction::Cancel)
			{
				SharedState->Result = EConcertResponseCode::Failed;
			}
		}
		else if (TaskAction == EConcertConnectionTaskAction::Cancel)
		{
			Cancel();
		}
	}

	virtual bool CanCancel() const override
	{
		// Always report we can be canceled (if we haven't been aborted) as even if the source control 
		// provider doesn't natively support cancellation, we just let it finish but disown the result
		return SharedState.IsValid();
	}

	virtual EConcertResponseCode GetStatus() const override
	{
		return SharedState ? SharedState->Result : EConcertResponseCode::Failed;
	}

	virtual FText GetPrompt() const override
	{
		return SharedState ? SharedState->PromptText : FText::GetEmpty();
	}

	virtual FConcertConnectionError GetError() const override
	{
		return SharedState ? SharedState->Error : FConcertConnectionError{ MultiUserClientUtil::GenericWorkspaceValidationErrorCode , LOCTEXT("ValidatingWorkspace_Aborted", "The workspace validation request was aborted.") };
	}

	virtual FSimpleDelegate GetErrorDelegate() const override
	{
		return FSimpleDelegate::CreateLambda([]()
		{
			FMessageLog("Concert").Open();
		});
	}

	virtual FText GetDescription() const override
	{
		return LOCTEXT("ValidatingWorkspace", "Validating Workspace...");
	}

protected:
	struct FSharedAsyncState
	{
		TArray<FString> ContentPaths;
		EConcertResponseCode Result = EConcertResponseCode::Pending;
		FText PromptText;
		FConcertConnectionError Error;
	};

	static EConcertResponseCode GetValidationModeResultOnFailure(const UConcertClientConfig* InClientConfig, bool bDoNotUseSoftAuto = false)
	{
		switch (InClientConfig->SourceControlSettings.ValidationMode)
		{
		case EConcertSourceValidationMode::Hard:
			return EConcertResponseCode::Failed;
		case EConcertSourceValidationMode::Soft:
			// InvalidRequest is used here to signify a prompt
			return EConcertResponseCode::InvalidRequest;
		case EConcertSourceValidationMode::SoftAutoProceed:
			if (bDoNotUseSoftAuto)
			{
				return EConcertResponseCode::InvalidRequest;
			}
			return EConcertResponseCode::Success;
		}
		return EConcertResponseCode::Failed;
	}

	virtual void Cancel() {}

	const UConcertClientConfig* ClientConfig;
	TSharedPtr<FSharedAsyncState, ESPMode::ThreadSafe> SharedState;
};

/**
 * Client pre-connection source control validation task
 */
class FConcertClientConnectionSourceControlValidationTask : public FConcertClientConnectionValidationTask
{
public:
	FConcertClientConnectionSourceControlValidationTask(const UConcertClientConfig* InClientConfig)
		: FConcertClientConnectionValidationTask(InClientConfig)
	{}

	virtual void Execute() override
	{
		// Source Control Validation Task
		check(SharedState && SharedState->Result == EConcertResponseCode::Pending);
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();

		// Query source control to make sure we don't have any local changes before allowing us to join a remote session
		if (SourceControlModule.IsEnabled() && SourceControlProvider.IsAvailable())
		{
			// Query all content paths
			TArray<FString> RootPaths;
			FPackageName::QueryRootContentPaths(RootPaths);
			for (const FString& RootPath : RootPaths)
			{
				const FString RootPathOnDisk = FPackageName::LongPackageNameToFilename(RootPath);
				SharedState->ContentPaths.Add(FPaths::ConvertRelativePathToFull(RootPathOnDisk));
			}

			UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
			UpdateStatusOperation->SetUpdateModifiedState(true);
			SourceControlProvider.Execute(UpdateStatusOperation.ToSharedRef(), SharedState->ContentPaths, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateStatic(&FConcertClientConnectionSourceControlValidationTask::HandleAsyncResult, SharedState.ToSharedRef(), ClientConfig));
		}
		else
		{
			SharedState->Result = EConcertResponseCode::Success;
		}
	}

private:
	virtual void Cancel() override
	{
		if (SharedState && UpdateStatusOperation)
		{
			ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
			ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();
			if (ensure(SourceControlModule.IsEnabled() && SourceControlProvider.IsAvailable()))
			{
				// Gracefully cancel the operation if we're able to
				// Otherwise just abort it by disowning the result
				if (SourceControlProvider.CanCancelOperation(UpdateStatusOperation.ToSharedRef()))
				{
					SourceControlProvider.CancelOperation(UpdateStatusOperation.ToSharedRef());
				}
				else
				{
					SharedState.Reset();
				}
			}
			UpdateStatusOperation.Reset();
		}
	}

	/** Callback for the source control result - deliberately not a member function as 'this' may be deleted while the request is in flight, so the shared state is used as a safe bridge */
	static void HandleAsyncResult(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, TSharedRef<FSharedAsyncState, ESPMode::ThreadSafe> InSharedState, const UConcertClientConfig* InClientConfig)
	{
		switch (InResult)
		{
		case ECommandResult::Succeeded:
		{
			ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
			ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();
			if (ensure(SourceControlProvider.IsEnabled() && SourceControlProvider.IsAvailable()))
			{
				FMessageLog MessageLog("Concert");
				TArray<FSourceControlStateRef> ModifiedStates = SourceControlProvider.GetCachedStateByPredicate([](const FSourceControlStateRef& StateRef)
					{
						return ((StateRef->IsAdded() || StateRef->IsDeleted() || StateRef->IsModified()) &&
							FPackageName::IsPackageFilename(StateRef->GetFilename()));
					});

				// Add warning to the multi-user message log
				for (const FSourceControlStateRef& StateRef : ModifiedStates)
				{
					MessageLog.Warning(FText::Format(LOCTEXT("ValidatingWorkspace_LocalChanges_Log", "Workspace validation failed: '{0}' contains local changes before joining a multi-user session."), FText::FromString(StateRef->GetFilename())));
				}

				if (ModifiedStates.Num() > 0)
				{
					InSharedState->Result = GetValidationModeResultOnFailure(InClientConfig);
					InSharedState->PromptText = LOCTEXT("ValidatingWorkspace_SCContinue", "Continue");
					InSharedState->Error.ErrorCode = MultiUserClientUtil::SourceControlValidationErrorCode;
					InSharedState->Error.ErrorText = InClientConfig->SourceControlSettings.ValidationMode == EConcertSourceValidationMode::Hard ?
						LOCTEXT("ValidatingWorkspace_LocalChangesHard", "This workspace has local changes. Please submit or revert these changes before attempting to connect.") :
						LOCTEXT("ValidatingWorkspace_LocalChangesSoft", "This workspace has local changes. Local changes won't be immediately available to other users.");
				}
				else
				{
					InSharedState->Result = EConcertResponseCode::Success;
				}
			}
		}
		break;
		case ECommandResult::Cancelled:
			InSharedState->Result = EConcertResponseCode::Failed;
			InSharedState->Error.ErrorCode = MultiUserClientUtil::SourceControlValidationCancelErrorCode;
			InSharedState->Error.ErrorText = LOCTEXT("ValidatingWorkspace_Canceled", "The workspace validation request was canceled.");
			break;
		default:
			InSharedState->Result = GetValidationModeResultOnFailure(InClientConfig, true);
			InSharedState->PromptText = LOCTEXT("ValidatingWorkspace_SCContinue", "Continue");
			InSharedState->Error.ErrorCode = MultiUserClientUtil::SourceControlValidationGenericErrorCode;
			InSharedState->Error.ErrorText = InClientConfig->SourceControlSettings.ValidationMode == EConcertSourceValidationMode::Hard ?
				LOCTEXT("ValidatingWorkspace_Failed", "The workspace validation request failed. Please check your revision control settings.") :
				LOCTEXT("ValidatingWorkspace_FailedSoft", "This workspace validation request failed. Please check your revision control settings. Local changes will not available to other users.");
			break;
		}
	}

	TSharedPtr<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation;
};

/**
 * Client pre-connection in memory change (dirty packages) validation task
 */
class FConcertClientConnectionDirtyPackagesValidationTask : public FConcertClientConnectionValidationTask
{
public:
	FConcertClientConnectionDirtyPackagesValidationTask(const UConcertClientConfig* InClientConfig)
		: FConcertClientConnectionValidationTask(InClientConfig)
	{}

	virtual void Execute() override
	{
		TArray<UPackage*> DirtyPackages;
#if WITH_EDITOR
		{
			FScopedSlowTask SlowTask(1.0f, LOCTEXT("ValidatingWorkspace_DirtyPackagesTask", "Checking for Dirty Packages..."));
			SlowTask.MakeDialog();

			UEditorLoadingAndSavingUtils::GetDirtyMapPackages(DirtyPackages);
			UEditorLoadingAndSavingUtils::GetDirtyContentPackages(DirtyPackages);
		}
#endif
		if (DirtyPackages.Num() > 0)
		{
			// Print which package are dirty to the message log
			FMessageLog MessageLog("Concert");
			for (UPackage* Package : DirtyPackages)
			{
				MessageLog.Warning(FText::Format(LOCTEXT("ValidatingWorkspace_InMemoryChanges_Log", "Workspace validation failed: '{0}' contains in-memory changes before joining a multi-user session."), FText::FromName(Package->GetFName())));
			}

			SharedState->Result = GetValidationModeResultOnFailure(ClientConfig);
			SharedState->PromptText = LOCTEXT("ValidatingWorkspace_DiscardChanges", "Discard");
			SharedState->Error.ErrorCode = MultiUserClientUtil::DirtyPackageValidationErrorCode;
			SharedState->Error.ErrorText = SharedState->Result == EConcertResponseCode::Failed ? 
				LOCTEXT("ValidatingWorkspace_InMemoryChanges_Fail", "This workspace has in-memory changes. Connection aborted.") : 
				LOCTEXT("ValidatingWorkspace_InMemoryChanges_Prompt", "This workspace has in-memory changes. Continue will discard changes.");
		}
		else
		{
			SharedState->Result = EConcertResponseCode::Success;
		}
	}
};

/**
 * Customize the multi-user settings to add validation and visual feedback when a user enter something invalid.
 */
class FMultiUserSettingsDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMultiUserSettingsDetails>();
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override
	{
		IDetailCategoryBuilder& ClientSettingsCategory = InDetailLayout.EditCategory(TEXT("Client Settings"), FText::GetEmpty(), ECategoryPriority::Important);

		DisplayNameHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UConcertClientConfig, ClientSettings.DisplayName));
		FString DisplayNamePropertyPath = DisplayNameHandle->GeneratePathToProperty();

		DefaultSessionNameHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UConcertClientConfig, DefaultSessionName));
		FString DefaultSessionNamePropertyPath = DefaultSessionNameHandle->GeneratePathToProperty();

		DefaultSessionToRestoreHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UConcertClientConfig, DefaultSessionToRestore));
		FString DefaultSessionToRestorePropertyPath = DefaultSessionToRestoreHandle->GeneratePathToProperty();

		DefaultSaveSessionAsHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UConcertClientConfig, DefaultSaveSessionAs));
		FString DefaultSaveSessionAsPropertyPath = DefaultSaveSessionAsHandle->GeneratePathToProperty();

		// Remove and re-add all default properties in 'Client Settings' category to keep their relative order. (Cannot do an 'in-place' customization of a row and keep its relative order)
		TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
		ClientSettingsCategory.GetDefaultProperties(DefaultProperties);
		for (TSharedRef<IPropertyHandle>& PropertyHandle : DefaultProperties)
		{
			PropertyHandle->MarkHiddenByCustomization();
			FString PropertyPath = PropertyHandle->GeneratePathToProperty();

			if (DisplayNamePropertyPath == PropertyPath) // Pointers are not the same, need to compare property paths.
			{
				CustomizeEditableTextBox(ClientSettingsCategory, DisplayNameHandle, FText::FromString(GetDefault<UConcertClientConfig>()->ClientSettings.DisplayName),
					FOnTextChanged::CreateSP(this, &FMultiUserSettingsDetails::OnDisplayNameChanged), FOnTextCommitted::CreateSP(this, &FMultiUserSettingsDetails::OnDisplayNameCommitted), DisplayNameText);
			}
			else if (DefaultSessionNamePropertyPath == PropertyPath)
			{
				CustomizeEditableTextBox(ClientSettingsCategory, DefaultSessionNameHandle, FText::FromString(GetDefault<UConcertClientConfig>()->DefaultSessionName),
					FOnTextChanged::CreateSP(this, &FMultiUserSettingsDetails::OnDefaultSessionNameChanged), FOnTextCommitted::CreateSP(this, &FMultiUserSettingsDetails::OnDefaultSessionNameCommitted), DefaultSessionNameText);
			}
			else if (DefaultSessionToRestorePropertyPath == PropertyPath)
			{
				CustomizeEditableTextBox(ClientSettingsCategory, DefaultSessionToRestoreHandle, FText::FromString(GetDefault<UConcertClientConfig>()->DefaultSessionToRestore),
					FOnTextChanged::CreateSP(this, &FMultiUserSettingsDetails::OnDefaultSessionToRestoreChanged), FOnTextCommitted::CreateSP(this, &FMultiUserSettingsDetails::OnDefaultSessionToRestoreCommitted), DefaultSessionToRestoreText);
			}
			else if (DefaultSaveSessionAsPropertyPath == PropertyPath)
			{
				CustomizeEditableTextBox(ClientSettingsCategory, DefaultSaveSessionAsHandle, FText::FromString(GetDefault<UConcertClientConfig>()->DefaultSaveSessionAs),
					FOnTextChanged::CreateSP(this, &FMultiUserSettingsDetails::OnDefaultSaveSessionAsChanged), FOnTextCommitted::CreateSP(this, &FMultiUserSettingsDetails::OnDefaultSaveSessionAsCommitted), DefaultSaveSessionAsText);
			}
			else
			{
				ClientSettingsCategory.AddProperty(PropertyHandle); // Add the default.
			}
		}
	}

	void CustomizeEditableTextBox(IDetailCategoryBuilder& CategoryBuilder, TSharedPtr<IPropertyHandle> PropertyHandle, FText InitialValue, const FOnTextChanged& OnTextChanged, const FOnTextCommitted& OnTextCommitted, TSharedPtr<SEditableTextBox>& OutTextBox)
	{
		CategoryBuilder.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SBox)
				.Padding(FMargin(0, 1, 0, 1)) // To ensure the text has same size as the default one.
				[
					SAssignNew(OutTextBox, SEditableTextBox)
					.Font(FAppStyle::GetFontStyle((TEXT("PropertyWindow.NormalFont"))))
					.Text(InitialValue)
					.SelectAllTextWhenFocused(true)
					.SelectAllTextOnCommit(true)
					.ClearKeyboardFocusOnCommit(false)
					.OnTextChanged(OnTextChanged)
					.OnTextCommitted(OnTextCommitted)
				]
			];
	}

	void OnDisplayNameChanged(const FText& Value)
	{
		DisplayNameText->SetError(ConcertSettingsUtils::ValidateDisplayName(Value.ToString()));
	}

	void OnDisplayNameCommitted(const FText& NewText, ETextCommit::Type /*CommitInfo*/)
	{
		FString DisplayName = NewText.ToString();
		if (ConcertSettingsUtils::ValidateDisplayName(DisplayName).IsEmpty())
		{
			DisplayNameHandle->SetValueFromFormattedString(DisplayName);
		}
	}

	void OnDefaultSessionNameChanged(const FText& NewText)
	{
		OnSessionNameChanged(NewText, DefaultSessionNameText);
	}

	void OnDefaultSessionNameCommitted(const FText& NewText, ETextCommit::Type /*CommitInfo*/)
	{
		OnSessionNameCommitted(NewText, DefaultSessionNameHandle);
	}

	void OnDefaultSessionToRestoreChanged(const FText& NewText)
	{
		OnSessionNameChanged(NewText, DefaultSessionToRestoreText);
	}

	void OnDefaultSessionToRestoreCommitted(const FText& NewText, ETextCommit::Type /*CommitInfo*/)
	{
		OnSessionNameCommitted(NewText, DefaultSessionToRestoreHandle);
	}

	void OnDefaultSaveSessionAsChanged(const FText& NewText)
	{
		OnSessionNameChanged(NewText, DefaultSaveSessionAsText);
	}

	void OnDefaultSaveSessionAsCommitted(const FText& NewText, ETextCommit::Type /*CommitInfo*/)
	{
		OnSessionNameCommitted(NewText, DefaultSaveSessionAsHandle);
	}

	void OnSessionNameChanged(const FText& Value, TSharedPtr<SEditableTextBox> TextBox)
	{
		FString SessionName = Value.ToString();
		if (SessionName.IsEmpty())
		{
			TextBox->SetError(FText::GetEmpty());
		}
		else
		{
			TextBox->SetError(ConcertSettingsUtils::ValidateSessionName(SessionName));
		}
	}

	void OnSessionNameCommitted(const FText& NewText, TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		// Only override the value if the session name is empty or valid.
		FString SessionName = NewText.ToString();
		if (SessionName.IsEmpty() || ConcertSettingsUtils::ValidateSessionName(SessionName).IsEmpty())
		{
			PropertyHandle->SetValueFromFormattedString(NewText.ToString());
		}
	}

private:
	TSharedPtr<IPropertyHandle> DisplayNameHandle;
	TSharedPtr<SEditableTextBox> DisplayNameText;

	TSharedPtr<IPropertyHandle> DefaultSessionNameHandle;
	TSharedPtr<SEditableTextBox> DefaultSessionNameText;

	TSharedPtr<IPropertyHandle> DefaultSessionToRestoreHandle;
	TSharedPtr<SEditableTextBox> DefaultSessionToRestoreText;

	TSharedPtr<IPropertyHandle> DefaultSaveSessionAsHandle;
	TSharedPtr<SEditableTextBox> DefaultSaveSessionAsText;
};


/**
 * Register and handle commands for button in the editor toolbar.
 */
class FConcertUICommands : public TCommands<FConcertUICommands>
{
public:
	FConcertUICommands()
		: TCommands<FConcertUICommands>("Concert", LOCTEXT("ConcertCommands", "Multi-User"), NAME_None, FConcertFrontendStyle::GetStyleSetName())
	{}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(TriggerToolbarButtonCmd, "Browse/Join/Leave", "Browse/Join/Leave Multi-User sessions depending on current state", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(OpenBrowser, "Session Browser...", "Open the Multi-User session browser", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(OpenSettings, "Multi-User Settings...", "Open the Multi-User settings", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(LaunchServer, "Launch Multi-User Server", "Launch a local Multi-User server", EUserInterfaceActionType::Button, FInputChord());
	}

	TSharedPtr<FUICommandInfo> TriggerToolbarButtonCmd; // The toolbar button triggers different commands based on the context.
	TSharedPtr<FUICommandInfo> OpenBrowser;
	TSharedPtr<FUICommandInfo> OpenSettings;
	TSharedPtr<FUICommandInfo> LaunchServer;
};

TSharedRef<SWidget> GenerateConcertMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	MenuBuilder.BeginSection("Concert", LOCTEXT("ConcertToolbarMenu", "Multi-User Menu"));
	{
		MenuBuilder.AddMenuEntry(FConcertUICommands::Get().OpenBrowser);
		MenuBuilder.AddMenuEntry(FConcertUICommands::Get().OpenSettings);
		MenuBuilder.AddMenuEntry(FConcertUICommands::Get().LaunchServer);
	}
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}


/** Implement the Multi-User module */
class FMultiUserClientModule : public IMultiUserClientModule
{
public:
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Concert_MultiUserClient);

		// Hook to the PreExit callback, needed to execute UObject related shutdowns
		FCoreDelegates::OnPreExit.AddRaw(this, &FMultiUserClientModule::HandleAppPreExit);

		// Create the client instance
		MultiUserClient = IConcertSyncClientModule::Get().CreateClient(TEXT("MultiUser"));
		MultiUserClient->GetConcertClient()->OnGetPreConnectionTasks().AddRaw(this, &FMultiUserClientModule::GetPreConnectionTasks);

		// Get the resolved config object
		UConcertClientConfig* ClientConfig = IConcertSyncClientModule::Get().ParseClientSettings(FCommandLine::Get());

		// Check if the client has the required plugin enabled to communicate with the server.
		bHasRequiredCommunicationPluginEnabled = MultiUserClientUtils::HasServerCompatibleCommunicationPluginEnabled();
		if (!bHasRequiredCommunicationPluginEnabled)
		{
			ClientConfig->bAutoConnect = false; // Prevent user from auto-connecting if there is no compatible plugin loaded to find and communicate with the server.
		}

		// Handle notification when a session is staring or ending.
		MultiUserClient->OnWorkspaceStartup().AddRaw(this, &FMultiUserClientModule::OnWorkspaceStartup);
		MultiUserClient->OnWorkspaceShutdown().AddRaw(this, &FMultiUserClientModule::OnWorkspaceShutdown);

		// Boot the client instance
		MultiUserClient->Startup(ClientConfig, EConcertSyncSessionFlags::Default_MultiUserSession);
		ReplicationManager = MakeShared<UE::MultiUserClient::Replication::FMultiUserReplicationManager>(MultiUserClient.ToSharedRef());

#if WITH_EDITOR
		if (GIsEditor)
		{
			VisibilityColumnToggler = MakeUnique<UE::MultiUserClient::FGameVisibilityColumnToggler>(MultiUserClient.ToSharedRef());
			DetailReplicationRowExtensionHandler = MakeUnique<UE::MultiUserClient::Replication::FDetailRowExtensionHandler>(
				MultiUserClient.ToSharedRef(), ReplicationManager.ToSharedRef(),
				UE::MultiUserClient::FGetConcertBrowserWidget::CreateRaw(this, &FMultiUserClientModule::GetOrOpenBrowserTabContent)
				);
			SandboxIntegration = MakeUnique<UE::MultiUserClient::FMultiUserSandboxIntegration>(MultiUserClient.ToSharedRef());
		}
#endif
		
		// Hook UI elements in the tool bar (and setup commands).
		RegisterUI();
		RegisterLogging();
	}

	virtual void ShutdownModule() override
	{
#if WITH_EDITOR
		VisibilityColumnToggler.Reset();
		DetailReplicationRowExtensionHandler.Reset();
#endif
		
		// Unhook AppPreExit and call it
		FCoreDelegates::OnPreExit.RemoveAll(this);
		HandleAppPreExit();

		OpenBrowserConsoleCommand.Reset();
		OpenSettingsConsoleCommand.Reset();
		DefaultConnectConsoleCommand.Reset();
		DisconnectConsoleCommand.Reset();
	}

	virtual TSharedPtr<IConcertSyncClient> GetClient() const override
	{
		return MultiUserClient;
	}

	virtual UE::MultiUserClient::IMultiUserReplication* GetReplication() const override
	{
		return ReplicationManager.Get();
	}

	/**
	 * Invokes the Multi-User browser tab
	 */
	virtual void OpenBrowser() override
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId(ConcertBrowserTabName));
	}

	/**
	 * Hot-links to Concert Settings.
	 */
	virtual void OpenSettings() override
	{
#if WITH_EDITOR
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->ShowViewer("Project", "Plugins", "Concert");
		}
#endif
	}

	/**
	 * Connect to the default connection setup
	 * @return true if the connection process started
	 */
	virtual bool DefaultConnect() override
	{
		IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();
		if (ConcertClient->CanAutoConnect() && ConcertClient->GetSessionConnectionStatus() == EConcertConnectionStatus::Disconnected)
		{
			ConcertClient->StartAutoConnect();
			return true;
		}
		return false;
	}

	/**
	 * Disconnect from the current session and warn for session changes if session owner.
	 */
	void DisconnectSession()
	{
		DisconnectSession(/*bAlwaysAskConfirmation*/false);
	}

	/**
	 * Disconnect from the current session and warn for session changes if session owner.
	 */
	virtual bool DisconnectSession(bool bAlwaysAskConfirmation) override
	{
		IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();
		if (TSharedPtr<IConcertClientSession> CurrentSession = ConcertClient->GetCurrentSession())
		{
			EAppReturnType::Type RetType = EAppReturnType::No; // Default: Don't persist and leave.

			// if the session owner disconnects from the session, warn them about session changes.
			if (ConcertClient->IsOwnerOf(CurrentSession->GetSessionInfo()) && WorkspaceFrontend->HasSessionChanges())
			{
				RetType = FMessageDialog::Open(EAppMsgType::YesNoCancel, EAppReturnType::No, LOCTEXT("OwnerDisconnectWarn", "You are about to leave a session containing changes. Do you want to persist the changes?"));
				if (RetType == EAppReturnType::Yes)
				{
					bool UserCanceledPersist = !WorkspaceFrontend->PromptPersistSessionChanges();
					if (UserCanceledPersist) // The user changed their mind, they don't want to persist.
					{
						return false; // Cancel the disconnection as well.
					}
				}
			}
			else if (bAlwaysAskConfirmation) // User was not prompted to persist session change(s), but a confirmation was requested.
			{
				RetType = FMessageDialog::Open(EAppMsgType::YesNoCancel, EAppReturnType::No, FText::Format(LOCTEXT("LeaveSessionConfirmation", "You are about to leave {0} session. Do you want to continue?"), FText::AsCultureInvariant(CurrentSession->GetSessionInfo().SessionName)));
				if (RetType != EAppReturnType::Yes)
				{
					return false; // User doesn't want to leave the session.
				}
			}

			bool bCanDisconnect = (RetType != EAppReturnType::Cancel);
			if (bCanDisconnect && CurrentSession->GetConnectionStatus() != EConcertConnectionStatus::Disconnected)
			{
				ConcertClient->DisconnectSession();
			}
		}
		else if (ConcertClient->IsAutoConnecting())
		{
			ConcertClient->StopAutoConnect();
		}

		return true; // The client is disconnected.
	}

	void RegisterTabSpawner(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup)
	{
		if (bHasRegisteredTabSpawners)
		{
			UnregisterTabSpawner();
		}
		bHasRegisteredTabSpawners = true;

		FTabSpawnerEntry& BrowserSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ConcertBrowserTabName,
			FOnSpawnTab::CreateRaw(this, &FMultiUserClientModule::SpawnConcertBrowserTab))
			.SetIcon(FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), TEXT("Concert.MultiUser")))
			.SetDisplayName(LOCTEXT("BrowserTabTitle", "Multi-User Browser"))
			.SetTooltipText(LOCTEXT("BrowserTooltipText", "Open the Multi-User session browser"))
			.SetMenuType(ETabSpawnerMenuType::Enabled);

		if (WorkspaceGroup.IsValid())
		{
			BrowserSpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
		}
		
#if WITH_EDITOR
		if (SandboxIntegration)
		{
			SandboxIntegration->GetEntryPoint()->OnSummonProviderUI().AddLambda([]
			{
				FGlobalTabmanager::Get()->TryInvokeTab(ConcertBrowserTabName);
			});
		}
#endif
	}

	void UnregisterTabSpawner()
	{
		bHasRegisteredTabSpawners = false;
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ConcertBrowserTabName);
	}

private:
	void OnWorkspaceStartup(const TSharedPtr<IConcertClientWorkspace>& InWorkspace)
	{
		//InWorkspace.OnPackageTooLargeError().AddRaw(this, &FMultiUserClientModule::HandlePackageTooLargeError);
	}

	void OnWorkspaceShutdown(const TSharedPtr<IConcertClientWorkspace>& InWorkspace)
	{
		//InWorkspace.OnPackageTooLargeError().RemoveAll(this);
	}

	void HandlePackageTooLargeError(const FConcertPackageInfo& PackageInfo, int64 PackageSize, int64 MaxPackageSizeSupported)
	{
		if (PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Added)
		{
			UE_LOGF(LogConcert, Warning, "The newly added package '%ls' was too large (%ls) to be synchronized on other Multi-User clients. It is recommended to save the package, leave and persist the session, submit the file to the revision control, sync all clients to this submit and create a new session.", *PackageInfo.PackageName.ToString(), *FText::AsMemory(PackageSize).ToString());
		}
		else if (PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Saved)
		{
			UE_LOGF(LogConcert, Warning, "The saved package '%ls' was too large (%ls) to be synchronized on other Multi-User clients. It is recommended to leave and persist the session, submit the file to the revision control, sync all clients to this submit and create a new session.", *PackageInfo.PackageName.ToString(), *FText::AsMemory(PackageSize).ToString());
		}
		else if (PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Renamed)
		{
			UE_LOGF(LogConcert, Warning, "The renamed package '%ls' was too large (%ls) to be synchronized on other Multi-User clients. It is recommended to leave and persist the session, submit the file to the revision control, sync all clients to this submit and create a new session.", *PackageInfo.NewPackageName.ToString(), *FText::AsMemory(PackageSize).ToString());
		}
		else // Other events are not expected to trigger a 'too large' error, but log it if something unexpected slips in.
		{
			UE_LOGF(LogConcert, Error, "A package event on package '%ls' was unexpectedly ignored because the package was too large (%ls).", *PackageInfo.PackageName.ToString(), *FText::AsMemory(PackageSize).ToString());
		}
	}

	void RegisterUI()
	{
		bHasRegisteredTabSpawners = false;

		// Initialize Style
		FConcertFrontendStyle::Initialize();
		UE::MultiUserClient::FMultiUserReplicationStyle::Initialize();

		// Multi-User front end currently relies on EditorStyle being loaded
		FModuleManager::LoadModuleChecked<IEditorStyleModule>("EditorStyle");

#if WITH_EDITOR
		RegisterTabSpawner(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory());

		// Register Workspace view
		RegisterWorkspaceUI();
		RegisterSettings();

		// Extend toolbar
		if (MultiUserClient->GetConcertClient()->GetConfiguration()->bInstallEditorToolbarButton)
		{
			AddEditorToolbarButton();
		}
#else
		RegisterTabSpawner(TSharedPtr<FWorkspaceItem>());
#endif

		// Expose commands to the console.
		RegisterConsoleCommands();
	}

	void UnregisterUI()
	{
		UnregisterTabSpawner();

#if WITH_EDITOR
		FConcertFrontendStyle::Shutdown();
		UE::MultiUserClient::FMultiUserReplicationStyle::Shutdown();
		
		UnregisterWorkspaceUI();
		UnregisterSettings();
		RemoveEditorToolbarButton();
#endif
	}

	void RegisterLogging()
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		FMessageLogInitializationOptions MessageLogOptions;
		MessageLogOptions.bShowPages = true;
		MessageLogOptions.bAllowClear = true;
		MessageLogOptions.MaxPageCount = 5;
		MessageLogModule.RegisterLogListing(TEXT("Concert"), LOCTEXT("ConcertLogLabel", "Multi-User"));
	}

	void UnregisterLogging()
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing(TEXT("Concert"));
	}

	void RegisterConsoleCommands() // Console commands can be triggered from 'Cmd' text box found at the bottom of the 'Output Log' windows for example.
	{
#if WITH_EDITOR
		if (!IsRunningGame()) // Prevent user from spawning Concert UI using the console command in -game. The Editor styles are not loaded and this may display incorrectly some UI element.
		{
			OpenBrowserConsoleCommand = MakeUnique<FAutoConsoleCommand>(
				TEXT("Concert.OpenBrowser"),
				TEXT("Open the Multi-User session browser"),
				FExecuteAction::CreateRaw(this, &FMultiUserClientModule::OpenBrowser)
				);

			OpenSettingsConsoleCommand = MakeUnique<FAutoConsoleCommand>(
				TEXT("Concert.OpenSettings"),
				TEXT("Open the Multi-User settings"),
				FExecuteAction::CreateRaw(this, &FMultiUserClientModule::OpenSettings)
				);
		}
#endif

		DefaultConnectConsoleCommand = MakeUnique<FAutoConsoleCommand>(
			TEXT("Concert.DefaultConnect"),
			TEXT("Connect to the default Multi-User session (as defined in the Multi-User settings)"),
			FExecuteAction::CreateLambda([this]() { DefaultConnect(); })
			);

		DisconnectConsoleCommand = MakeUnique<FAutoConsoleCommand>(
			TEXT("Concert.Disconnect"),
			TEXT("Disconnect from the current session"),
			FExecuteAction::CreateRaw(this, &FMultiUserClientModule::DisconnectSession)
			);
	}

	// Module shutdown is dependent on the UObject system which is currently shutdown on AppExit
	void HandleAppPreExit()
	{
		// if UObject system isn't initialized, skip shutdown
		if (!UObjectInitialized())
		{
			return;
		}

		UnregisterUI();
		UnregisterLogging();

		if (MultiUserClient)
		{
			MultiUserClient->OnWorkspaceStartup().RemoveAll(this);
			MultiUserClient->OnWorkspaceShutdown().RemoveAll(this);

			MultiUserClient->GetConcertClient()->OnGetPreConnectionTasks().RemoveAll(this);
			MultiUserClient->Shutdown();
			MultiUserClient.Reset();
		}
	}

	void GetPreConnectionTasks(const IConcertClient& InClient, TArray<TUniquePtr<IConcertClientConnectionTask>>& OutTasks)
	{
		const TCHAR* CommandLine = FCommandLine::Get();
		bool bSkipSCChecks = FParse::Param(CommandLine, TEXT("-CONCERTSKIPSOURCECONTROLCHECKS"));
		FParse::Bool(CommandLine, TEXT("-CONCERTSKIPSOURCECONTROLCHECKS="), bSkipSCChecks);

		if (!bSkipSCChecks)
		{
			OutTasks.Emplace(MakeUnique<FConcertClientConnectionSourceControlValidationTask>(InClient.GetConfiguration()));
		}
		else
		{
			UE_LOGF(LogConcert, Display, "Source control validation task was bypassed.");
		}
		OutTasks.Emplace(MakeUnique<FConcertClientConnectionDirtyPackagesValidationTask>(InClient.GetConfiguration()));
	}

	/** Returns the toolbar icon title. */
	FText GetToolbarButtonIconTitle() const
	{
		IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();

		// If the user is connected, clicking the button will 'leave' the session.
		if (ConcertClient->GetSessionConnectionStatus() == EConcertConnectionStatus::Connected)
		{
			return LOCTEXT("Leave", "Leave");
		}

		// If the user is auto connecting, clicking the button will cancel the request.
		if (ConcertClient->IsAutoConnecting())
		{
			return LOCTEXT("Cancel", "Cancel");
		}

		// If auto-connect is configured, clicking the button will start a routine to join the session.
		if (ConcertClient->CanAutoConnect())
		{
			return LOCTEXT("Join", "Join");
		}

		// The auto-connect is not configured, clicking the button will open (or flash) the multi-users browser.
		return LOCTEXT("Browse", "Browse");
	}

	/** Return the toolbar icon. */
	FSlateIcon GetToolbarButtonIcon() const
	{
		IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();
		if (ConcertClient->GetSessionConnectionStatus() == EConcertConnectionStatus::Connected)
		{
			return FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Leave", "Concert.Leave.Small");
		}
		else if (ConcertClient->IsAutoConnecting())
		{
			return FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Cancel", "Concert.Cancel.Small");
		}
		else if (ConcertClient->CanAutoConnect())
		{
			return FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Join", "Concert.Join.Small");
		}
		
		return FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), "Concert.Browse", "Concert.Browse.Small");
	}

	/** Return the toolbar icon tooltip. */
	FText GetToolbarButtonTooltip() const
	{
		IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();

		// If the user is connected, clicking the button will 'leave' the session.
		if (ConcertClient->GetSessionConnectionStatus() == EConcertConnectionStatus::Connected)
		{
			return FText::Format(LOCTEXT("LeaveTooltip", "Leave '{0}' session."), FText::FromString(ConcertClient->GetCurrentSession()->GetSessionInfo().SessionName));
		}

		// If the user is auto connecting, clicking the button will cancel the request.
		if (ConcertClient->IsAutoConnecting())
		{
			const UConcertClientConfig* ClientConfig = ConcertClient->GetConfiguration();
			check(ClientConfig);
			return FText::Format(LOCTEXT("CancelJoinTooltip", "Cancel joining default session '{0}'."), FText::FromString(ClientConfig->DefaultSessionName));
		}

		// If auto-connect is configured, clicking the button will start a routine to join the session.
		if (ConcertClient->CanAutoConnect())
		{
			if (bHasRequiredCommunicationPluginEnabled)
			{
				const UConcertClientConfig* ClientConfig = ConcertClient->GetConfiguration();
				check(ClientConfig);
				return FText::Format(LOCTEXT("JoinTooltip", "Join default session '{0}'"), FText::FromString(ClientConfig->DefaultSessionName));
			}
			else
			{
				return MultiUserClientUtils::GetNoCompatibleCommunicationPluginEnabledText();
			}
		}

		// The auto-connect is not configured, clicking the button will open (or flash) the multi-users browser.
		return LOCTEXT("BrowseTooltip", "Open session browser");
	}

	bool IsToolbarButtonEnabled()
	{
		IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();
		if (ConcertClient->CanAutoConnect())
		{
			return bHasRequiredCommunicationPluginEnabled;
		}
		return true;
	}

	void OnToolbarButtonClicked()
	{
		if (bHasRequiredCommunicationPluginEnabled)
		{
			IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();

			// If the user is connected, clicking the button will 'leave' the session.
			if (ConcertClient->GetSessionConnectionStatus() == EConcertConnectionStatus::Connected)
			{
				DisconnectSession();
				return;
			}

			// If the user is auto connecting, clicking the button will cancel the request.
			if (ConcertClient->IsAutoConnecting())
			{
				ConcertClient->StopAutoConnect();
				return;
			}

			// If auto-connect is configured, clicking the button will start a routine to join the session.
			if (ConcertClient->CanAutoConnect())
			{
				ConcertClient->StartAutoConnect();
				return;
			}
		}

		// The auto-connect is not configured, clicking the button will open (or flash) the multi-users browser.
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId(ConcertBrowserTabName));
	}

	TSharedPtr<SConcertBrowser> GetOrOpenBrowserTabContent() const
	{
		// If the tab is already open, it's ok that its flashed. Some UI code is getting the content of the tab to show something in it.
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId(ConcertBrowserTabName));
		return BrowserTab.Pin();
	}
	
	/**
	 * Creates a new Concert Browser front-end tab.
	 *
	 * @param SpawnTabArgs The arguments for the tab to spawn.
	 * @return The spawned tab.
	 */
	TSharedRef<SDockTab> SpawnConcertBrowserTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedPtr<UE::FileSandboxUI::IExternalSandboxActiveViewModel> ExternalSandboxViewModel;
#if WITH_EDITOR
		ExternalSandboxViewModel = SandboxIntegration ? SandboxIntegration->GetExternalSandboxViewModel().ToSharedPtr() : nullptr;
#endif
		
		return SNew(SDockTab)
			.TabRole(NomadTab)
			[
				SAssignNew(BrowserTab, SConcertBrowser, 
					MultiUserClient.ToSharedRef(), ReplicationManager.ToSharedRef(), ExternalSandboxViewModel
					)
			];
	}

	static FString GetConfiguredMultiUserServerExePathname()
	{
		return GetMultiUserServerExePathname(GetMutableDefault<UConcertClientConfig>()->ServerType);
	}

	static FString GetMultiUserServerExePathname(const EConcertServerType ServerType)
	{
		check(ServerType == EConcertServerType::Console || ServerType == EConcertServerType::Slate);
		
		const FString MultiUserServerName(ServerType == EConcertServerType::Console ? MultiUserServerAppName : MultiUserSlateServerAppName);
		FString ServerPath = FPlatformProcess::GenerateApplicationPath(MultiUserServerName, FApp::GetBuildConfiguration());

		// Validate it exists and fall back to development if it doesn't.
		if (!IFileManager::Get().FileExists(*ServerPath) && FApp::GetBuildConfiguration() != EBuildConfiguration::Development)
		{
			ServerPath = FPlatformProcess::GenerateApplicationPath(MultiUserServerName, EBuildConfiguration::Development);
		}

		return ServerPath;
	}

	virtual bool IsConcertServerRunning() override
	{
		// Check both because user can change config
		const FString ConsoleServerPath = GetMultiUserServerExePathname(EConcertServerType::Console);
		const FString SlateServerPath = GetMultiUserServerExePathname(EConcertServerType::Slate);
		return FPlatformProcess::IsApplicationRunning(*FPaths::GetCleanFilename(ConsoleServerPath))
			|| FPlatformProcess::IsApplicationRunning(*FPaths::GetCleanFilename(SlateServerPath));
	}

	/**
	 * Launch a Concert collaboration server on the local machine.
	 */
	virtual TOptional<FProcHandle> LaunchConcertServer(TOptional<FServerLaunchOverrides> LaunchOverrides = {}) override
	{
		FAsyncTaskNotificationConfig NotificationConfig;
		NotificationConfig.bKeepOpenOnFailure = true;
		NotificationConfig.TitleText = LOCTEXT("LaunchingUnrealMultiUserServer", "Launching Unreal Multi-User Server...");
		NotificationConfig.LogCategory = &LogConcert;

		FAsyncTaskNotification Notification(NotificationConfig);

		// Log a warning if the UDP Messaging plugin is not enabled.
		if (!bHasRequiredCommunicationPluginEnabled)
		{
			MultiUserClientUtils::LogNoCompatibleCommunicationPluginEnabled();
		}

		// Find concert server location for our build configuration
		FString ServerPath = GetConfiguredMultiUserServerExePathname();

		FText LaunchMultiUserErrorTitle = LOCTEXT("LaunchUnrealMultiUserServerErrorTitle", "Failed to Launch the Unreal Multi-User Server");
		if (!IFileManager::Get().FileExists(*ServerPath))
		{
			Notification.SetComplete(
				LaunchMultiUserErrorTitle,
				LOCTEXT("LaunchUnrealMultiUserServerError_ExecutableMissing", "Could not find the executable. Have you compiled the Unreal Multi-User Server?"), 
				false
				);
			return TOptional<FProcHandle>();
		}

		// Validate we do not have it running locally 
		FString ServerAppName = FPaths::GetCleanFilename(ServerPath);
		if (FPlatformProcess::IsApplicationRunning(*ServerAppName))
		{
			Notification.SetComplete(
				LaunchMultiUserErrorTitle,
				LOCTEXT("LaunchUnrealMultiUserServerError_AlreadyRunning", "An Unreal Multi-User Server instance is already running."), 
				false
				);
			return TOptional<FProcHandle>();
		}

		FString CmdLineArgs = GenerateConcertServerCommandLine(LaunchOverrides);
		FProcHandle ProcHandle = FPlatformProcess::CreateProc(*ServerPath, *CmdLineArgs, true, false, false, nullptr, 0, nullptr, nullptr, nullptr);
		if (ProcHandle.IsValid())
		{
			Notification.SetComplete(
				LOCTEXT("LaunchedUnrealMultiUserServer", "Launched Unreal Multi-User Server"), FText(), true);

			return ProcHandle;
		}
		else // Very unlikely in practice, but possible in theory.
		{
			Notification.SetComplete(
				LaunchMultiUserErrorTitle,
				LOCTEXT("LaunchUnrealMultiUserServerError_InvalidHandle", "Failed to create the Multi-User Server process."),
				false);
		}
		
		return TOptional<FProcHandle>();
	}

	void ShutdownConcertServer() override
	{
		auto GetCleanPath = [](EConcertServerType Type) -> FString
		{
			FString ServerPath = GetMultiUserServerExePathname(Type);
			ServerPath = FPaths::ConvertRelativePathToFull(ServerPath);
			FPaths::NormalizeDirectoryName(ServerPath);
			return ServerPath;
		};
		const FString ConsoleServerPath = GetCleanPath(EConcertServerType::Console);
		const FString SlateServerPath = GetCleanPath(EConcertServerType::Slate);

		FPlatformProcess::FProcEnumerator ProcIter;
		while (ProcIter.MoveNext())
		{
			FPlatformProcess::FProcEnumInfo ProcInfo = ProcIter.GetCurrent();
			FString Candidate = ProcInfo.GetFullPath();
			FPaths::NormalizeDirectoryName(Candidate);
			if (Candidate == ConsoleServerPath || Candidate == SlateServerPath)
			{
				MultiUserClientUtil::KillProcess(ProcInfo.GetPID());
				// Continue -> Will close all servers instances.
			}
		}
	}

	void RegisterWorkspaceUI()
	{
		WorkspaceFrontend = MakeShared<FConcertWorkspaceUI>();
		MultiUserClient->OnWorkspaceStartup().AddRaw(this, &FMultiUserClientModule::InstallWorkspaceUI);
		MultiUserClient->OnWorkspaceShutdown().AddRaw(this, &FMultiUserClientModule::UninstallWorkspaceUI);
	}

	void UnregisterWorkspaceUI()
	{
		WorkspaceFrontend.Reset();
		if (MultiUserClient.IsValid())
		{
			MultiUserClient->OnWorkspaceStartup().RemoveAll(this);
			MultiUserClient->OnWorkspaceShutdown().RemoveAll(this);
		}
	}

	void InstallWorkspaceUI(const TSharedPtr<IConcertClientWorkspace>& InClientWorkspace)
	{
		if (WorkspaceFrontend.IsValid())
		{
			WorkspaceFrontend->InstallWorkspaceExtensions(InClientWorkspace, MultiUserClient);
		}
	}

	void UninstallWorkspaceUI(const TSharedPtr<IConcertClientWorkspace>& /* InClientWorkspace */)
	{
		if (WorkspaceFrontend.IsValid())
		{
			WorkspaceFrontend->UninstallWorspaceExtensions();
		}
	}

#if WITH_EDITOR
	void AddEditorToolbarButton() // The button in the Level editor toolbar.
	{
		if (GIsEditor)
		{
			// Setup Concert Toolbar
			FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

			// Register command list
			FConcertUICommands::Register();
			const TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();

			// Connect to the default server and session
			CommandList->MapAction(FConcertUICommands::Get().TriggerToolbarButtonCmd,
				FExecuteAction::CreateRaw(this, &FMultiUserClientModule::OnToolbarButtonClicked),
				FCanExecuteAction::CreateRaw(this, &FMultiUserClientModule::IsToolbarButtonEnabled)
			);

			// Browser menu
			CommandList->MapAction(FConcertUICommands::Get().OpenBrowser,
				FExecuteAction::CreateRaw(this, &FMultiUserClientModule::OpenBrowser)
			);

			// Concert Settings
			CommandList->MapAction(FConcertUICommands::Get().OpenSettings,
				FExecuteAction::CreateRaw(this, &FMultiUserClientModule::OpenSettings)
			);

			// Launch Server
			CommandList->MapAction(FConcertUICommands::Get().LaunchServer,
				FExecuteAction::CreateLambda([this](){ LaunchConcertServer();} )
			);

			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
			FToolMenuSection& Section = Menu->FindOrAddSection("Concert");

			FToolMenuEntry ConcertButtonEntry = FToolMenuEntry::InitToolBarButton(
				FConcertUICommands::Get().TriggerToolbarButtonCmd,
				TAttribute<FText>::Create([this]() { return GetToolbarButtonIconTitle(); }),
				TAttribute<FText>::Create([this]() { return GetToolbarButtonTooltip(); }),
				TAttribute<FSlateIcon>::Create([this]() { return GetToolbarButtonIcon(); })
			);
			ConcertButtonEntry.SetCommandList(CommandList);

			const FToolMenuEntry ConcertComboEntry = FToolMenuEntry::InitComboButton(
			"ConcertToolbarMenu",
			FUIAction(),
			FOnGetContent::CreateStatic(&GenerateConcertMenuContent, CommandList.ToSharedRef()),
			LOCTEXT("ConcertToolbarMenu_Label", "Multi-User Utilities"),
			LOCTEXT("ConcertToolbarMenu_Tooltip", "Multi-User Commands"),
			FSlateIcon(),
			true //bInSimpleComboBox
			);

			Section.AddEntry(ConcertButtonEntry);
			Section.AddEntry(ConcertComboEntry);
		}
	}

	void RemoveEditorToolbarButton()
	{
		if (GIsEditor)
		{
			if (UObjectInitialized())
			{
				UToolMenus::Get()->RemoveSection("LevelEditor.LevelEditorToolBar.User", "Concert");
			}
			
			FConcertUICommands::Unregister();
		}
	}

	void RegisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			// TODO: make only one section for both settings objects
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "Concert",
				LOCTEXT("ConcertFrontendSettingsName", "Multi-User Editing"),
				LOCTEXT("ConcertFrontendSettingsDescription", "Configure the Multi-User settings."),
				GetMutableDefault<UConcertClientConfig>()
				);

			if (SettingsSection.IsValid())
			{
				SettingsSection->OnModified().BindRaw(this, &FMultiUserClientModule::HandleSettingsSaved);
			}

			SettingsModule->RegisterSettings("Project", "Plugins", "Concert Sync",
				LOCTEXT("ConcertFrontendSyncSettingsName", "Multi-User Transactions"),
				LOCTEXT("ConcertFrontendSyncSettingsDescription", "Configure the Multi-User Transactions settings."),
				GetMutableDefault<UConcertSyncConfig>()
				);

			SettingsModule->RegisterSettings("Project", "Plugins", "Multi User",
				LOCTEXT("MultiUserReplicationSettingsName", "Multi-User Replication"),
				LOCTEXT("MultiUserReplicationSettingsDescription", "Configure the Multi-User Replication settings."),
				UMultiUserReplicationSettings::Get()
				);

			// Register a special layout to validate user input when configuring the settings.
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomClassLayout(UConcertClientConfig::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FMultiUserSettingsDetails::MakeInstance));
		}
	}

	void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "Concert");
			SettingsModule->UnregisterSettings("Project", "Plugins", "Concert Sync");

			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout(UConcertClientConfig::StaticClass()->GetFName());
		}
	}

	bool HandleSettingsSaved()
	{
		IConcertClientRef ConcertClient = MultiUserClient->GetConcertClient();
		ConcertClient->Configure(GetDefault<UConcertClientConfig>());
		return true;
	}
#endif

private:
	FString GenerateConcertServerCommandLine(TOptional<FServerLaunchOverrides> LaunchOverrides) const
	{
		// @note: Those settings are UDP/QUIC Messaging specific

		FString CmdLine;
		FConfigFile* EngineConfig = GConfig ? GConfig->FindConfigFileWithBaseName(FName(TEXT("Engine"))) : nullptr;
		if (EngineConfig)
		{
			TArray<FString> Settings;
			FString Setting;

			// Unicast endpoint setting
			EngineConfig->GetString(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("UnicastEndpoint"), Setting);

			// if the unicast endpoint port is bound, add 1 to the port for server
			const UConcertClientConfig* ClientConfig = GetDefault<UConcertClientConfig>();
			if (Setting.ParseIntoArray(Settings, TEXT(":"), false) == 2)
			{
				if (ClientConfig && ClientConfig->ClientSettings.ServerPort != 0)
				{
					Setting = FString::Printf(TEXT("%s:%d"), *Settings[0], ClientConfig->ClientSettings.ServerPort);
				}
				else if (Settings[1] != TEXT("0"))
				{
					int32 Port = FCString::Atoi(*Settings[1]);
					Port += 1;
					Setting = FString::Printf(TEXT("%s:%d"), *Settings[0], Port);
				}
			}
			CmdLine = TEXT("-UDPMESSAGING_TRANSPORT_UNICAST=") + Setting;

			// Multicast endpoint setting
			EngineConfig->GetString(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("MulticastEndpoint"), Setting);
			CmdLine += TEXT(" -UDPMESSAGING_TRANSPORT_MULTICAST=") + Setting;

			// Static endpoints setting
			Settings.Empty(1);
			EngineConfig->GetArray(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("StaticEndpoints"), Settings);
			if (Settings.Num() > 0)
			{
				CmdLine += TEXT(" -UDPMESSAGING_TRANSPORT_STATIC=");
				CmdLine += Settings[0];
				for (int32 i = 1; i < Settings.Num(); ++i)
				{
					CmdLine += ',';
					CmdLine += Settings[i];
				}
			}

			// Set the Quic unicast endpoint
			FString QuicUnicastEndpoint;
			EngineConfig->GetString(TEXT("/Script/QuicMessaging.QuicMessagingSettings"), TEXT("UnicastEndpoint"), QuicUnicastEndpoint);
			CmdLine += TEXT(" -QUICMESSAGING_TRANSPORT_UNICAST=") + QuicUnicastEndpoint;

			if (LaunchOverrides)
			{
				if (!LaunchOverrides->ServerName.IsEmpty())
				{
					CmdLine += FString::Printf(TEXT(" -CONCERTSERVER=\"%s\""), *(LaunchOverrides->ServerName));
				}
			}
		}

		// Set Quic to operate in server mode
		CmdLine += TEXT(" -QUICMESSAGING_TRANSPORT_ISCLIENT=False");

		return CmdLine;
	}

	TSharedPtr<IConcertSyncClient> MultiUserClient;
	/** Interacts with the replication system on behalf of Multi-User. */
	TSharedPtr<UE::MultiUserClient::Replication::FMultiUserReplicationManager> ReplicationManager;

	/** The content of the MU tab, if open. Invalid or stale if closed. */
	TWeakPtr<SConcertBrowser> BrowserTab;

#if WITH_EDITOR
	/** Toggles the game visibility column in the Levels tab when joining & leaving sessions. */
	TUniquePtr<UE::MultiUserClient::FGameVisibilityColumnToggler> VisibilityColumnToggler;

	/** Adds UI to Details panel rows for quickly adding properties to replication. */
	TUniquePtr<UE::MultiUserClient::Replication::FDetailRowExtensionHandler> DetailReplicationRowExtensionHandler;
	
	/** Responsible for registering Multi User with the sandbox system. */
	TUniquePtr<UE::MultiUserClient::FMultiUserSandboxIntegration> SandboxIntegration;
#endif

	/** True if the tab spawners have been registered for this module */
	bool bHasRegisteredTabSpawners = false;

	/** Tracks whether the client has enabled a transport plugin compatible with the server, so they can communicate. */
	bool bHasRequiredCommunicationPluginEnabled = true;

	/** UI view and commands on the Concert client workspace. */
	TSharedPtr<FConcertWorkspaceUI> WorkspaceFrontend;

	/** Console command for opening the Concert Browser */
	TUniquePtr<FAutoConsoleCommand> OpenBrowserConsoleCommand;

	/** Console command for opening the Concert Settings */
	TUniquePtr<FAutoConsoleCommand> OpenSettingsConsoleCommand;

	/** Console command for connecting to the default Concert session */
	TUniquePtr<FAutoConsoleCommand> DefaultConnectConsoleCommand;

	/** Console command for disconnecting from the current Concert session */
	TUniquePtr<FAutoConsoleCommand> DisconnectConsoleCommand;
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMultiUserClientModule, MultiUserClient);
