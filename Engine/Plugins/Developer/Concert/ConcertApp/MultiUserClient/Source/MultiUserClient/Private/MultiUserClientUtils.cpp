// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserClientUtils.h"

#include "Interfaces/IPluginManager.h"
#include "ConcertLogGlobal.h"

#define LOCTEXT_NAMESPACE "MultiUserClientUtils"

namespace MultiUserClientUtils
{

bool IsUdpMessagingPluginEnabled()
{
	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("UdpMessaging")))
	{
		return Plugin->IsEnabled();
	}
	return false;
}

bool HasServerCompatibleCommunicationPluginEnabled()
{
	// Concert needs and is only tested using UDP.
	static bool UdpMessagingAvailable = IsUdpMessagingPluginEnabled();
	return UdpMessagingAvailable;
}

TArray<FString> GetServerCompatibleCommunicationPlugins()
{
	// The name as displayed/searchable in the Plugin panel.
	return TArray<FString>{ FString(TEXT("Udp Messaging")) };
	//return TArray<FString>{ FString(TEXT("Udp Messaging")), FString(TEXT("Tcp Messaging")) }; // To test the log/messages in case Tcp gets supported in the future.
}

FText GetNoCompatibleCommunicationPluginEnabledText()
{
	TArray<FString> CompatibleTransportPlugins;
	CompatibleTransportPlugins = GetServerCompatibleCommunicationPlugins();

	return CompatibleTransportPlugins.Num() == 1 ?
		FText::Format(LOCTEXT("MissingCommunicationPlugin_One", "Multi-User needs '{0}' plugin to function.\nEnable the plugin from 'Edit -> Plugins' menu"), FText::AsCultureInvariant(CompatibleTransportPlugins[0])) :
		FText::Format(LOCTEXT("MissingCommunicationPlugin_Many", "Multi-User needs one of the following plugins to function: {0}.\nEnable a plugin from 'Edit -> Plugins' menu"), FText::AsCultureInvariant(FString::Join(CompatibleTransportPlugins, TEXT(", "))));
}

void LogNoCompatibleCommunicationPluginEnabled()
{
	// Multi-User doesn't directly depends on a communication protocol, but rather on MessageBus. The client is responsible to enable the plugin used by MessageBus. For example 'UdpMessaging'.
	TArray<FString> CompatibleTransportPlugins;
	CompatibleTransportPlugins = GetServerCompatibleCommunicationPlugins();
	if (CompatibleTransportPlugins.Num() == 1)
	{
		UE_LOGF(LogConcert, Warning, "The '%ls' plugin is disabled. Multi-User requires '%ls' to function correctly", *CompatibleTransportPlugins[0], *CompatibleTransportPlugins[0]);
	}
	else
	{
		UE_LOGF(LogConcert, Warning, "Multi-User requires one of the following plugins to function correctly: %ls", *FString::Join(CompatibleTransportPlugins, TEXT(", ")));
	}
}

}

#undef LOCTEXT_NAMESPACE
