// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace MultiUserClientUtils
{

/** Checks if the client has at least one server compatible transport plugin enabled, for example, 'UdpMessaging'. */
bool HasServerCompatibleCommunicationPluginEnabled();

/** Returns the list of compatible transport plugins usable by the client to communicate with the server, for example 'UdpMessaging'. */
TArray<FString> GetServerCompatibleCommunicationPlugins();

/** Returns a text saying that a communication plugin is missing to get Multi-User working. */
FText GetNoCompatibleCommunicationPluginEnabledText();

/** Outputs a log saying that a communication plugin is missing to get Multi-User working. */
void LogNoCompatibleCommunicationPluginEnabled();

}
