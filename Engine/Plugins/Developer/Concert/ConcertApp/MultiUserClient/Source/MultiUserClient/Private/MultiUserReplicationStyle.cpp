// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserReplicationStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"
#include "Styling/StyleColors.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FMultiUserReplicationStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )

namespace UE::MultiUserClient
{
	struct FButtonColor
	{
		FLinearColor Normal;
		FLinearColor Hovered;
		FLinearColor Pressed;

		FButtonColor(const FLinearColor& Color)
		{
			Normal = Color * 0.8f;
			Normal.A = Color.A;
			Hovered = Color * 1.0f;
			Hovered.A = Color.A;
			Pressed = Color * 0.6f;
			Pressed.A = Color.A;
		}
	};
	
	FString FMultiUserReplicationStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MultiUserClient"))->GetContentDir();
		return (ContentDir / RelativePath) + Extension;
	}

	TSharedPtr< class FSlateStyleSet > FMultiUserReplicationStyle::StyleSet;

	FName FMultiUserReplicationStyle::GetStyleSetName()
	{
		return FName(TEXT("MultiUserReplicationStyle"));
	}

	void FMultiUserReplicationStyle::Initialize()
	{
		// Only register once
		if (StyleSet.IsValid())
		{
			return;
		}

		StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
		StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
		
		const FVector2D Icon16x16(16.0f, 16.0f); 
		const FVector2D Icon24x24(24.0f, 24.0f); 
		const FVector2D Icon48x48(48.0f, 48.0f);

		const FLinearColor IconColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));

		// Column widths - see FConcertFrontendStyle::Initialize() also ("Default Concert Replication Columns Widths")
		StyleSet->Set("AllClients.Object.MuteToggle", 25.f);
		StyleSet->Set("AllClients.Object.OwnerSize", 200.f);
		StyleSet->Set("AllClients.Property.OwnerSize", 200.f);

		// Timing
		StyleSet->Set("AllClients.Reassignment.DisplayThrobberAfterSeconds", 0.2f);

		// Icons
		StyleSet->Set("MultiUser.Icons.AddProperty",				new IMAGE_PLUGIN_BRUSH("icon_AddProperty_48x", Icon48x48, IconColorAndOpacity));
		StyleSet->Set("MultiUser.Icons.AddProperty.Small",		new IMAGE_PLUGIN_BRUSH("icon_AddProperty_48x", Icon24x24, IconColorAndOpacity));
		StyleSet->Set("MultiUser.Icons.RemoveProperty",			new IMAGE_PLUGIN_BRUSH("icon_RemoveProperty_48x", Icon48x48, IconColorAndOpacity));
		StyleSet->Set("MultiUser.Icons.RemoveProperty.Small",	new IMAGE_PLUGIN_BRUSH("icon_RemoveProperty_48x", Icon24x24, IconColorAndOpacity));

		// Muting
		FSlateImageBrush* PlayBrush = new IMAGE_PLUGIN_BRUSH("generic_play_16x", Icon16x16);
		FSlateImageBrush* PauseBrush = new IMAGE_PLUGIN_BRUSH("generic_pause_16x", Icon16x16);
		StyleSet->Set("MultiUser.Icons.Play", PlayBrush);
		StyleSet->Set("MultiUser.Icons.Pause", PauseBrush);
		StyleSet->Set("AllClients.MuteToggle.Style", FCheckBoxStyle()		
			.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
			.SetUncheckedImage(IMAGE_PLUGIN_BRUSH("generic_pause_16x", Icon16x16, FStyleColors::Foreground))
			.SetUncheckedHoveredImage(IMAGE_PLUGIN_BRUSH("generic_pause_16x", Icon16x16, FStyleColors::ForegroundHover))  
			.SetUncheckedPressedImage(IMAGE_PLUGIN_BRUSH("generic_pause_16x", Icon16x16, FStyleColors::ForegroundHover)) 
			.SetCheckedImage(IMAGE_PLUGIN_BRUSH("generic_play_16x", Icon16x16, FStyleColors::Foreground)) 
			.SetCheckedHoveredImage(IMAGE_PLUGIN_BRUSH("generic_play_16x", Icon16x16, FStyleColors::ForegroundHover)) 
			.SetCheckedPressedImage(IMAGE_PLUGIN_BRUSH("generic_play_16x", Icon16x16, FStyleColors::ForegroundHover)) 
		);
		
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
	};

	void FMultiUserReplicationStyle::Shutdown()
	{
		if (StyleSet.IsValid())
		{
			FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
			ensure(StyleSet.IsUnique());
			StyleSet.Reset();
		}
	}

	TSharedPtr<class ISlateStyle> FMultiUserReplicationStyle::Get()
	{
		return StyleSet;
	}
}

#undef IMAGE_PLUGIN_BRUSH
