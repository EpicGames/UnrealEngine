// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FSlateStyleSet;
class ISlateStyle;

namespace UE::MultiUserClient
{
	/**
	 * Style settings specifically for all replication widgets.
	 * Separate class to make it more straight forward to move to a separate module (there are no specific plans for that only future proofing).
	 */
	class FMultiUserReplicationStyle
	{
	public:
	
		static void Initialize();
		static void Shutdown();

		static TSharedPtr<ISlateStyle> Get();

		static FName GetStyleSetName();
	
	private:
	
		static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);
		static TSharedPtr<FSlateStyleSet> StyleSet;
	};
}
