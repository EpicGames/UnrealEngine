// Copyright Epic Games, Inc. All Rights Reserved.

#include "AuthorityChangeTracker.h"

#include "EAuthorityMutability.h"
#include "IClientAuthoritySynchronizer.h"
#include "Replication/Misc/GlobalAuthorityCache.h"

namespace UE::MultiUserClient::Replication
{
	FAuthorityChangeTracker::FAuthorityChangeTracker(
		const FGuid& InClientId,
		const IClientAuthoritySynchronizer& InAuthoritySynchronizer,
		FGlobalAuthorityCache& InAuthorityCache
		)
		: ClientId(InClientId)
		, AuthoritySynchronizer(InAuthoritySynchronizer)
		, AuthorityCache(InAuthorityCache)
	{
		AuthorityCache.OnCacheChanged().AddRaw(this, &FAuthorityChangeTracker::OnClientChanged);
	}

	FAuthorityChangeTracker::~FAuthorityChangeTracker()
	{
		// Strictly not needed because AuthoritySynchronizer will die at the same time as us but let's be nice
		AuthorityCache.OnCacheChanged().RemoveAll(this);
	}

	void FAuthorityChangeTracker::SetAuthorityIfAllowed(TConstArrayView<FSoftObjectPath> ObjectPaths, bool bNewAuthorityState)
	{
		bool bAddedAnything = false;
		
		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			bool* bCurrentValue = NewAuthorityStates.Find(ObjectPath);
			const bool bValueWasChanged = !bCurrentValue || *bCurrentValue != bNewAuthorityState;
			
			if (bValueWasChanged && CanSetAuthorityFor(ObjectPath))
			{
				bAddedAnything = true;
				NewAuthorityStates.Add(ObjectPath, bNewAuthorityState);
			}
		}

		if (bAddedAnything)
		{
			OnChangedOwnedObjectsDelegate.Broadcast();
		}
	}

	void FAuthorityChangeTracker::ClearAuthorityChange(TConstArrayView<FSoftObjectPath> ObjectPaths)
	{
		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			NewAuthorityStates.Remove(ObjectPath);
		}
	}

	void FAuthorityChangeTracker::RefreshChanges()
	{
		for (auto It = NewAuthorityStates.CreateIterator(); It; ++It)
		{
			const TPair<FSoftObjectPath, bool>& NewAuthorityState = *It;
			const FSoftObjectPath& ObjectPath = NewAuthorityState.Key;

			const bool bHasAuthority = AuthoritySynchronizer.HasAuthorityOver(ObjectPath); 
			if (bHasAuthority == NewAuthorityState.Value
				|| !CanSetAuthorityFor(ObjectPath))
			{
				It.RemoveCurrent();
			}
		}
	}

	bool FAuthorityChangeTracker::GetAuthorityStateAfterApplied(const FSoftObjectPath& ObjectPath) const
	{
		const bool* ChangeValue = NewAuthorityStates.Find(ObjectPath);
		return ChangeValue ? *ChangeValue : AuthoritySynchronizer.HasAuthorityOver(ObjectPath);
	}

	bool FAuthorityChangeTracker::CanSetAuthorityFor(const FSoftObjectPath& ObjectPath) const
	{
		return GetChangeAuthorityMutability(ObjectPath) == EAuthorityMutability::Allowed;
	}

	EAuthorityMutability FAuthorityChangeTracker::GetChangeAuthorityMutability(const FSoftObjectPath& ObjectPath) const
	{
		return AuthorityCache.CanClientTakeAuthorityAfterSubmission(ObjectPath, ClientId);
	}

	TOptional<FConcertReplication_ChangeAuthority_Request> FAuthorityChangeTracker::BuildChangeRequest(const FGuid& StreamId) const
	{
		if (NewAuthorityStates.IsEmpty())
		{
			return {};
		}
		
		FConcertReplication_ChangeAuthority_Request ChangeRequest;
		for (const TPair<FSoftObjectPath, bool>& NewAuthorityState : NewAuthorityStates)
		{
			if (NewAuthorityState.Value)
			{
				ChangeRequest.TakeAuthority.Add(NewAuthorityState.Key).StreamIds = { StreamId };
			}
			else
			{
				ChangeRequest.ReleaseAuthority.Add(NewAuthorityState.Key).StreamIds = { StreamId };
			}
		}
		
		return ChangeRequest;
	}
}
