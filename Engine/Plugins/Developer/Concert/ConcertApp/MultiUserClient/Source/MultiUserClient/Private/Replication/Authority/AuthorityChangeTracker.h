// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/ContainersFwd.h"
#include "Replication/IConcertClientReplicationManager.h"

namespace UE::MultiUserClient::Replication
{
	class FGlobalAuthorityCache;
}

struct FSoftObjectPath;

namespace UE::MultiUserClient::Replication
{
	class IClientAuthoritySynchronizer;
	enum class EAuthorityMutability : uint8;
	
	/** Keeps track of authority changes the local editor instance makes to a client. */
	class FAuthorityChangeTracker
	{
	public:
		/**
		 * @param InClientId The endpoint ID of the owning client
		 * @param InAuthoritySynchronizer Used to get the authority state on the server. The caller ensures it outlives the constructed instance.
		 * @param InAuthorityCache Used to predict authority conflicts. The caller ensures it outlives the constructed instance.
		 */
		FAuthorityChangeTracker(const FGuid& InClientId, const IClientAuthoritySynchronizer& InAuthoritySynchronizer UE_LIFETIMEBOUND, FGlobalAuthorityCache& InAuthorityCache UE_LIFETIMEBOUND);
		~FAuthorityChangeTracker();
		
		/** Marks that the authority should be changed to bNewAuthorityState. */
		void SetAuthorityIfAllowed(TConstArrayView<FSoftObjectPath> ObjectPaths, bool bNewAuthorityState);
		/** Reverts any local, unsubmitted changes made for the ObjectPaths. */
		void ClearAuthorityChange(TConstArrayView<FSoftObjectPath> ObjectPaths);

		/** Diffs NewAuthorityStates to the current authority states and removes entries. */
		void RefreshChanges();
		
		/** Gets the authority state the object will have if the changes are applied. */
		bool GetAuthorityStateAfterApplied(const FSoftObjectPath& ObjectPath) const;
		
		/** @return Whether it is valid to set authority for the given ObjectPath: that is if the object has properties registered. */
		bool CanSetAuthorityFor(const FSoftObjectPath& ObjectPath) const;
		/** @return Detailed information why the object's authority can or cannot be changed.  */
		EAuthorityMutability GetChangeAuthorityMutability(const FSoftObjectPath& ObjectPath) const;

		/** Builds a change request from the local changes, if there are changes. */
		TOptional<FConcertReplication_ChangeAuthority_Request> BuildChangeRequest(const FGuid& StreamId) const;

		/** Called entries are added to NewAuthorityStates. NOT called when objects are removed. */
		DECLARE_MULTICAST_DELEGATE(FOnAuthorityChangeMade);
		FOnAuthorityChangeMade& OnChangedOwnedObjects() { return OnChangedOwnedObjectsDelegate; }
		
	private:

		/** Id of the client this change tracker is tracking. */
		const FGuid ClientId;

		/** Knows the current authority state of the client and determines whether we support changing this client's authority at all. */
		const IClientAuthoritySynchronizer& AuthoritySynchronizer;
		/** Used to determine whether other clients have authority over objects. */
		FGlobalAuthorityCache& AuthorityCache;

		/** Object to the authority state it should have. */
		TMap<FSoftObjectPath, bool> NewAuthorityStates;

		/** Called when NewAuthorityStates is updated. */
		FOnAuthorityChangeMade OnChangedOwnedObjectsDelegate;
		
		void OnClientChanged(const FGuid& Guid) { RefreshChanges(); }
	};
}


