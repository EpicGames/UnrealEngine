// Copyright Epic Games, Inc. All Rights Reserved.

#include "AuthoritySynchronizer_LocalClient.h"

#include "IConcertSyncClient.h"
#include "Replication/IConcertClientReplicationManager.h"

namespace UE::MultiUserClient::Replication
{
	FAuthoritySynchronizer_LocalClient::FAuthoritySynchronizer_LocalClient(TSharedRef<IConcertSyncClient> InClient)
		: Client(MoveTemp(InClient))
	{
		IConcertClientReplicationManager* ReplicationManager = Client->GetReplicationManager();
		if (ensure(ReplicationManager))
		{
			ReplicationManager->OnPostAuthorityChanged().AddRaw(this, &FAuthoritySynchronizer_LocalClient::OnAuthorityChanged);
		}
	}

	FAuthoritySynchronizer_LocalClient::~FAuthoritySynchronizer_LocalClient()
	{
		if (IConcertClientReplicationManager* ReplicationManager = Client->GetReplicationManager())
		{
			ReplicationManager->OnPostAuthorityChanged().RemoveAll(this);
		}
	}

	bool FAuthoritySynchronizer_LocalClient::HasAuthorityOver(const FSoftObjectPath& ObjectPath) const
	{
		const IConcertClientReplicationManager* ReplicationManager = Client->GetReplicationManager();
		return ensure(ReplicationManager) && !ReplicationManager->GetClientOwnedStreamsForObject(ObjectPath).IsEmpty();
	}

	bool FAuthoritySynchronizer_LocalClient::HasAnyAuthority() const
	{
		const IConcertClientReplicationManager* ReplicationManager = Client->GetReplicationManager();
		if (!ensure(ReplicationManager))
		{
			return false;
		}
		
		bool bReached = false;
		ReplicationManager->ForEachClientOwnedObject([&bReached](const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)
		{
			bReached = true;
			return EBreakBehavior::Break;
		});
		return bReached;
	}

	void FAuthoritySynchronizer_LocalClient::OnAuthorityChanged() const
	{
		OnServerStateChangedDelegate.Broadcast();
	}
}
