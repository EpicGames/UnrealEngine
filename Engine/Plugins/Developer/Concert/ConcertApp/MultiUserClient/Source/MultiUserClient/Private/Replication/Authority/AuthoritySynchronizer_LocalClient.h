// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientAuthoritySynchronizer.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;

namespace UE::MultiUserClient::Replication
{
	class FAuthoritySynchronizer_LocalClient : public FAuthoritySynchronizer_Base
	{
	public:
		
		FAuthoritySynchronizer_LocalClient(TSharedRef<IConcertSyncClient> InClient);
		virtual ~FAuthoritySynchronizer_LocalClient() override;

		//~ Begin IClientAuthoritySynchronizer Interface
		virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const override;
		virtual bool HasAnyAuthority() const override;
		//~ End IClientAuthoritySynchronizer Interface

	private:

		/** Needed to access IConcertClientReplicationManager */
		TSharedRef<IConcertSyncClient> Client;
		
		void OnAuthorityChanged() const;
	};
}


