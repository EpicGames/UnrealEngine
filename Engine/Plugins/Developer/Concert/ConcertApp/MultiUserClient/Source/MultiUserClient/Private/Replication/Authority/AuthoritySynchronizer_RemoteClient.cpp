// Copyright Epic Games, Inc. All Rights Reserved.

#include "AuthoritySynchronizer_RemoteClient.h"

#include "Replication/Misc/Query/StreamAndAuthorityQueryService.h"

namespace UE::MultiUserClient::Replication
{
	FAuthoritySynchronizer_RemoteClient::FAuthoritySynchronizer_RemoteClient(const FGuid& RemoteEndpointId, FStreamAndAuthorityQueryService& InQueryService)
		: QueryService(InQueryService)
		, QueryStreamHandle(
    		QueryService.RegisterAuthorityQuery(
    			RemoteEndpointId,
    			FAuthorityQueryDelegate::CreateRaw(this, &FAuthoritySynchronizer_RemoteClient::HandleAuthorityQuery)
    			)
    		)
	{}

	FAuthoritySynchronizer_RemoteClient::~FAuthoritySynchronizer_RemoteClient()
	{
		QueryService.UnregisterAuthorityQuery(QueryStreamHandle);
	}

	bool FAuthoritySynchronizer_RemoteClient::HasAnyAuthority() const
	{
		return !LastServerState.IsEmpty();
	}

	void FAuthoritySynchronizer_RemoteClient::HandleAuthorityQuery(const TArray<FConcertAuthorityClientInfo>& PerStreamAuthority)
	{
		TSet<FSoftObjectPath> OldServerState = MoveTemp(LastServerState);
		for (const FConcertAuthorityClientInfo& Info : PerStreamAuthority)
		{
			LastServerState.Append(Info.AuthoredObjects);
		}
		
		const bool bAreEqual = OldServerState.Num() == LastServerState.Num() && LastServerState.Includes(OldServerState);
		if (!bAreEqual)
		{
			OnServerStateChangedDelegate.Broadcast();
		}
	}
}
