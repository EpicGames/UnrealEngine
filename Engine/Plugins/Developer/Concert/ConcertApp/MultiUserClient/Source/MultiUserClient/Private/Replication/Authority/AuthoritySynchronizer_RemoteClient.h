// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientAuthoritySynchronizer.h"

#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "UObject/SoftObjectPath.h"

struct FGuid;
struct FConcertAuthorityClientInfo;

namespace UE::MultiUserClient::Replication
{
	class FStreamAndAuthorityQueryService;

	class FAuthoritySynchronizer_RemoteClient : public FAuthoritySynchronizer_Base
	{
	public:
		
		FAuthoritySynchronizer_RemoteClient(const FGuid& RemoteEndpointId, FStreamAndAuthorityQueryService& InQueryService UE_LIFETIMEBOUND);
		virtual ~FAuthoritySynchronizer_RemoteClient() override;

		//~ Begin IClientAuthoritySynchronizer Interface
		virtual bool HasAnyAuthority() const override;
		virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const override { return LastServerState.Contains(ObjectPath); }
		//~ End IClientAuthoritySynchronizer Interface

	private:

		/** Queries the server in regular intervals. This services outlives our object. */
		FStreamAndAuthorityQueryService& QueryService;
		
		/** Used to unregister HandleStreamQuery upon destruction. */
		const FDelegateHandle QueryStreamHandle;

		/** The most up to date server state of the remote client's authority. */
		TSet<FSoftObjectPath> LastServerState; 

		void HandleAuthorityQuery(const TArray<FConcertAuthorityClientInfo>& PerStreamAuthority);
	};
}


