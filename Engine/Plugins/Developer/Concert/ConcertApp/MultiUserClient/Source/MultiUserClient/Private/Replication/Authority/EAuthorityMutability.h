// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::MultiUserClient::Replication
{
	/** Various reasons in which an object can be when considering changing its authority */
	enum class EAuthorityMutability : uint8
	{
		/** The object has no registered properties. Cannot take authority. */
		NotApplicable,
		
		/** There is a conflict with another client */
		Conflict,
		
		/** Authority can be taken */
		Allowed
	};
}