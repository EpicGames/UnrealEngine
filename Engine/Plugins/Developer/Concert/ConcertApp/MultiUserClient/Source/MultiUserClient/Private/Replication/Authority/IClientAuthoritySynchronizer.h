// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

struct FSoftObjectPath;

namespace UE::MultiUserClient::Replication
{
	/** Synchronizes the a client's authority state with the server. */
	class IClientAuthoritySynchronizer
	{
	public:
		
		/** @return Whether this client is sending anything at all. */
		virtual bool HasAnyAuthority() const = 0;

		/** @return Whether the local instance thinks this client has authority over ObjectPath. */
		virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const = 0;

		DECLARE_MULTICAST_DELEGATE(FOnServerStateChanged);
		/** @return Event executed when authority state has been updated. */
		virtual FOnServerStateChanged& OnServerAuthorityChanged() = 0;

		virtual ~IClientAuthoritySynchronizer() = default;
	};

	/** Util base class for implementing the events */
	class FAuthoritySynchronizer_Base
		: public IClientAuthoritySynchronizer
	{
	public:

		//~ Begin IClientAuthoritySynchronizer Interface
		virtual FOnServerStateChanged& OnServerAuthorityChanged() override { return OnServerStateChangedDelegate; }
		//~ End IClientAuthoritySynchronizer Interface

	protected:

		/** Triggered by subclasses when the authority state changes. */
		FOnServerStateChanged OnServerStateChangedDelegate;
	};
}