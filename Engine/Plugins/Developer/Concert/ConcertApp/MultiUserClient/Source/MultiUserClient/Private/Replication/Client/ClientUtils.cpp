// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientUtils.h"

#include "ConcertMessageData.h"
#include "IConcertClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Widgets/Client/SClientName.h"

namespace UE::MultiUserClient::Replication::ClientUtils
{
	FString GetClientDisplayName(const IConcertClient& InLocalClientInstance, const FGuid& InClientEndpointId)
	{
		const TSharedPtr<IConcertClientSession> Session = InLocalClientInstance.GetCurrentSession();
		return ensure(Session) ? GetClientDisplayName(*Session, InClientEndpointId) : FString{};
	}
	
	FString GetClientDisplayName(const IConcertClientSession& InSession, const FGuid& InClientEndpointId)
	{
		const bool bIsLocalClient = InSession.GetSessionClientEndpointId() == InClientEndpointId;
		if (bIsLocalClient)
		{
			return ConcertSharedSlate::SClientName::GetDisplayText(InSession.GetLocalClientInfo(), bIsLocalClient).ToString();
		}

		FConcertSessionClientInfo ClientInfo;
		if (InSession.FindSessionClient(InClientEndpointId, ClientInfo))
		{
			return ConcertSharedSlate::SClientName::GetDisplayText(ClientInfo.ClientInfo, bIsLocalClient).ToString();
		}

		ensureMsgf(false, TEXT("Bad args"));
		return InClientEndpointId.ToString(EGuidFormats::DigitsWithHyphens);
	}
	
	bool GetClientDisplayInfo(const IConcertClient& InLocalClientInstance, const FGuid& InClientEndpointId, FConcertClientInfo& OutClientInfo)
	{
		const TSharedPtr<IConcertClientSession> Session = InLocalClientInstance.GetCurrentSession();
		return GetClientDisplayInfo(*Session, InClientEndpointId, OutClientInfo);
	}
	
	bool GetClientDisplayInfo(const IConcertClientSession& InSession, const FGuid& InClientEndpointId, FConcertClientInfo& OutClientInfo)
	{
		const bool bIsLocalClient = InSession.GetSessionClientEndpointId() == InClientEndpointId;
		if (bIsLocalClient)
		{
			OutClientInfo = InSession.GetLocalClientInfo();
			return true;
		}

		FConcertSessionClientInfo ClientInfo;
		if (InSession.FindSessionClient(InClientEndpointId, ClientInfo))
		{
			OutClientInfo = MoveTemp(ClientInfo.ClientInfo);
			return true;
		}

		return false;
	}
	
	TArray<const FOnlineClient*> GetSortedClientList(const IConcertClient& InLocalClientInstance, const FOnlineClientManager& InReplicationManager)
	{
		return GetSortedClientList(*InLocalClientInstance.GetCurrentSession(), InReplicationManager);
	}

	TArray<const FOnlineClient*> GetSortedClientList(const IConcertClientSession& InSession, const FOnlineClientManager& InReplicationManager)
	{
		TArray<const FOnlineClient*> Result;
		TMap<const FOnlineClient*, FConcertClientInfo> ClientToDisplayInfo;
		for (const TNonNullPtr<const FRemoteClient> RemoteClient : InReplicationManager.GetRemoteClients())
		{
			FConcertClientInfo Info;
			if (GetClientDisplayInfo(InSession, RemoteClient->GetEndpointId(), Info))
			{
				Result.Add(RemoteClient);
				ClientToDisplayInfo.Add(RemoteClient, MoveTemp(Info));
			}
		}

		Result.Sort([&ClientToDisplayInfo](const FOnlineClient& Left, const FOnlineClient& Right)
		{
			const FConcertClientInfo& LeftInfo = ClientToDisplayInfo[&Left];
			const FConcertClientInfo& RightInfo = ClientToDisplayInfo[&Right];
			const FText LeftDisplayName = ConcertSharedSlate::SClientName::GetDisplayText(LeftInfo);
			const FText RightDisplayName = ConcertSharedSlate::SClientName::GetDisplayText(RightInfo);
			return LeftDisplayName.ToString() <= RightDisplayName.ToString();
		});
		
		Result.Insert(&InReplicationManager.GetLocalClient(), 0);
		return Result;
	}
}
