// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class FString;
class IConcertClient;
class IConcertClientSession;

struct FConcertClientInfo;
struct FGuid;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClient;
	class FOnlineClientManager;
}

namespace UE::MultiUserClient::Replication::ClientUtils
{
	/**
	 * Gets the display name for a client. Appends (me) if the client is local.
	 * 
	 * @param InLocalClientInstance Used to look up client display info
	 * @param InClientEndpointId The endpoint ID of the client whose name to get
	 * @return The display name or empty
	 */
	FString GetClientDisplayName(const IConcertClient& InLocalClientInstance, const FGuid& InClientEndpointId);
	FString GetClientDisplayName(const IConcertClientSession& InSession, const FGuid& InClientEndpointId);

	/**
	 * Gets the display info for a given client
	 *
	 * @param InLocalClientInstance Used to look up client display info
	 * @param InClientEndpointId The endpoint ID of the client whose name to get
	 * @param OutClientInfo The client display info to get
	 * @return Whether OutClientInfo holds a value
	 */
	bool GetClientDisplayInfo(const IConcertClient& InLocalClientInstance, const FGuid& InClientEndpointId, FConcertClientInfo& OutClientInfo);
	bool GetClientDisplayInfo(const IConcertClientSession& InSession, const FGuid& InClientEndpointId, FConcertClientInfo& OutClientInfo);

	/**
	 * Gets all replication clients in a sorted array. The local client will always be first and then come all remote clients sorted alphabetically.
	 * @param InLocalClientInstance Used to look up client display info
	 * @param InReplicationManager Used to obtain all clients
	 * @return Sorted client array
	 */
	TArray<const FOnlineClient*> GetSortedClientList(const IConcertClient& InLocalClientInstance, const FOnlineClientManager& InReplicationManager);
	TArray<const FOnlineClient*> GetSortedClientList(const IConcertClientSession& InSession, const FOnlineClientManager& InReplicationManager);
}
