// Copyright Epic Games, Inc. All Rights Reserved.

#include "EndpointCache.h"

#include "IConcertClientWorkspace.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

namespace UE::MultiUserClient::Replication
{
	void FEndpointCache::UpdateEndpoints(const IConcertClientWorkspace& Workspace)
	{
		while (NextFirstActivityToFetch < Workspace.GetLastActivityId())
		{
			constexpr int64 MaxToFetch = 1000;
			TArray<FConcertSessionActivity> DummyActivities;
			TMap<FGuid, FConcertClientInfo> NewEndpoints;
			Workspace.GetActivities(NextFirstActivityToFetch, MaxToFetch, NewEndpoints, DummyActivities);
			NextFirstActivityToFetch += DummyActivities.Num();

			MergeEndpointsWith(NewEndpoints);
		}
	}

	int32 FEndpointCache::FindClientIndexByEndpointId(const FGuid& EndpointId) const
	{
		return EndpointMetaData.IndexOfByPredicate([&EndpointId](const FClientMetaData& MetaData)
		{
			return MetaData.AssociatedEndpoints.Contains(EndpointId);
		});
	}

	void FEndpointCache::MergeEndpointsWith(const TMap<FGuid, FConcertClientInfo>& NewEndpoints)
	{
		for (const TPair<FGuid, FConcertClientInfo>& NewEndpointPair : NewEndpoints)
		{
			const FGuid& EndpointId = NewEndpointPair.Key;
			const FConcertClientInfo& EncounteredClientInfo = NewEndpointPair.Value;
				
			// We'll consider a client equal if its DisplayName and DeviceName are equal...
			// This will not work if you have two UE instances with the same name on the same machine (unsupported).
			const int32 EndpointIndex = KnownClients.IndexOfByPredicate([&EncounteredClientInfo](const FConcertClientInfo& ExistingClientInfo)
			{
				return ConcertSyncCore::Replication::AreLogicallySameClients(EncounteredClientInfo, ExistingClientInfo);
			});
			const bool bAlreadyKnown = KnownClients.IsValidIndex(EndpointIndex);
			if (bAlreadyKnown)
			{
				EndpointMetaData[EndpointIndex].AssociatedEndpoints.AddUnique(EndpointId);
				continue;
			}

			const int32 MetaDataIndex = EndpointMetaData.IndexOfByPredicate([&](const FClientMetaData& MetaData)
			{
				return MetaData.GetLastKnownEndpointId() == EndpointId;
			});
			const bool bIsNameChange = EndpointMetaData.IsValidIndex(MetaDataIndex);
			if (bIsNameChange)
			{
				KnownClients[EndpointIndex] = EncounteredClientInfo;
				EndpointMetaData[EndpointIndex].AssociatedEndpoints.AddUnique(EndpointId);
			}
			else
			{
				KnownClients.Add(EncounteredClientInfo);
				EndpointMetaData.Emplace(NewEndpointPair.Key);
			}
		}
	}
}
