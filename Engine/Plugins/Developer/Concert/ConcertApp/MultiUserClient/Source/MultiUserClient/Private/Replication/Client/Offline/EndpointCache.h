// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"

#include "Containers/Map.h"
#include "HAL/Platform.h"

class IConcertClientWorkspace;

namespace UE::MultiUserClient::Replication
{
	/** Goes through activity history and saves all endpoints is encounters. */
	class FEndpointCache
	{
	public:

		/** Updates the list of endpoints, if needed. */
		void UpdateEndpoints(const IConcertClientWorkspace& Workspace);

		/** @return An index to GetKnownClients() if a client with EndpointId is known. */
		int32 FindClientIndexByEndpointId(const FGuid& EndpointId) const;

		/** @return The last endpoint that the client at GetKnownClients()[Index] was associated with. */
		const FGuid& GetLastAssociatedEndpoint(int32 Index) const { return EndpointMetaData[Index].GetLastKnownEndpointId(); }
		
		/** @return The clients that are known */
		const TArray<FConcertClientInfo>& GetKnownClients() const { return KnownClients; }

	private:

		/** The next activity to start fetching from.*/
		int64 NextFirstActivityToFetch = 1; // Activity IDs start with 1

		/**
		 * The clients encountered thus far. Multiple endpoint IDs are associated with a client.
		 * 
		 * Client info is grouped by DeviceName and DisplayName even though they are technically new endpoints.
		 * Example: Client1 joins, leaves, then joins again.
		 * On the server, this would generate two distinct endpoints with different endpoint IDs, but we'll merge them into one assuming it's the
		 * same machine. This will not work if you have two UE instances with the same name on the same machine, which is unsupported by this class.
		 */
		TArray<FConcertClientInfo> KnownClients;

		struct FClientMetaData
		{
			/** All endpoint IDs that are associated with this client. */
			TArray<FGuid> AssociatedEndpoints;

			const FGuid& GetLastKnownEndpointId() const { return AssociatedEndpoints[AssociatedEndpoints.Num() - 1]; }

			explicit FClientMetaData(const FGuid& LastKnownEndpointId)
				: AssociatedEndpoints({ LastKnownEndpointId })
			{}
		};
		/** Additional data about. Each index maps to KnownEndpoints. Array based to make GetKnownEndpoints interface easier. */
		TArray<FClientMetaData> EndpointMetaData;

		/** Adds NewEndpoints to EndpointMetaData. */
		void MergeEndpointsWith(const TMap<FGuid, FConcertClientInfo>& NewEndpoints);
	};
}


