// Copyright Epic Games, Inc. All Rights Reserved.

#include "OfflineClient.h"

#include "Replication/ReplicationWidgetFactories.h"

namespace UE::MultiUserClient::Replication
{
	FOfflineClient::FOfflineClient(
		IConcertClientWorkspace& InWorkspace,
		FConcertClientInfo InClientInfo,
		const FGuid& LastAssociatedEndpoint
		)
		: ClientInfo(MoveTemp(InClientInfo))
		, LastAssociatedEndpoint(LastAssociatedEndpoint)
		, ContentPredictor(InWorkspace, ClientInfo)
		, StreamModel(
			ConcertSharedSlate::CreateReadOnlyStreamModel(
				TAttribute<const FConcertObjectReplicationMap*>::CreateLambda(
					[this]{ return &ContentPredictor.GetServerState(); }
					)
				)
			)
	{
		ContentPredictor.OnPredictionChanged().AddLambda([this]
		{
			OnStreamPredictionChangedDelegate.Broadcast();
		});
	}
}
