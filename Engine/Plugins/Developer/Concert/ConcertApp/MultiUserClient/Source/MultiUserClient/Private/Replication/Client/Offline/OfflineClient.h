// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "RejoinStreamAndAuthorityPredictor.h"

#include "Templates/UnrealTemplate.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSharedSlate { class IReplicationStreamModel; }

namespace UE::MultiUserClient::Replication
{
	/**
	 * Info about a client that had joined the MU session but no are longer is present.
	 * This is used to display info about the properties the client would receive if they joined back into the session. 
	 */
	class FOfflineClient : public FNoncopyable
	{
	public:

		FOfflineClient(
			IConcertClientWorkspace& InWorkspace UE_LIFETIMEBOUND,
			FConcertClientInfo InClientInfo,
			const FGuid& LastAssociatedEndpoint
			);

		FORCEINLINE const FConcertClientInfo& GetClientInfo() const { return ClientInfo; }
		FORCEINLINE const FGuid& GetLastAssociatedEndpoint() const { return LastAssociatedEndpoint; }
		FORCEINLINE const FConcertBaseStreamInfo& GetPredictedStream() const { return ContentPredictor.GetPredictedStream(); }
		
		/**
		 * This is used so the UI can construct the IReplicationStreamEditor.
		 * @see CreateBaseStreamEditor and FCreateEditorParams.
		 * 
		 * @note You must make sure to release this object when this FReplicationClient is destroyed.
		 * Listen for events on the owning FOfflineClientManager::OnDisconnectedClientsChanged.
		 * @see FReplicationClientManager and FMultiUserReplicationManager
		 */
		TSharedRef<ConcertSharedSlate::IReplicationStreamModel> GetStreamModel() const { return StreamModel; }

		DECLARE_MULTICAST_DELEGATE(FStreamPredictionChanged);
		/** Broadcasts when the stream, which the offline client will get upon rejoining, has changed. */
		FStreamPredictionChanged& OnStreamPredictionChanged() { return OnStreamPredictionChangedDelegate; }

	private:

		/** The client's last registered display data. */
		const FConcertClientInfo ClientInfo;
		/** The last endpoint ID this client was associated with- */
		const FGuid LastAssociatedEndpoint;

		/** Predicts the stream as activities are produced. */
		FRejoinStreamAndAuthorityPredictor ContentPredictor;
		/**
		 * Read-only model of the state predicted by StreamSynchronizer.
		 * TODO DP: In the future, this may be made editable to allow the user to edit the future state in the UI.
		 */
		const TSharedRef<ConcertSharedSlate::IReplicationStreamModel> StreamModel;
		
		/** Broadcasts when the stream, which the offline client will get upon rejoining, has changed. */
		FStreamPredictionChanged OnStreamPredictionChangedDelegate;
	};
}


