// Copyright Epic Games, Inc. All Rights Reserved.

#include "OfflineClientManager.h"

#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "Replication/Client/ClientUtils.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

namespace UE::MultiUserClient::Replication
{
	FOfflineClientManager::FOfflineClientManager(
		IConcertSyncClient& ClientInstance,
		FOnlineClientManager& OnlineClientManager
		)
		: ClientInstance(ClientInstance)
		, OnlineClientManager(OnlineClientManager)
	{
		ClientInstance.GetWorkspace()->OnActivityAddedOrUpdated().AddRaw(this, &FOfflineClientManager::OnActivityAddedOrProduced);
		OnlineClientManager.OnRemoteClientsChanged().AddRaw(this, &FOfflineClientManager::RefreshOfflineClients);
		RefreshOfflineClients();
	}

	FOfflineClientManager::~FOfflineClientManager()
	{
		ClientInstance.GetWorkspace()->OnActivityAddedOrUpdated().RemoveAll(this);
		OnlineClientManager.OnRemoteClientsChanged().RemoveAll(this);
	}

	const FOfflineClient* FOfflineClientManager::FindClient(const FGuid& EndpointId) const
	{
		const int32 CacheIndex = EndpointCache.FindClientIndexByEndpointId(EndpointId);
		const TArray<FConcertClientInfo>& KnownClients = EndpointCache.GetKnownClients();
		if (!KnownClients.IsValidIndex(CacheIndex))
		{
			return nullptr;
		}

		const int32 ClientIndex = FindClientIndexByInfo(KnownClients[CacheIndex]);
		return Clients.IsValidIndex(ClientIndex) ? Clients[ClientIndex].Get() : nullptr;
	}

	FOfflineClient* FOfflineClientManager::FindClient(const FGuid& EndpointId)
	{
		return const_cast<FOfflineClient*>(
			const_cast<const FOfflineClientManager*>(this)->FindClient(EndpointId)
			);
	}

	void FOfflineClientManager::OnActivityAddedOrProduced(const FConcertClientInfo&, const FConcertSyncActivity&, const FStructOnScope&)
	{
		const TSharedPtr<IConcertClientWorkspace> Workspace = ClientInstance.GetWorkspace();
		check(Workspace);
		EndpointCache.UpdateEndpoints(*Workspace);
	}

	void FOfflineClientManager::RefreshOfflineClients()
	{
		const TSharedPtr<IConcertClientWorkspace> Workspace = ClientInstance.GetWorkspace();
		check(Workspace);
		EndpointCache.UpdateEndpoints(*Workspace);

		const bool bChanged = UpdateClientList(*Workspace);
		if (bChanged)
		{
			OnClientsChangedDelegate.Broadcast();
		}
	}

	bool FOfflineClientManager::UpdateClientList(IConcertClientWorkspace& Workspace)
	{
		bool bChanged = false;
		const TArray<FConcertClientInfo>& KnownClients = EndpointCache.GetKnownClients();
		for (int32 i = 0; i < KnownClients.Num(); ++i)
		{
			const FConcertClientInfo& KnownEndpointInfo = KnownClients[i];
			if (IsClientOnline(KnownEndpointInfo))
			{
				const int32 Index = FindClientIndexByInfo(KnownEndpointInfo);
				if (Clients.IsValidIndex(Index))
				{
					bChanged = true;
					
					TUniquePtr<FOfflineClient>& OfflineClient = Clients[Index];
					OnPreClientRemovedDelegate.Broadcast(*OfflineClient.Get());
					
					Clients.RemoveAt(Index);
				}
			}
			else
			{
				bChanged = true;

				const FGuid LastId = EndpointCache.GetLastAssociatedEndpoint(i);
				const int32 AddedIndex = Clients.Emplace(
					MakeUnique<FOfflineClient>(Workspace, KnownEndpointInfo, LastId)
					);
				FOfflineClient& Client = *Clients[AddedIndex].Get();
				Client.OnStreamPredictionChanged().AddLambda([this, &Client]{ OnClientContentChangedDelegate.Broadcast(Client); });
				OnPostClientAddedDelegate.Broadcast(Client);
			}
		}
		return bChanged;
	}

	bool FOfflineClientManager::IsClientOnline(const FConcertClientInfo& QueryClientInfo) const
	{
		bool bIsOnline = false;
		OnlineClientManager.ForEachClient([this, &QueryClientInfo, &bIsOnline](const FOnlineClient& Client)
		{
			FConcertClientInfo ClientInfo;
			const bool bGotInfo = ClientUtils::GetClientDisplayInfo(*ClientInstance.GetConcertClient(), Client.GetEndpointId(), ClientInfo);
			bIsOnline = bGotInfo && ConcertSyncCore::Replication::AreLogicallySameClients(QueryClientInfo, ClientInfo);
			return bIsOnline ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		return bIsOnline;
	}
	
	int32 FOfflineClientManager::FindClientIndexByInfo(const FConcertClientInfo& KnownEndpointInfo) const
	{
		return Clients.IndexOfByPredicate([&KnownEndpointInfo](const TUniquePtr<FOfflineClient>& OfflineClient)
		{
			return ConcertSyncCore::Replication::AreLogicallySameClients(OfflineClient->GetClientInfo(), KnownEndpointInfo);
		});
	}
}
