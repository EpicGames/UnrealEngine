// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EndpointCache.h"
#include "OfflineClient.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/EBreakBehavior.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

struct FConcertSyncActivity;
class IConcertClientWorkspace;
class IConcertSyncClient;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
	
	/**
	 * Manages clients that had joined the MU session but no are longer is present.
	 *
	 * Endpoint IDs are saved in the database. Endpoint IDs with the same DisplayName and DeviceName are associated with single FOfflineClient.
	 * While FEndpointCache detects name changes, the rest of the system is not currently set up to handle them, e.g. FindClient.
	 */
	class FOfflineClientManager : public FNoncopyable
	{
	public:
		
		/**
		 * @param ClientInstance Used to obtain disconnected endpoint info from the server. The caller ensures it outlives the new object.
		 * @param OnlineClientManager Used to determine whether a client should be considered offline. The caller ensures it outlives the new object.
		 */
		FOfflineClientManager(
			IConcertSyncClient& ClientInstance UE_LIFETIMEBOUND,
			FOnlineClientManager& OnlineClientManager UE_LIFETIMEBOUND
			);
		~FOfflineClientManager();

		/** @return The offline client that was associated with this endpoint ID. */
		const FOfflineClient* FindClient(const FGuid& EndpointId) const;
		/** @return The offline client that was associated with this endpoint ID. */
		FOfflineClient* FindClient(const FGuid& EndpointId);
		
		/** Iterates through every offline client. */
		template<typename TLambda> requires std::is_invocable_r_v<EBreakBehavior, TLambda, FOfflineClient&>
		void ForEachClient(TLambda&& Consumer) const;

		DECLARE_MULTICAST_DELEGATE(FOfflineClientsChanged);
		/** Broadcasts when Clients is changed. */
		FOfflineClientsChanged& OnClientsChanged() { return OnClientsChangedDelegate; }
		
		DECLARE_MULTICAST_DELEGATE_OneParam(FOfflineClientDelegate, FOfflineClient&);
		/** Called just after an offline client is has been added. Called before OnClientsChanged. */
		FOfflineClientDelegate& OnPostClientAdded() { return OnPostClientAddedDelegate; }
		/** Called just before an offline client is about to be removed. Called before OnClientsChanged. */
		FOfflineClientDelegate& OnPreClientRemoved() { return OnPreClientRemovedDelegate; }
		/** Broadcasts after an offline client's stream content has changed. Also called after OnClientsChangedDelegate. */
		FOfflineClientDelegate& OnClientContentChanged() { return OnClientContentChangedDelegate; }

	private:

		/** The local client connected to the MU session. Used to obtain disconnected endpoint info from the server. */
		IConcertSyncClient& ClientInstance;
		/** Used to determine whether a client should be considered offline. */
		FOnlineClientManager& OnlineClientManager;

		/** Info about clients that were once connected but are no longer. */
		TArray<TUniquePtr<FOfflineClient>> Clients;

		/** Keeps track of all known endpoints. */
		FEndpointCache EndpointCache;

		/** Broadcasts when Clients is changed. */
		FOfflineClientsChanged OnClientsChangedDelegate;
		/** Broadcasts just after an offline client is has been added. Called before OnClientsChanged. */
		FOfflineClientDelegate OnPostClientAddedDelegate;
		/** Broadcasts just before an offline client is about to be removed. Called before OnClientsChanged. */
		FOfflineClientDelegate OnPreClientRemovedDelegate;
		/** Broadcasts after an offline client's stream content has changed. Not called as part of OnPostClientAddedDelegate. */
		FOfflineClientDelegate OnClientContentChangedDelegate;

		/** Updates the list of known endpoints in response to an activity being added. */
		void OnActivityAddedOrProduced(const FConcertClientInfo&, const FConcertSyncActivity&, const FStructOnScope&);
		
		/** Refreshes Clients based on which clients are online. */
		void RefreshOfflineClients();
		/** Iterates through EndpointCache and updates Clients.*/
		bool UpdateClientList(IConcertClientWorkspace& Workspace);
		/** @return Whether a client with the given info is online. */
		bool IsClientOnline(const FConcertClientInfo& QueryClientInfo) const;

		/** @return Index to Clients that matches ClientInfo. */
		int32 FindClientIndexByInfo(const FConcertClientInfo& ClientInfo) const;
	};

	template <typename TLambda> requires std::is_invocable_r_v<EBreakBehavior, TLambda, FOfflineClient&>
	void FOfflineClientManager::ForEachClient(TLambda&& Consumer) const
	{
		for (const TUniquePtr<FOfflineClient>& ClientPtr : Clients)
		{
			if (Consumer(*ClientPtr) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}
}


