// Copyright Epic Games, Inc. All Rights Reserved.

#include "RejoinStreamAndAuthorityPredictor.h"

#include "EndpointCache.h"
#include "IConcertClientWorkspace.h"
#include "Replication/Messages/ReplicationActivity.h"

#include "Replication/Misc/ClientPredictionUtils.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

namespace UE::MultiUserClient::Replication
{
	namespace Private
	{
		static void ExtractMultiUserContent(
			const TArray<FConcertBaseStreamInfo>& Streams,
			const TArray<FConcertObjectInStreamID>& Authority,
			FConcertBaseStreamInfo& OutPredictedStream,
			TSet<FSoftObjectPath>& OutPredictedAuthority
			)
		{
			// A user may have created their custom streams, so find the MultiUserStreamID one.
			const FConcertBaseStreamInfo* MultiUserStream = Streams.FindByPredicate([](const FConcertBaseStreamInfo& Stream)
			{
				return Stream.Identifier == MultiUserStreamID;
			});
			if (!MultiUserStream)
			{
				return;
			}
			
			OutPredictedStream = *MultiUserStream;
			// Again... a user may have created their custom streams, so only return the authority of the MultiUserStreamID one.
			Algo::TransformIf(Authority, OutPredictedAuthority,
				[](const FConcertObjectInStreamID& ObjectInStreamID) { return ObjectInStreamID.StreamId == MultiUserStreamID; },
				[](const FConcertObjectInStreamID& ObjectInStreamID) { return ObjectInStreamID.Object; }
			);
		}
		
		static void AnalyzeActivityHistory(
			const IConcertClientWorkspace& Workspace,
			const FConcertClientInfo& ClientInfo,
			FConcertBaseStreamInfo& OutPredictedStream,
			TSet<FSoftObjectPath>& OutPredictedAuthority
			)
		{
			TArray<FConcertBaseStreamInfo> Streams;
			TArray<FConcertObjectInStreamID> Authority;
			const TOptional<int64> FoundActivity = ConcertSyncClient::Replication::IncrementalBacktrackActivityHistoryForActivityThatSetsContent(
				Workspace, ClientInfo, Streams, Authority
				);
			if (FoundActivity.IsSet())
			{
				ExtractMultiUserContent(Streams, Authority, OutPredictedStream, OutPredictedAuthority);
			}
		}
	}

	FRejoinStreamAndAuthorityPredictor::FRejoinStreamAndAuthorityPredictor(IConcertClientWorkspace& InWorkspace, FConcertClientInfo InClientInfo)
		: Workspace(InWorkspace)
		, ClientInfo(MoveTemp(InClientInfo))
		, PredictedStream({ .Identifier = MultiUserStreamID })
	{
		// Whenever a leave replication activity is received, update the predicted stream.
		// Usually when a remote client leaves, a FOfflineClient is created is response to the session's client list changing but the leave replication
		// activity is only received later. Hence, we need to listen for changes.
		Workspace.OnActivityAddedOrUpdated().AddRaw(this, &FRejoinStreamAndAuthorityPredictor::OnActivityAddedOrUpdated);
		
		AnalyzeHistory();
	}

	FRejoinStreamAndAuthorityPredictor::~FRejoinStreamAndAuthorityPredictor()
	{
		Workspace.OnActivityAddedOrUpdated().RemoveAll(this);
	}

	void FRejoinStreamAndAuthorityPredictor::OnActivityAddedOrUpdated(
		const FConcertClientInfo& ActivityClientInfo,
		const FConcertSyncActivity& Activity,
		const FStructOnScope& Summary
		)
	{
		if (ConcertSyncCore::Replication::AreLogicallySameClients(ClientInfo, ActivityClientInfo)
			&& Activity.EventType == EConcertSyncActivityEventType::Replication)
		{
			AnalyzeHistory();
		}
	}
	
	void FRejoinStreamAndAuthorityPredictor::AnalyzeHistory()
	{
		FConcertBaseStreamInfo Prediction;
		Private::AnalyzeActivityHistory(Workspace, ClientInfo, Prediction, PredictedAuthority);

		if (Prediction != PredictedStream)
		{
			PredictedStream = MoveTemp(Prediction);
			OnPredictionChangedDelegate.Broadcast();
		}
	}
}
