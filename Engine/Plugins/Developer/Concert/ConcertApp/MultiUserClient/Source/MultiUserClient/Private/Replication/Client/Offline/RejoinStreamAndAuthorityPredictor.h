// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "ConcertSyncSessionTypes.h"
#include "Replication/Authority/IClientAuthoritySynchronizer.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Stream/IClientStreamSynchronizer.h"
#include "Replication/Stream/MultiUserStreamId.h"

#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"

class IConcertClientWorkspace;

namespace UE::MultiUserClient::Replication
{
	/**
	 * Predicts the stream and authority state a client will have when they rejoin the session.
	 * The prediction returns the stream content in the latest EConcertSyncReplicationActivityType::LeaveReplication pertaining to ClientInfo.
	 */
	class FRejoinStreamAndAuthorityPredictor
		: public FStreamSynchronizer_Base
		, public FAuthoritySynchronizer_Base
		, public FNoncopyable
	{
	public:
		
		FRejoinStreamAndAuthorityPredictor(IConcertClientWorkspace& InWorkspace UE_LIFETIMEBOUND, FConcertClientInfo InClientInfo);
		~FRejoinStreamAndAuthorityPredictor();

		FORCEINLINE const FConcertBaseStreamInfo& GetPredictedStream() const { return PredictedStream; }
		
		//~ Begin IClientStreamSynchronizer Interface
		virtual FGuid GetStreamId() const override { return MultiUserStreamID; }
		virtual const FConcertObjectReplicationMap& GetServerState() const override { return PredictedStream.ReplicationMap; }
		virtual const FConcertStreamFrequencySettings& GetFrequencySettings() const override { return PredictedStream.FrequencySettings; }
		//~ End IClientStreamSynchronizer Interface

		//~ Begin IClientAuthoritySynchronizer Interface
		virtual bool HasAnyAuthority() const override { return !PredictedAuthority.IsEmpty(); }
		virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const override { return PredictedAuthority.Contains(ObjectPath); }
		//~ End IClientAuthoritySynchronizer Interface

		/** Broadcasts when PredictedStream changes. */
		FSimpleMulticastDelegate& OnPredictionChanged() { return OnPredictionChangedDelegate; }

	private:

		/** Used to listen for activity changes. */
		IConcertClientWorkspace& Workspace;
		/** Client for which we're predicting the state. */
		const FConcertClientInfo ClientInfo;

		/** The stream content the local client thinks the offline client will have. */
		FConcertBaseStreamInfo PredictedStream;
		/** The most up to date server state of the remote client's authority. */
		TSet<FSoftObjectPath> PredictedAuthority;

		/** Broadcasts when PredictedStream changes. */
		FSimpleMulticastDelegate OnPredictionChangedDelegate;

		/** Refreshes the prediction state if a new leave replication event is produced. */
		void OnActivityAddedOrUpdated(
			const FConcertClientInfo& ActivityClientInfo, const FConcertSyncActivity& Activity, const FStructOnScope& Summary
			);

		/** Refreshes PredicatedStream. */
		void AnalyzeHistory();
	};
}

