// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalClient.h"

#include "IConcertSyncClient.h"
#include "Replication/Authority/AuthoritySynchronizer_LocalClient.h"
#include "Replication/Submission/SubmissionWorkflow_LocalClient.h"

namespace UE::MultiUserClient::Replication
{
	FLocalClient::FLocalClient(
		FReplicationDiscoveryContainer& InDiscoveryContainer,
		FGlobalAuthorityCache& InAuthorityCache,
		UMultiUserReplicationStream& InClientStreamContent,
		TUniquePtr<IClientStreamSynchronizer> InStreamSynchronizer,
		TSharedRef<IConcertSyncClient> InClient
		)
		: FOnlineClient(
			InClient->GetConcertClient()->GetCurrentSession()->GetSessionClientEndpointId(),
			InDiscoveryContainer,
			InAuthorityCache,
			InClientStreamContent,
			MoveTemp(InStreamSynchronizer),
			MakeUnique<FAuthoritySynchronizer_LocalClient>(InClient),
			MakeUnique<FSubmissionWorkflow_LocalClient>(MoveTemp(InClient)))
		, RemoteSubmissionListener(InClient->GetConcertClient()->GetCurrentSession().ToSharedRef(), GetStreamSynchronizer(), GetSubmissionQueue())
	{}
}
