// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineClient.h"
#include "Replication/Submission/Remote/RemoteSubmissionListener.h"

namespace UE::MultiUserClient::Replication
{
	class FGlobalAuthorityCache;

	/** Holds extra information about a local replication client. */
	class FLocalClient : public FOnlineClient
	{
	public:
		
		/**
		 * @param InDiscoveryContainer Used for auto-discovering properties added to this client's stream. The caller ensures it outlives the constructed instance.
		 * @param InAuthorityCache Caches authority state of all clients. Passed to subsystems. The caller ensures it outlives the constructed instance.
		 * @param InClientStreamContent Object that this client's stream changes are to be written into. The caller ensures it outlives the constructed instance.
		 * @param InStreamSynchronizer Implementation for obtaining stream registered on server. The constructed instance takes ownership.
		 * @param InClient The local editor's running client. Passed to subsystems so they can send messages to the current session. Keeps strong reference.
		 */
		FLocalClient(
			FReplicationDiscoveryContainer& InDiscoveryContainer,
			FGlobalAuthorityCache& InAuthorityCache,
			UMultiUserReplicationStream& InClientStreamContent,
			TUniquePtr<IClientStreamSynchronizer> InStreamSynchronizer,
			TSharedRef<IConcertSyncClient> InClient
			);

	private:
		
		/** Listens for and handles for submission request made by remote clients' SubmissionWorkflow.*/
		FRemoteSubmissionListener RemoteSubmissionListener;
	};
}

