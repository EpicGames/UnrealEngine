// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineClient.h"

#include "Assets/MultiUserReplicationStream.h"
#include "Replication/ClientReplicationWidgetFactories.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Stream/Discovery/MultiUserStreamExtender.h"
#include "Replication/Stream/StreamChangeTracker.h"
#include "Replication/Submission/Data/AuthoritySubmission.h"
#include "Settings/MultiUserReplicationSettings.h"

#include "Misc/CoreDelegates.h"

namespace UE::MultiUserClient::Replication
{
	FOnlineClient::FOnlineClient(
		const FGuid& EndpointId,
		FReplicationDiscoveryContainer& InDiscoveryContainer,
		FGlobalAuthorityCache& InAuthorityCache,
		UMultiUserReplicationStream& InClientStreamContent,
		TUniquePtr<IClientStreamSynchronizer> InStreamSynchronizer,
		TUniquePtr<IClientAuthoritySynchronizer> InAuthoritySynchronizer,
		TUniquePtr<ISubmissionWorkflow> InSubmissionWorkflow
		)
		: EndpointId(EndpointId)
		, ClientStreamContent(&InClientStreamContent)
		, StreamSynchronizer(MoveTemp(InStreamSynchronizer))
		, AuthoritySynchronizer(MoveTemp(InAuthoritySynchronizer))
		, SubmissionWorkflow(MoveTemp(InSubmissionWorkflow))
		, SubmissionQueue(*SubmissionWorkflow)
		, ExternalRequestHandler(
			StreamSynchronizer->GetStreamId(),
			FGetStreamContent::CreateLambda([this](){ return &StreamSynchronizer->GetServerState(); }),
			SubmissionQueue
			)
		, StreamExtender(MakeShared<FMultiUserStreamExtender>(EndpointId, InDiscoveryContainer))
		, LocalClientEditModel(
			// Transact ClientContentStorage
			ConcertClientSharedSlate::CreateTransactionalStreamModel(
				// Read & write the stream data in ClientContentStorage
				CreateBaseStreamModel(GetClientStreamObject()->MakeReplicationMapGetterAttribute(), StreamExtender),
				*GetClientStreamObject()
				)
			)
		, LocalClientStreamDiffer(
			GetStreamSynchronizer(),
			GetClientStreamObject()->MakeReplicationMapGetterAttribute(),
			FStreamChangeTracker::FOnModifyReplicationMap::CreateLambda([this](){ GetClientStreamObject()->Modify(); })
			)
		, LocalAuthorityDiffer(EndpointId, *AuthoritySynchronizer, InAuthorityCache)
		, LocalFrequencyChangeTracker(*StreamSynchronizer)
		, ChangeRequestBuilder(EndpointId, InAuthorityCache, *StreamSynchronizer, LocalClientStreamDiffer, LocalAuthorityDiffer, LocalFrequencyChangeTracker)
		, AutoSubmissionPolicy(SubmissionQueue, ChangeRequestBuilder, LocalClientEditModel.Get(), LocalAuthorityDiffer, LocalFrequencyChangeTracker)
		, LevelModificationHandler(GetClientEditModel().Get())
	{
		LocalClientEditModel->OnObjectsChanged().AddRaw(this, &FOnlineClient::OnObjectsChanged);
		LocalClientEditModel->OnPropertiesChanged().AddRaw(this, &FOnlineClient::OnPropertiesChanged);
		LocalAuthorityDiffer.OnChangedOwnedObjects().AddRaw(this, &FOnlineClient::DeferOnModelChanged);

		SubmissionWorkflow->OnAuthorityRequestCompleted_AnyThread().AddRaw(this, &FOnlineClient::OnAuthoritySubmissionCompleted);
		StreamSynchronizer->OnServerStreamChanged().AddRaw(this, &FOnlineClient::OnServerStateChanged);

		LevelModificationHandler.OnHierarchyNeedsRefresh().AddLambda([this]()
		{
			OnHierarchyNeedsRefreshDelegate.Broadcast();
		});
	}

	FOnlineClient::~FOnlineClient()
	{
		FCoreDelegates::OnEndFrame.RemoveAll(this);
	}

	bool FOnlineClient::AllowsEditing() const
	{
		return CanEverSubmit(SubmissionWorkflow->GetUploadability()); 
	}

	void FOnlineClient::OnObjectsChanged(
		TConstArrayView<UObject*> AddedObjects,
		TConstArrayView<FSoftObjectPath> RemovedObjects,
		ConcertSharedSlate::EReplicatedObjectChangeReason ReplicatedObjectChangeReason
		)
	{
		DeferOnModelChanged();
	}

	void FOnlineClient::OnPropertiesChanged()
	{
		DeferOnModelChanged();
	}

	void FOnlineClient::OnServerStateChanged()
	{
		// Whenever this client's server state changes, the UI must be refreshed.

		// The UI adds empty actors. However, we never send them to the server...
		TSet<FSoftObjectPath> StagedObjects;
		for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : GetClientStreamObject()->ReplicationMap.ReplicatedObjects)
		{
			if (Pair.Value.PropertySelection.ReplicatedProperties.IsEmpty())
			{
				StagedObjects.Add(Pair.Key);
			}
		}
		
		// ... if the user removes the last property from the entire actor-component hierarchy, we want the hierarchy to continue to displayed...
		GetClientStreamObject()->ReplicationMap = GetStreamSynchronizer().GetServerState();
		// ... so add back the staged objects
		for (const FSoftObjectPath& StagedObject : StagedObjects)
		{
			if (const UObject* Object = StagedObject.ResolveObject())
			{
				GetClientStreamObject()->ReplicationMap.ReplicatedObjects.Add(StagedObject) = FConcertReplicatedObjectInfo::Make(*Object);
			}
		}
		// To remove the hierarchy, the user must click the actor and delete it explicitly, which will call IEditableReplicationStream::RemoveObjects on the staged objects.
		
		DeferOnModelChanged();
	}

	void FOnlineClient::DeferOnModelChanged()
	{
		if (!bHasDeferredDataChanges)
		{
			bHasDeferredDataChanges = true;
			FCoreDelegates::OnEndFrame.AddRaw(this, &FOnlineClient::ProcessOnModelChanged);
		}
	}

	void FOnlineClient::ProcessOnModelChanged()
	{
		check(bHasDeferredDataChanges);
		bHasDeferredDataChanges = false;
		FCoreDelegates::OnEndFrame.RemoveAll(this);
		
		// Could improve performance by just considering what actually changed instead of doing a full rebuild
		// This must be done before SetAuthorityIfAllowed because it uses the cache for checking whether the object has properties assigned
		LocalClientStreamDiffer.RefreshChangesCache();
		const TArray<UObject*> AddedObjects = DetermineAddedObjects();
		
		// Better UX for user: automatically take authority for newly added objects (but only if it is allowed and causes no conflicts)
		TArray<FSoftObjectPath> ObjectPaths;
		Algo::Transform(AddedObjects, ObjectPaths, [](UObject* Object){ return FSoftObjectPath(Object); });
		LocalAuthorityDiffer.SetAuthorityIfAllowed(ObjectPaths, true);
		
		// Refresh because local authority changes may no longer be valid after modifying the stream
		LocalAuthorityDiffer.RefreshChanges();
		
		// Check whether added objects are supposed to default to some special frequency settings
		ApplyDefaultFrequencySettings(AddedObjects);

		// Finally, let everybody else know.
		OnModelChangedDelegate.Broadcast();
		AutoSubmissionPolicy.ProcessAccumulatedChangesAndSubmit();
	}
	
	TArray<UObject*> FOnlineClient::DetermineAddedObjects() const
	{
		TArray<UObject*> AddedObjects;

		const FGuid MultiUserStreamId = StreamSynchronizer->GetStreamId();
		const FConcertObjectReplicationMap& CurrentState = StreamSynchronizer->GetServerState();
		for (const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& Pair : LocalClientStreamDiffer.GetCachedDeltaChange().ObjectsToPut)
		{
			const FSoftObjectPath& ObjectPath = Pair.Key.Object;
			const bool bIsNew = Pair.Key.StreamId == MultiUserStreamId && !CurrentState.ReplicatedObjects.Contains(ObjectPath);
			UObject* Object = bIsNew ? ObjectPath.ResolveObject() : nullptr;
			if (Object)
			{
				AddedObjects.Add(Object);
			}
		}

		return AddedObjects;
	}

	void FOnlineClient::ApplyDefaultFrequencySettings(const TArray<UObject*>& AddedObjects)
	{
		UMultiUserReplicationSettings* Settings = UMultiUserReplicationSettings::Get();
		for (UObject* AddedObject : AddedObjects)
		{
			const TOptional<FConcertObjectReplicationSettings> FrequencyOverride = Settings->DetermineObjectFrequencySettings(*AddedObject);
			if (FrequencyOverride)
			{
				LocalFrequencyChangeTracker.AddOverride(AddedObject, *FrequencyOverride);
			}
		}
	}

	void FOnlineClient::OnAuthoritySubmissionCompleted(const FSubmitAuthorityChangesRequest& Request, const FSubmitAuthorityChangesResponse& Response)
	{
		if (!Response.Response)
		{
			return;
		}

		// Use case: You and another client submit at the same time. You lose. Revert your local changes so the checkboxes accurately reflect the authority state.
		for (const TPair<FSoftObjectPath, FConcertStreamArray>& RejectedObjectPair : Response.Response.GetValue().RejectedObjects)
		{
			LocalAuthorityDiffer.ClearAuthorityChange({ RejectedObjectPair.Key });
		}
	}
}
