// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Authority/AuthorityChangeTracker.h"
#include "Replication/Authority/IClientAuthoritySynchronizer.h"
#include "Replication/Editor/UnrealEditor/ModifyObjectInLevelHandler.h"
#include "Replication/Misc/Frequency/FrequencyChangeTracker.h"
#include "Replication/Stream/IClientStreamSynchronizer.h"
#include "Replication/Stream/StreamChangeTracker.h"
#include "Replication/Submission/AutoSubmissionPolicy.h"
#include "Replication/Submission/ChangeRequestBuilder.h"
#include "Replication/Submission/External/ExternalClientChangeRequestHandler.h"
#include "Replication/Submission/ISubmissionWorkflow.h"
#include "Replication/Submission/Queue/SubmissionQueue.h"

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class UMultiUserReplicationStream;
class IConcertClient;

namespace UE::ConcertSharedSlate
{
	class IEditableReplicationStreamModel;
	enum class EReplicatedObjectChangeReason : uint8;
}

namespace UE::MultiUserClient::Replication
{
	class FMultiUserStreamExtender;
	class FReplicationDiscoveryContainer;
	class ISubmissionWorkflow;
	
	/**
	 * Initializes systems that are common for local or remote clients.
	 * This class' responsibility is to initialize all systems that exist for the life time of a client in a session.
	 */
	class FOnlineClient : public FNoncopyable
	{
	public:
		
		/**
		 * @param EndpointId The endpoint ID this instance corresponds to.
		 * @param InDiscoveryContainer Used for auto-discovering properties added to this client's stream. The caller ensures it outlives the constructed instance.
		 * @param InAuthorityCache Caches authority state of all clients. Passed to subsystems. The caller ensures it outlives the constructed instance.
		 * @param InClientStreamContent Object that this client's stream changes are to be written into. The caller ensures it outlives the constructed instance.
		 * @param InStreamSynchronizer Implementation for obtaining stream registered on server. The constructed instance takes ownership.
		 * @param InAuthoritySynchronizer Implementation for obtaining the client's authority state on server. The constructed instance takes ownership.
		 * @param InSubmissionWorkflow Implementation for for changing client streams and authority on the server. The constructed instance takes ownership.
		 */
		FOnlineClient(
			const FGuid& EndpointId,
			FReplicationDiscoveryContainer& InDiscoveryContainer UE_LIFETIMEBOUND,
			FGlobalAuthorityCache& InAuthorityCache UE_LIFETIMEBOUND,
			UMultiUserReplicationStream& InClientStreamContent UE_LIFETIMEBOUND,
			TUniquePtr<IClientStreamSynchronizer> InStreamSynchronizer,
			TUniquePtr<IClientAuthoritySynchronizer> InAuthoritySynchronizer,
			TUniquePtr<ISubmissionWorkflow> InSubmissionWorkflow
			);
		~FOnlineClient();

		UMultiUserReplicationStream* GetClientStreamObject() const { return ClientStreamContent; }
		FMultiUserStreamExtender& GetStreamExtender() { return *StreamExtender; }
		/**
		 * This is used so the UI can construct the IReplicationStreamEditor.
		 * @see CreateBaseStreamEditor and FCreateEditorParams.
		 * 
		 * @note You must make sure to release this object when this FReplicationClient is destroyed.
		 * Listen for events on the owning FReplicationClientManager::OnPreRemoteClientRemoved and FMultiUserReplicationManager::OnLeaveSession
		 * @see FReplicationClientManager and FMultiUserReplicationManager
		 */
		TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> GetClientEditModel() const { return LocalClientEditModel; }
		IClientStreamSynchronizer& GetStreamSynchronizer() const { return *StreamSynchronizer.Get(); }
		IClientAuthoritySynchronizer& GetAuthoritySynchronizer() const { return *AuthoritySynchronizer.Get(); }
		
		const FStreamChangeTracker& GetStreamDiffer() const { return LocalClientStreamDiffer; }
		FStreamChangeTracker& GetStreamDiffer() { return LocalClientStreamDiffer; }
		
		const FAuthorityChangeTracker& GetAuthorityDiffer() const { return LocalAuthorityDiffer; }
		FAuthorityChangeTracker& GetAuthorityDiffer() { return LocalAuthorityDiffer; }

		/** @return The object with which you can submit changes. You can use it to listen to general events. If you want to submit changes, you should prefer to use the SubmissionQueue. */
		const ISubmissionWorkflow& GetSubmissionWorkflow() const { return *SubmissionWorkflow; }
		ISubmissionWorkflow& GetSubmissionWorkflow() { return *SubmissionWorkflow; }
		/** @return Implements simple game-thread based queue for SubmissionWorkflow. Only one submission can be in progress at any given time. */
		FSubmissionQueue& GetSubmissionQueue() { return SubmissionQueue; }
		/** @return Gets the adapter handling requests received by IMultiUserReplication::EnqueuesChanges */
		FExternalClientChangeRequestHandler& GetExternalRequestHandler() { return ExternalRequestHandler; }

		/** @return The endpoint ID of this client in the Concert session. */
		const FGuid& GetEndpointId() const { return EndpointId; }
		/** @return Whether it is allowed to edit the stream and authority for this client. */
		bool AllowsEditing() const;
		
		bool operator==(const FOnlineClient& Client) const { return GetEndpointId() == Client.GetEndpointId(); }

		DECLARE_MULTICAST_DELEGATE(FOnModelExternallyChanged);
		/**
		 * Broadcast when the data underlying the model has changed for any reason:
		 * - Edited directly by client
		 * - Changed by transaction
		 * - Server state changed
		 */
		FOnModelExternallyChanged& OnModelChanged() { return OnModelChangedDelegate; }
		
		DECLARE_MULTICAST_DELEGATE(FOnHiearchyNeedsRefresh);
		/** Broadcasts when the hierarchy may have changed. */
		FOnHiearchyNeedsRefresh& OnHierarchyNeedsRefresh() { return OnHierarchyNeedsRefreshDelegate; }
		
	private:

		/** This client's Concert Endpoint ID. */
		const FGuid EndpointId;
		
		/** The state of the server is synched up with this object and displayed in the UI. */
		TObjectPtr<UMultiUserReplicationStream> ClientStreamContent;
		
		/** Keeps the client's stream state on the server in sync. */
		TUniquePtr<IClientStreamSynchronizer> StreamSynchronizer;
		/** Keeps the client's authority state on the server in sync.*/
		TUniquePtr<IClientAuthoritySynchronizer> AuthoritySynchronizer;
		/** Handles the logic of submitting and reverting for this client. */
		TUniquePtr<ISubmissionWorkflow> SubmissionWorkflow;

		/** Allows systems to queue pending submissions to SubmissionWorkflow in case a submission is in progress. */
		FSubmissionQueue SubmissionQueue;
		/** Enqueues external change requests into the SubmissionQueue. */
		FExternalClientChangeRequestHandler ExternalRequestHandler;

		/** Injected into LocalClientEditModel during construction. Adds additional objects and properties when a new object is added to the stream. */
		TSharedRef<FMultiUserStreamExtender> StreamExtender;
		/**
		 * Used to detect changes made to the client's config by the local editor.
		 * Those changes can later be applied to the client.
		 * 
		 * The SessionContent UObject must be kept alive as long as this model lives.
		 * 
		 * It should be noted that the UI keeps a strong reference to this model.
		 * The UI is destroyed right before FReplicationClient, which is guaranteed by FMultiUserReplicationManager.
		 * @see FMultiUserReplicationManager::OnLeaveSession
		 */
		TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> LocalClientEditModel;
		
		/** Tracks changes made to server's state of the client's streams and prepares to upload them using StreamSynchronizer. */
		FStreamChangeTracker LocalClientStreamDiffer;
		/** Tracks changes made to the client's authority state. */
		FAuthorityChangeTracker LocalAuthorityDiffer;
		/** Tracks enqueued local changes to object replication frequency. */
		FFrequencyChangeTracker LocalFrequencyChangeTracker;
		
		/** Shared logic for building stream and authority change requests based on local change made. */
		FChangeRequestBuilder ChangeRequestBuilder;
		/** Automatically submits changes as they are made by the user. */
		FAutoSubmissionPolicy AutoSubmissionPolicy;
		
		/** Handles updating the local client's stream when the level is modified, e.g. when an actor is removed. */
		ConcertClientSharedSlate::FModifyObjectInLevelHandler LevelModificationHandler;

		/** Broadcasts when the hierarchy may have changed. */
		FOnHiearchyNeedsRefresh OnHierarchyNeedsRefreshDelegate;

		/**
		 * Broadcast when the data underlying the model has changed for any reason:
		 * - Edited directly by client
		 * - Changed by transaction
		 * - Server state changed
		 */
		FOnModelExternallyChanged OnModelChangedDelegate;

		/** Whether there are any deferred data changes to process. */
		bool bHasDeferredDataChanges = false;

		// Respond to model changing
		void OnObjectsChanged(TConstArrayView<UObject*> AddedObjects, TConstArrayView<FSoftObjectPath> RemovedObjects, ConcertSharedSlate::EReplicatedObjectChangeReason ReplicatedObjectChangeReason);
		void OnPropertiesChanged();

		void OnServerStateChanged();

		/** Defers rebuilding operations, such as refreshing state and calling OnModelChanged, in case there are multiple changes in the same frame. */
		void DeferOnModelChanged();
		/** Processes all changes that have happened to the stream this frame. */
		void ProcessOnModelChanged();
		
		/** Determines the objects that were newly added. */
		TArray<UObject*> DetermineAddedObjects() const;

		/** Applies default replication frequency settings specified in the Multi User settings */
		void ApplyDefaultFrequencySettings(const TArray<UObject*>& ChangeData);
		
		/** Removes authority if request fails */
		void OnAuthoritySubmissionCompleted(const FSubmitAuthorityChangesRequest& Request, const FSubmitAuthorityChangesResponse& Response);
	};
}

