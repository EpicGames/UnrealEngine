// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineClientManager.h"

#include "Assets/MultiUserReplicationStream.h"
#include "IConcertSyncClient.h"
#include "LocalClient.h"
#include "RemoteClient.h"
#include "Replication/Stream/StreamSynchronizer_LocalClient.h"
#include "Replication/Stream/MultiUserStreamId.h"

#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace UE::MultiUserClient::Replication
{
	namespace Private
	{
		static UMultiUserReplicationStream* MakeClientContent()
		{
			UMultiUserReplicationStream* Stream = NewObject<UMultiUserReplicationStream>(GetTransientPackage(), NAME_None, RF_Transient | RF_Transactional);
			Stream->StreamId = MultiUserStreamID;
			return Stream;
		}
	}
	
	FOnlineClientManager::FOnlineClientManager(
		const TSharedRef<IConcertSyncClient>& InClient,
		const TSharedRef<IConcertClientSession>& InSession,
		FReplicationDiscoveryContainer& InRegisteredExtenders,
		FStreamAndAuthorityQueryService& InQueryService
		)
		: ConcertClient(InClient)
		, Session(InSession)
		, RegisteredExtenders(InRegisteredExtenders)
		, QueryService(InQueryService)
		, AuthorityCache(*this)
		, LocalClient([this, InClient]()
		{
			UMultiUserReplicationStream* ClientPreset = Private::MakeClientContent();
			return FLocalClient(RegisteredExtenders, AuthorityCache, *ClientPreset, MakeUnique<FStreamSynchronizer_LocalClient>(InClient, ClientPreset->StreamId), InClient);
		}())
		, ReassignmentLogic(*this)
	{
		AuthorityCache.RegisterEvents();
		InSession->OnSessionClientChanged().AddRaw(this, &FOnlineClientManager::OnSessionClientChanged);

		for (const FGuid& ClientEndpointId : InSession->GetSessionClientEndpointIds())
		{
			CreateRemoteClient(ClientEndpointId);
		}
	}

	FOnlineClientManager::~FOnlineClientManager()
	{
		if (const TSharedPtr<IConcertClientSession> SessionPin = Session.Pin())
		{
			SessionPin->OnSessionClientChanged().RemoveAll(this);
		}
	}

	TArray<TNonNullPtr<const FRemoteClient>> FOnlineClientManager::GetRemoteClients() const
	{
		TArray<TNonNullPtr<const FRemoteClient>> Result;
		Algo::Transform(RemoteClients, Result, [](const TUniquePtr<FRemoteClient>& Client) -> TNonNullPtr<const FRemoteClient>
		{
			return Client.Get();
		});
		return Result;
	}

	TArray<TNonNullPtr<FRemoteClient>> FOnlineClientManager::GetRemoteClients()
	{
		TArray<TNonNullPtr<FRemoteClient>> Result;
		Algo::Transform(RemoteClients, Result, [](const TUniquePtr<FRemoteClient>& Client) -> TNonNullPtr<FRemoteClient>
		{
			return Client.Get();
		});
		return Result;
	}

	TArray<TNonNullPtr<const FOnlineClient>> FOnlineClientManager::GetClients(TFunctionRef<bool(const FOnlineClient& Client)> Predicate) const
	{
		TArray<TNonNullPtr<const FOnlineClient>> Result;
		ForEachClient([&Predicate, &Result](const FOnlineClient& Client)
		{
			if (Predicate(Client))
			{
				Result.Add(&Client);
			}
			return EBreakBehavior::Continue;
		});
		return Result;
	}

	const FRemoteClient* FOnlineClientManager::FindRemoteClient(const FGuid& EndpointId) const
	{
		const TUniquePtr<FRemoteClient>* Client = RemoteClients.FindByPredicate([&EndpointId](const TUniquePtr<FRemoteClient>& Client)
		{
			return Client->GetEndpointId() == EndpointId;
		});
		return Client ? Client->Get() : nullptr;
	}

	void FOnlineClientManager::ForEachClient(TFunctionRef<EBreakBehavior(const FOnlineClient&)> ProcessClient) const
	{
		if (ProcessClient(GetLocalClient()) == EBreakBehavior::Break)
		{
			return;
		}
		for (const TNonNullPtr<const FRemoteClient>& RemoteClient : GetRemoteClients())
		{
			if (ProcessClient(*RemoteClient) == EBreakBehavior::Break)
			{
				return;
			}
		}
	}

	void FOnlineClientManager::ForEachClient(TFunctionRef<EBreakBehavior(FOnlineClient&)> ProcessClient)
	{
		const FOnlineClientManager* ConstThis = this;
		ConstThis->ForEachClient([&ProcessClient](const FOnlineClient& Client)
		{
			// const_cast safe here because remote clients are never const and GetLocalClient() is not const here since this overload is non-const
			return ProcessClient(const_cast<FOnlineClient&>(Client));
		});
	}

	void FOnlineClientManager::AddReferencedObjects(FReferenceCollector& Collector)
	{
		ForEachClient([&Collector](const FOnlineClient& Client)
		{
			TObjectPtr<UMultiUserReplicationStream> Content = Client.GetClientStreamObject();
			Collector.AddReferencedObject(Content);
			ensureMsgf(Content == Client.GetClientStreamObject(), TEXT("Did not expect reference to be obliterated"));
			return EBreakBehavior::Continue;
		});
	}

	void FOnlineClientManager::OnSessionClientChanged(IConcertClientSession&, EConcertClientStatus NewStatus, const FConcertSessionClientInfo& ClientInfo)
	{
		const FGuid& ClientEndpointId = ClientInfo.ClientEndpointId;
		switch (NewStatus)
		{
			
		case EConcertClientStatus::Connected:
			CreateRemoteClient(ClientEndpointId);
			break;
			
		case EConcertClientStatus::Disconnected:
			{
				const int32 Index = RemoteClients.IndexOfByPredicate(
					[&ClientEndpointId](const TUniquePtr<FRemoteClient>& Client)
					{
						return Client->GetEndpointId() == ClientEndpointId;
					});
				if (!ensure(RemoteClients.IsValidIndex(Index)))
				{
					return;
				}

				{
					const TUniquePtr<FRemoteClient> Client = MoveTemp(RemoteClients[Index]);
					OnPreRemoteClientRemovedDelegate.Broadcast(*Client.Get());
					RemoteClients.RemoveAtSwap(Index);
				}
				// We want to broadcast after the client has been fully cleaned up
				OnRemoteClientsChangedDelegate.Broadcast();
			}
			break;
			
		case EConcertClientStatus::Updated:
			break;
		default: checkNoEntry();
		}
	}
	
	void FOnlineClientManager::CreateRemoteClient(const FGuid& ClientEndpointId, bool bBroadcastDelegate)
	{
		TUniquePtr<FRemoteClient> RemoteClientPtr = MakeUnique<FRemoteClient>(
			ClientEndpointId,
			RegisteredExtenders,
			ConcertClient->GetConcertClient(),
			AuthorityCache,
			*Private::MakeClientContent(),
			QueryService
			);
		FRemoteClient& RemoteClient = *RemoteClientPtr;
		RemoteClients.Emplace(
			MoveTemp(RemoteClientPtr)
			);

		if (bBroadcastDelegate)
		{
			OnPostRemoteClientAddedDelegate.Broadcast(RemoteClient);
			OnRemoteClientsChangedDelegate.Broadcast();
		}
	}
}
