// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "LocalClient.h"
#include "OnlineClient.h"
#include "Replication/Misc/GlobalAuthorityCache.h"
#include "Replication/Submission/MultiEdit/ReassignObjectPropertiesLogic.h"

#include "UObject/GCObject.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/NonNullPointer.h"

class IConcertClientSession;
class IConcertSyncClient;
namespace UE::ConcertSharedSlate { class IEditableReplicationStreamModel; }
enum class EConcertClientStatus : uint8;
struct FConcertSessionClientInfo;

namespace UE::MultiUserClient::Replication
{
	class FRemoteClient;
	class FReplicationDiscoveryContainer;
	class FOnlineClient;
	class FStreamAndAuthorityQueryService;
	class FStreamChangeTracker;

	/**
	 * Keeps track of clients connected to replication via a MU session.
	 * It synchronizes their stream data in a UMultiUserReplicationSessionPreset.
	 * 
	 * This class is instantiated only for as long as the local client is in a Concert session and is owned by FMultiUserReplicationManager,
	 * which drives the lifetime.
	 */
	class FOnlineClientManager
		: public FGCObject
		, public FNoncopyable
	{
	public:
		
		/**
		 * @param InClient The local client. The owning FMultiUserReplicationManager ensures it outlives the constructed instance.
		 * @param InSession The session to observe. The owning FMultiUserReplicationManager ensures it outlives the constructed instance.
		 * @param InRegisteredExtenders Used for auto-discovering properties added to this client's stream. Passed to the clients. The owning FMultiUserReplicationManager ensures it outlives the constructed instance.
		 * @param InQueryService Used for querying info about clients' streams. Passed to the clients. The owning FMultiUserReplicationManager ensures it outlives the constructed instance.
		 */
		FOnlineClientManager(
			const TSharedRef<IConcertSyncClient>& InClient,
			const TSharedRef<IConcertClientSession>& InSession,
			FReplicationDiscoveryContainer& InRegisteredExtenders UE_LIFETIMEBOUND,
			FStreamAndAuthorityQueryService& InQueryService UE_LIFETIMEBOUND
			);
		virtual ~FOnlineClientManager() override;

		const FLocalClient& GetLocalClient() const { return LocalClient; }
		FLocalClient& GetLocalClient() { return LocalClient; }
		TArray<TNonNullPtr<const FRemoteClient>> GetRemoteClients() const;
		TArray<TNonNullPtr<FRemoteClient>> GetRemoteClients();

		/** @return Gets all clients given a predicate */
		TArray<TNonNullPtr<const FOnlineClient>> GetClients(TFunctionRef<bool(const FOnlineClient& Client)> Predicate = [](const auto&){ return true; }) const;
		
		const FGlobalAuthorityCache& GetAuthorityCache() const { return AuthorityCache; }
		FGlobalAuthorityCache& GetAuthorityCache() { return AuthorityCache; }
		
		const FReassignObjectPropertiesLogic& GetReassignmentLogic() const { return ReassignmentLogic; }
		FReassignObjectPropertiesLogic& GetReassignmentLogic() { return ReassignmentLogic; }

		/** Util for finding a remote client by its EndpointId. */
		const FRemoteClient* FindRemoteClient(const FGuid& EndpointId) const;
		FRemoteClient* FindRemoteClient(const FGuid& EndpointId)
		{
			const FOnlineClientManager* ConstThis = this;
			return const_cast<FRemoteClient*>(ConstThis->FindRemoteClient(EndpointId));
		}

		/** Util for finding a local or remote client by its EndpointId. */
		const FOnlineClient* FindClient(const FGuid& EndpointId) const
		{
			return GetLocalClient().GetEndpointId() == EndpointId
				? &GetLocalClient()
				: static_cast<const FOnlineClient*>(FindRemoteClient(EndpointId));
		}
		FOnlineClient* FindClient(const FGuid& EndpointId)
		{
			const FOnlineClientManager* ConstThis = this;
			return const_cast<FOnlineClient*>(ConstThis->FindClient(EndpointId));
		}

		/** Iterates through every client */
		void ForEachClient(TFunctionRef<EBreakBehavior(const FOnlineClient&)> ProcessClient) const;
		/** Iterates through every client */
		void ForEachClient(TFunctionRef<EBreakBehavior(FOnlineClient&)> ProcessClient);
		
		DECLARE_MULTICAST_DELEGATE(FRemoteClientsChanged);
		/** Called when RemoteClients changes. Called after OnPostRemoteClientAdded. */
		FRemoteClientsChanged& OnRemoteClientsChanged() { return OnRemoteClientsChangedDelegate; }
		
		DECLARE_MULTICAST_DELEGATE_OneParam(FRemoteClientDelegate, FRemoteClient&);
		/** Called just after a remote client is has been added to RemoteClients. Called before OnRemoteClientsChanged. */
		FRemoteClientDelegate& OnPostRemoteClientAdded() { return OnPostRemoteClientAddedDelegate; }
		/** Called just before a remote client is about to be removed from RemoteClients. */
		FRemoteClientDelegate& OnPreRemoteClientRemoved() { return OnPreRemoteClientRemovedDelegate; }
		
		//~ Begin FGCObject Interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FReplicationStreamSynchronizer"); }
		//~ End FGCObject Interface
		
	private:

		/** The local Concert client */
		TSharedRef<IConcertSyncClient> ConcertClient;
		/**
		 * The session the local client is in.
		 * 
		 * This FReplicationClientManager's owner is supposed to make sure this FReplicationClientManager is destroyed
		 * when the session shuts down.
		 */
		const TWeakPtr<IConcertClientSession> Session;

		/**
		 * Passed to clients for constructing for the purposes of auto-discovering properties and additional objects when adding properties to the stream.
		 * The owning FMultiUserReplicationManager ensures it outlives this FReplicationClientManager instance.
		 */
		FReplicationDiscoveryContainer& RegisteredExtenders;
		
		/**
		 * Sends FConcertReplication_QueryReplicationInfo_Request in regular intervals.
		 * Shared by all remote clients so all requests are bundled reducing the number of network requests. 
		 */
		FStreamAndAuthorityQueryService& QueryService;
		/** Keeps a cache of object to owning clients. */
		FGlobalAuthorityCache AuthorityCache;
		
		/** Manages the local client */
		FLocalClient LocalClient;
		/**
		 * Manages remote clients. Updated when client connects or disconnects to the active session.
		 * UI keeps references to systems inside the client so it is TSharedRef in case TArray is reallocated.
		 */
		TArray<TUniquePtr<FRemoteClient>> RemoteClients;
		
		/** Called when RemoteClients changes. */
		FRemoteClientsChanged OnRemoteClientsChangedDelegate;
		/** Called when RemoteClients changes. Called after OnPostRemoteClientAdded. */
		FRemoteClientDelegate OnPostRemoteClientAddedDelegate;
		/** Called just before a remote client is about to be removed from RemoteClients. */
		FRemoteClientDelegate OnPreRemoteClientRemovedDelegate; 

		/** Used for transferring ownership from multiple clients to one. Used by multi client view. */
		FReassignObjectPropertiesLogic ReassignmentLogic;
		
		/** Updates RemoteClients depending on the change. */
		void OnSessionClientChanged(IConcertClientSession&, EConcertClientStatus NewStatus, const FConcertSessionClientInfo& ClientInfo);
		
		/** Shared logic for creating a remote client. */
		void CreateRemoteClient(const FGuid& ClientEndpointId, bool bBroadcastDelegate = true);
	};
}

