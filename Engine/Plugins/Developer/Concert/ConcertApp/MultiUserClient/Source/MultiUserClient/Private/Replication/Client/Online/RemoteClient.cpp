// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteClient.h"

#include "IConcertClient.h"
#include "Assets/MultiUserReplicationClientPreset.h"
#include "Replication/Authority/AuthoritySynchronizer_RemoteClient.h"
#include "Replication/Stream/StreamSynchronizer_RemoteClient.h"
#include "Replication/Submission/Remote/SubmissionWorkflow_RemoteClient.h"

namespace UE::MultiUserClient::Replication
{
	FRemoteClient::FRemoteClient(
		const FGuid& InConcertClientId,
		FReplicationDiscoveryContainer& InDiscoveryContainer,
		TSharedRef<IConcertClient> InClient,
		FGlobalAuthorityCache& InAuthorityCache,
		UMultiUserReplicationStream& InClientStreamContent,
		FStreamAndAuthorityQueryService& QueryService
		)
		: FOnlineClient(
			InConcertClientId,
			InDiscoveryContainer,
			InAuthorityCache,
			InClientStreamContent,
			MakeUnique<FStreamSynchronizer_RemoteClient>(InConcertClientId, QueryService),
			MakeUnique<FAuthoritySynchronizer_RemoteClient>(InConcertClientId,QueryService),
			MakeUnique<FSubmissionWorkflow_RemoteClient>(InClient->GetCurrentSession().ToSharedRef(), InConcertClientId))
	{}
}
