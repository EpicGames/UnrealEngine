// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineClient.h"

#include "HAL/Platform.h"

class IConcertClient;

namespace UE::MultiUserClient::Replication
{
	class FStreamAndAuthorityQueryService;
	
	/** Holds extra information about a remote replication client. */
	class FRemoteClient : public FOnlineClient
	{
	public:
		
		/**
		 * @param InConcertClientId The endpoint ID this instance corresponds to.
		 * @param InDiscoveryContainer Used for auto-discovering properties added to this client's stream. The caller ensures it outlives the constructed instance.
		 * @param InClient The local editor's running client. Passed to subsystems so they can send messages to the current session. Keeps strong reference.
		 * @param InAuthorityCache Caches authority state of all clients. Passed to subsystems. The caller ensures it outlives the constructed instance.
		 * @param InClientStreamContent Object that this client's stream changes are to be written into. The caller ensures it outlives the constructed instance.
		 * @param QueryService Service shared across remote clients which periodically asks the server about this client's state. The caller ensures it outlives the constructed instance.
		 */
		FRemoteClient(
			const FGuid& InConcertClientId,
			FReplicationDiscoveryContainer& InDiscoveryContainer UE_LIFETIMEBOUND,
			TSharedRef<IConcertClient> InClient,
			FGlobalAuthorityCache& InAuthorityCache UE_LIFETIMEBOUND,
			UMultiUserReplicationStream& InClientStreamContent UE_LIFETIMEBOUND,
			FStreamAndAuthorityQueryService& QueryService UE_LIFETIMEBOUND
			);
	};
}

