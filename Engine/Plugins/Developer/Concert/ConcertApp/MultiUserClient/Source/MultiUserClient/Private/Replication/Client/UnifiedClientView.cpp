// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnifiedClientView.h"

#include "IConcertSyncClient.h"
#include "Offline/OfflineClient.h"
#include "Offline/OfflineClientManager.h"
#include "Online/OnlineClientManager.h"
#include "Replication/Client/ClientUtils.h"

#include "Misc/Optional.h"

namespace UE::MultiUserClient::Replication
{
	FUnifiedClientView::FUnifiedClientView(
		IConcertSyncClient& SyncClient,
		FOnlineClientManager& OnlineClientManager,
		FOfflineClientManager& OfflineClientManager
		)
		: SyncClient(SyncClient)
		, OnlineClientManager(OnlineClientManager)
		, OfflineClientManager(OfflineClientManager)
		, StreamCache(*this, OnlineClientManager, OfflineClientManager)
	{
		// Technically it's enough to subscribe only to OfflineClientManager because that changes when OnlineClientManager changes...
		// but we'll just be safe for now.
		OnlineClientManager.OnRemoteClientsChanged().AddRaw(this, &FUnifiedClientView::BroadcastOnClientsChanged);
		OfflineClientManager.OnClientsChanged().AddRaw(this, &FUnifiedClientView::BroadcastOnClientsChanged);
	}

	FUnifiedClientView::~FUnifiedClientView()
	{
		OnlineClientManager.OnRemoteClientsChanged().RemoveAll(this);
		OfflineClientManager.OnClientsChanged().RemoveAll(this);
	}

	TArray<FGuid> FUnifiedClientView::GetClients() const
	{
		TArray<FGuid> Result;
		ForEachClient([&Result](const FGuid& ClientId)
		{
			Result.Add(ClientId);
			return EBreakBehavior::Continue;
		});
		return Result;
	}

	TArray<FGuid> FUnifiedClientView::GetOnlineClients() const
	{
		TArray<FGuid> Result;
		ForEachOnlineClient([&Result](const FGuid& ClientId)
		{
			Result.Add(ClientId);
			return EBreakBehavior::Continue;
		});
		return Result;
	}

	FGuid FUnifiedClientView::GetLocalClient() const
	{
		return OnlineClientManager.GetLocalClient().GetEndpointId();
	}

	TOptional<FConcertClientInfo> FUnifiedClientView::GetClientInfoByEndpoint(const FGuid& EndpointId) const
	{
		FConcertClientInfo ClientInfo;
		const bool bIsOnline = ClientUtils::GetClientDisplayInfo(*SyncClient.GetConcertClient(), EndpointId, ClientInfo);
		if (bIsOnline)
		{
			return ClientInfo;
		}
		
		if (const FOfflineClient* OfflineClient = OfflineClientManager.FindClient(EndpointId))
		{
			return OfflineClient->GetClientInfo();
		}
		return {};
	}

	TOptional<EClientType> FUnifiedClientView::GetClientType(const FGuid& EndpointId) const
	{
		if (OnlineClientManager.GetLocalClient().GetEndpointId() == EndpointId)
		{
			return EClientType::Local;
		}
		if (OnlineClientManager.FindClient(EndpointId))
		{
			return EClientType::Remote;
		}
		if (OfflineClientManager.FindClient(EndpointId))
		{
			return EClientType::Offline;
		}
		return {};
	}

	TSharedPtr<const ConcertSharedSlate::IReplicationStreamModel> FUnifiedClientView::GetClientStreamById(const FGuid& EndpointId) const
	{
		if (const FOnlineClient* Client = OnlineClientManager.FindClient(EndpointId))
		{
			return Client->GetClientEditModel();
		}
		if (const FOfflineClient* Client = OfflineClientManager.FindClient(EndpointId))
		{
			return Client->GetStreamModel();
		}
		return nullptr;
	}

	TSharedPtr<ConcertSharedSlate::IEditableReplicationStreamModel> FUnifiedClientView::GetEditableClientStreamById(const FGuid& EndpointId) const
	{
		const FOnlineClient* Client = OnlineClientManager.FindClient(EndpointId);
		return Client ? Client->GetClientEditModel().ToSharedPtr() : nullptr;
	}
}
