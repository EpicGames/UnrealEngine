// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineClient.h"
#include "Online/OnlineClientManager.h"
#include "UnifiedStreamCache.h"

#include "HAL/Platform.h"
#include "Offline/OfflineClientManager.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;

struct FConcertClientInfo;
template<typename OptionalType> struct TOptional;

namespace UE::ConcertSharedSlate
{
	class IEditableReplicationStreamModel;
	class IReplicationStreamModel;
}

namespace UE::MultiUserClient::Replication
{
	enum class EClientType : uint8
	{
		Local,
		Remote,
		Offline
	};
	inline bool IsOnlineClient(EClientType Type) { return Type == EClientType::Local || Type == EClientType::Remote; }
	inline bool IsOfflineClient(EClientType Type) { return Type == EClientType::Offline; }
	
	/**
	 * Uses FOnlineClientManager and FOfflineClientManager to preset a unified interface of querying client info to the UI.
	 * This abstraction effectively implements the Adapter design pattern.
	 *
	 * This task of this class is not to hide the fact that there are online and offline clients: the goal is to unify the interface and put shared
	 * logic that needs to handle online and offline client interactions here.
	 */
	class FUnifiedClientView : public FNoncopyable
	{
	public:
		
		FUnifiedClientView(
			IConcertSyncClient& SyncClient UE_LIFETIMEBOUND,
			FOnlineClientManager& OnlineClientManager UE_LIFETIMEBOUND,
			FOfflineClientManager& OfflineClientManager UE_LIFETIMEBOUND
			);
		~FUnifiedClientView();

		/** Enumerates the endpoint ID of every user; in the case of offline clients, only the latest associated ID is listed.. */
		template<typename TLambda> requires std::is_invocable_r_v<EBreakBehavior, TLambda, const FGuid&>
		void ForEachClient(TLambda&& Callback) const;
		/** Enumerates the endpoint ID of every online client. */
		template<typename TLambda> requires std::is_invocable_r_v<EBreakBehavior, TLambda, const FGuid&>
		void ForEachOnlineClient(TLambda&& Callback) const;

		TArray<FGuid> GetClients() const;
		TArray<FGuid> GetOnlineClients() const;
		FGuid GetLocalClient() const;
		
		/** @return Client info associated with EndpointId. */
		TOptional<FConcertClientInfo> GetClientInfoByEndpoint(const FGuid& EndpointId) const;
		/** @return Gets the type of client */
		TOptional<EClientType> GetClientType(const FGuid& EndpointId) const;
		
		/** @return The stream model of the client associated with EndpointId. */
		TSharedPtr<const ConcertSharedSlate::IReplicationStreamModel> GetClientStreamById(const FGuid& EndpointId) const;
		/** @return The editable stream model of the client associated with EndpointId, if EndpointId refers to an online client. */
		TSharedPtr<ConcertSharedSlate::IEditableReplicationStreamModel> GetEditableClientStreamById(const FGuid& EndpointId) const;

		const FUnifiedStreamCache& GetStreamCache() const { return StreamCache; }
		FUnifiedStreamCache& GetStreamCache() { return StreamCache; }

		/** Broadcasts when the list of online of offline clients has changed. */
		FSimpleMulticastDelegate OnClientsChanged() { return OnClientsChangedDelegate; }

	private:

		/** Used to look up client info from session. */
		IConcertSyncClient& SyncClient;

		FOnlineClientManager& OnlineClientManager;
		FOfflineClientManager& OfflineClientManager;
		
		/** Broadcasts when the list of online of offline clients has changed. */
		FSimpleMulticastDelegate OnClientsChangedDelegate;
		
		/**
		 * Unified interface for querying stream content efficiently.
		 * Important: Has dependency on OnClientsChangedDelegate hence this must be constructed after.
		 */
		FUnifiedStreamCache StreamCache;

		void BroadcastOnClientsChanged() const { OnClientsChangedDelegate.Broadcast(); }
	};

	template <typename TLambda> requires std::is_invocable_r_v<EBreakBehavior, TLambda, const FGuid&>
	void FUnifiedClientView::ForEachClient(TLambda&& Callback) const
	{
		EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
		OnlineClientManager.ForEachClient([&Callback, &BreakBehavior](const FOnlineClient& Client)
		{
			BreakBehavior = Callback(Client.GetEndpointId());
			return BreakBehavior;
		});
		if (BreakBehavior == EBreakBehavior::Break)
		{
			return;
		}
		OfflineClientManager.ForEachClient([&Callback](const FOfflineClient& Client)
		{
			return Callback(Client.GetLastAssociatedEndpoint());
		});
	}

	template <typename TLambda> requires std::is_invocable_r_v<EBreakBehavior, TLambda, const FGuid&>
	void FUnifiedClientView::ForEachOnlineClient(TLambda&& Callback) const
	{
		OnlineClientManager.ForEachClient([&Callback](const FOnlineClient& Client)
		{
			return Callback(Client.GetEndpointId());
		});
	}
}

