// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnifiedClientViewExtensions.h"

#include "ConcertMessageData.h"
#include "Widgets/Client/SClientName.h"

#include "Containers/UnrealString.h"
#include "Misc/Optional.h"

namespace UE::MultiUserClient::Replication
{
	FString GetClientDisplayString(const FUnifiedClientView& ClientView, const FGuid& ClientEndpointId, EDisplayStringOption Option)
	{
		return GetClientDisplayText(ClientView, ClientEndpointId, Option).ToString();
	}

	FText GetClientDisplayText(const FUnifiedClientView& ClientView, const FGuid& ClientEndpointId, EDisplayStringOption Option)
	{
		const TOptional<FConcertClientInfo> ClientInfo = ClientView.GetClientInfoByEndpoint(ClientEndpointId);
		if (!ClientInfo)
		{
			return FText::GetEmpty();
		}


		const FText ParenthesesContent = Option == EDisplayStringOption::NameAndParentheses
			? GetParenthesesContent(ClientView, ClientEndpointId)
			: FText::GetEmpty();
		return ConcertSharedSlate::SClientName::GetDisplayText(*ClientInfo, ParenthesesContent);
	}

	FText GetParenthesesContent(const FUnifiedClientView& ClientView, const FGuid& ClientEndpointId)
	{
		const TOptional<EClientType> ClientType = ClientView.GetClientType(ClientEndpointId);
		if (!ClientType)
		{
			return FText::GetEmpty();
		}

		if (ClientType == EClientType::Local)
		{
			return ConcertSharedSlate::ParenthesesClientNameContent::LocalClient;
		}
			
		return IsOfflineClient(*ClientType)
			? ConcertSharedSlate::ParenthesesClientNameContent::OfflineClient
			: FText::GetEmpty();
	}

	TOptional<FGuid> FindClientIdByStream(const FUnifiedClientView& ClientView, const ConcertSharedSlate::IReplicationStreamModel& SearchedStream)
	{
		TOptional<FGuid> Result;
		ClientView.ForEachClient([&ClientView, &SearchedStream, &Result](const FGuid& EndpointId)
		{
			const TSharedPtr<const ConcertSharedSlate::IReplicationStreamModel> ResolvedStream = ClientView.GetClientStreamById(EndpointId);
			if (ResolvedStream.Get() == &SearchedStream)
			{
				Result = EndpointId;
				return EBreakBehavior::Break;
			}
			return EBreakBehavior::Continue;
		});
		return Result;
	}

	TArray<FGuid> GetSortedOnlineClients(const FUnifiedClientView& ClientView, EDisplayStringOption Option)
	{
		TMap<FGuid, FString> ClientToDisplayString;
		TArray<FGuid> SortedClients = ClientView.GetOnlineClients();
		for (const FGuid& EndpointId : SortedClients)
		{
			ClientToDisplayString.Add(EndpointId, GetClientDisplayString(ClientView, EndpointId, Option));
		}

		SortedClients.Sort([&ClientToDisplayString](const FGuid& Left, const FGuid& Right)
		{
			return ClientToDisplayString[Left] < ClientToDisplayString[Right];
		});
		return SortedClients;
	}
}
