// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnifiedClientView.h"
#include "Widgets/Client/ClientInfoDelegate.h"

#include "HAL/Platform.h"
#include "Internationalization/Text.h"

class FString;
struct FGuid;

namespace UE::ConcertSharedSlate { class IReplicationStreamModel; }

namespace UE::MultiUserClient::Replication
{
	enum class EDisplayStringOption : uint8
	{
		/** E.g. "OfflineUserName", "LocalUserName", and "RemoteUserName". */
		NameOnly,
		/** E.g. "OfflineUserName(Offline)", "LocalUserName(Me)", and "RemoteUserName". */
		NameAndParentheses
	};

	/** @return Gets the display string to use for the given client; use it e.g. for search. Returns empty FString if the client is not found.*/
	FString GetClientDisplayString(
		const FUnifiedClientView& ClientView,
		const FGuid& ClientEndpointId,
		EDisplayStringOption Option = EDisplayStringOption::NameAndParentheses
		);
	/** @return Gets the display text to use for displaying the given client. Returns empty FText if the client is not found. */
	FText GetClientDisplayText(
		const FUnifiedClientView& ClientView,
		const FGuid& ClientEndpointId,
		EDisplayStringOption Option = EDisplayStringOption::NameAndParentheses
		);
	
	/** @return "You" or "Offline" depending on ClientEndpointId. */
	FText GetParenthesesContent(const FUnifiedClientView& ClientView, const FGuid& ClientEndpointId);

	/** @return Endpoint ID of the client that has the given Stream. */
	TOptional<FGuid> FindClientIdByStream(const FUnifiedClientView& ClientView, const ConcertSharedSlate::IReplicationStreamModel& SearchedStream);

	/** @return Gets all online clients sorted by display name. */
	TArray<FGuid> GetSortedOnlineClients(const FUnifiedClientView& ClientView, EDisplayStringOption Option = EDisplayStringOption::NameAndParentheses);
	
	/** @return Delegate that will get client info from online clients if possible and falls back to offline clients. */
	inline ConcertSharedSlate::FGetOptionalClientInfo MakeOnlineThenOfflineClientInfoGetter(const FUnifiedClientView& ClientView UE_LIFETIMEBOUND)
	{
		return ConcertSharedSlate::FGetOptionalClientInfo::CreateLambda([&ClientView](const FGuid& ClientEndpointId)
		{
			return ClientView.GetClientInfoByEndpoint(ClientEndpointId);
		});
	}

	/** @return Delegate that will return (You) or (Offline) for a client. */
	inline ConcertSharedSlate::FGetClientParenthesesContent MakeLocalAndOfflineParenthesesContentGetter(const FUnifiedClientView& ClientView UE_LIFETIMEBOUND)
	{
		return ConcertSharedSlate::FGetClientParenthesesContent::CreateLambda([&ClientView](const FGuid& ClientEndpointId)
		{
			return GetParenthesesContent(ClientView, ClientEndpointId);
		});
	}
}
