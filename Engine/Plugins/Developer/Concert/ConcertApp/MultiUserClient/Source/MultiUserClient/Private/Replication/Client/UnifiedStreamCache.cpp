// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnifiedStreamCache.h"

#include "Replication/Data/ConcertPropertySelection.h"

#include "Offline/OfflineClientManager.h"
#include "Online/OnlineClientManager.h"
#include "UnifiedClientView.h"

namespace UE::MultiUserClient::Replication::UnifiedStreamCache
{
	static bool HasNonOverlappingProperties(
		const FSoftObjectPath& ObjectPath,
		const FConcertReplicatedObjectInfo& ObjectInfo,
		const FGlobalAuthorityCache& OnlineCache,
		FOnlineClientManager& OnlineClientManager
		)
	{
		TSet<FConcertPropertyChain> NonOverlappingProperties = ObjectInfo.PropertySelection.ReplicatedProperties;
		
		OnlineCache.ForEachClientWithObjectInStream(ObjectPath,
			[&ObjectPath, &OnlineClientManager, &NonOverlappingProperties](const FGuid& ClientId)
			{
				const FOnlineClient* Client = OnlineClientManager.FindClient(ClientId);
				const FConcertReplicatedObjectInfo* OnlineObjectInfo = Client
					? Client->GetStreamSynchronizer().GetServerState().ReplicatedObjects.Find(ObjectPath)
					: nullptr;
				if (!ensure(OnlineObjectInfo))
				{
					return EBreakBehavior::Continue;
				}

				NonOverlappingProperties = NonOverlappingProperties.Difference(OnlineObjectInfo->PropertySelection.ReplicatedProperties);
				return NonOverlappingProperties.IsEmpty() ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});
		
		return !NonOverlappingProperties.IsEmpty();
	}
}

namespace UE::MultiUserClient::Replication
{
	FUnifiedStreamCache::FUnifiedStreamCache(
		FUnifiedClientView& InOwner,
		FOnlineClientManager& InOnlineClientManager,
		FOfflineClientManager& InOfflineClientManager
		)
		: Owner(InOwner)
		, OnlineClientManager(InOnlineClientManager)
		, OfflineClientManager(InOfflineClientManager)
	{
		Owner.OnClientsChanged().AddRaw(this, &FUnifiedStreamCache::BroadcastOnCacheChanged);
		OnlineClientManager.GetAuthorityCache().OnCacheChanged().AddRaw(this, &FUnifiedStreamCache::OnOnlineCacheChanged);
	}

	FUnifiedStreamCache::~FUnifiedStreamCache()
	{
		Owner.OnClientsChanged().RemoveAll(this);
		OnlineClientManager.GetAuthorityCache().OnCacheChanged().RemoveAll(this);
	}

	const FConcertObjectReplicationMap* FUnifiedStreamCache::GetReplicationMapFor(const FGuid& ClientId) const
	{
		if (const FOnlineClient* Client = OnlineClientManager.FindClient(ClientId))
		{
			return &Client->GetStreamSynchronizer().GetServerState();
		}
		if (const FOfflineClient* Client = OfflineClientManager.FindClient(ClientId))
		{
			return &Client->GetPredictedStream().ReplicationMap;
		}
		return nullptr;
	}

	void FUnifiedStreamCache::EnumerateClientsWithObject(
		const FSoftObjectPath& ObjectPath,
		const TFunctionRef<EBreakBehavior(const FGuid& ClientId)>& Callback,
		EClientEnumerationMode Option
		) const
	{
		EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
		const FGlobalAuthorityCache& OnlineCache = OnlineClientManager.GetAuthorityCache();
		OnlineCache.ForEachClientWithObjectInStream(ObjectPath, [&BreakBehavior, &Callback](const FGuid& ClientId)
		{
			BreakBehavior = Callback(ClientId);
			return BreakBehavior;
		});

		if (BreakBehavior == EBreakBehavior::Break
			|| Option == EClientEnumerationMode::SkipOfflineClients)
		{
			return;
		}

		const bool bShouldCheckNonOverlapping = Option == EClientEnumerationMode::SkipOfflineClientsThatFullyOverlapWithOnlineClients;
		OfflineClientManager.ForEachClient(
			[this, &ObjectPath, &Callback, bShouldCheckNonOverlapping, &OnlineCache](const FOfflineClient& Client)
			{
				const FConcertReplicatedObjectInfo* ObjectInfo = Client.GetPredictedStream().ReplicationMap.ReplicatedObjects.Find(ObjectPath);
				const bool bContainsObject = ObjectInfo != nullptr;
				const bool bShouldListObject = bContainsObject
					&& (!bShouldCheckNonOverlapping
						|| UnifiedStreamCache::HasNonOverlappingProperties(ObjectPath, *ObjectInfo, OnlineCache, OnlineClientManager));
				return bShouldListObject ? Callback(Client.GetLastAssociatedEndpoint()) : EBreakBehavior::Continue;
			});
	}

	void FUnifiedStreamCache::EnumerateClientsWithObjectAndProperty(
		const FSoftObjectPath& ObjectPath,
		const FConcertPropertyChain& Property,
		const TFunctionRef<EBreakBehavior(const FGuid& ClientId)>& Callback,
		EClientEnumerationMode Option
		) const
	{
		EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
		bool bEncounteredOnlineClient = false;
		const FGlobalAuthorityCache& OnlineCache = OnlineClientManager.GetAuthorityCache();
		OnlineCache.ForEachClientWithObjectInStream(ObjectPath,
			[this, &ObjectPath, &Property, &Callback, &BreakBehavior, &bEncounteredOnlineClient](const FGuid& ClientId)
			{
				const FOnlineClient* Client = OnlineClientManager.FindClient(ClientId);
				if (!Client)
				{
					return EBreakBehavior::Continue;
				}

				const FConcertReplicatedObjectInfo* ObjectInfo = Client->GetStreamSynchronizer().GetServerState().ReplicatedObjects.Find(ObjectPath);
				const bool bShouldInvoke = ObjectInfo && ObjectInfo->PropertySelection.ReplicatedProperties.Contains(Property);
				if (!bShouldInvoke)
				{
					return EBreakBehavior::Continue;
				}

				bEncounteredOnlineClient = true;
				BreakBehavior = Callback(ClientId);
				return BreakBehavior;
			}
		);

		if (BreakBehavior == EBreakBehavior::Break
			|| Option == EClientEnumerationMode::SkipOfflineClients
			// If there is an online client, the offline clients are not supposed to be displayed because the caller requested no property overlaps.
			|| (Option == EClientEnumerationMode::SkipOfflineClientsThatFullyOverlapWithOnlineClients && bEncounteredOnlineClient))
		{
			return;
		}
		
		OfflineClientManager.ForEachClient(
			[this, &ObjectPath, &Callback](const FOfflineClient& Client)
			{
				const FConcertReplicatedObjectInfo* ObjectInfo = Client.GetPredictedStream().ReplicationMap.ReplicatedObjects.Find(ObjectPath);
				const bool bContainsObject = ObjectInfo != nullptr;
				return bContainsObject ? Callback(Client.GetLastAssociatedEndpoint()) : EBreakBehavior::Continue;
			});
	}
}
