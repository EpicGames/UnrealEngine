// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/FunctionFwd.h"

enum class EBreakBehavior : uint8;
struct FConcertObjectReplicationMap;
struct FConcertPropertyChain;
struct FGuid;
struct FSoftObjectPath;

namespace UE::MultiUserClient::Replication
{
	class FOfflineClientManager;
	class FOnlineClientManager;
	class FUnifiedClientView;

	enum class EClientEnumerationMode : uint8
	{
		All,
		/** Skip offline clients if all their properties are also owned by online clients. */
		SkipOfflineClientsThatFullyOverlapWithOnlineClients,
		/** Offline clients are supposed to be skipped completely. */
		SkipOfflineClients
	};
	
	/** Access point for querying stream content of both online and offline clients through a single interface. */
	class FUnifiedStreamCache : public FNoncopyable
	{
	public:
		
		FUnifiedStreamCache(
			FUnifiedClientView& InOwner UE_LIFETIMEBOUND,
			FOnlineClientManager& InOnlineClientManager UE_LIFETIMEBOUND,
			FOfflineClientManager& InOfflineClientManager UE_LIFETIMEBOUND
			);
		~FUnifiedStreamCache();

		/** @return Gets the replication map for the specified client. */
		const FConcertObjectReplicationMap* GetReplicationMapFor(const FGuid& ClientId) const;
		
		/**
		 * Lists out all clients that have ObjectPath registered in their stream.
		 * Does not mean that they have authority over it.
		 */
		void EnumerateClientsWithObject(
			const FSoftObjectPath& ObjectPath,
			const TFunctionRef<EBreakBehavior(const FGuid& ClientId)>& Callback,
			EClientEnumerationMode Option = EClientEnumerationMode::SkipOfflineClientsThatFullyOverlapWithOnlineClients
			) const;

		/**
		 * Lists out all clients that have ObjectPath with Property assigned to it registered in their stream.
		 * Does not mean that they have authority over it.
		 */
		void EnumerateClientsWithObjectAndProperty(
			const FSoftObjectPath& ObjectPath,
			const FConcertPropertyChain& Property,
			const TFunctionRef<EBreakBehavior(const FGuid& ClientId)>& Callback,
			EClientEnumerationMode Option = EClientEnumerationMode::SkipOfflineClientsThatFullyOverlapWithOnlineClients
			) const;

		/** Broadcasts when the content of a client has changed. */
		FSimpleMulticastDelegate& OnCacheChanged() { return OnCacheChangedDelegate; }

	private:
		
		FUnifiedClientView& Owner;
		FOnlineClientManager& OnlineClientManager;
		FOfflineClientManager& OfflineClientManager;

		/** Broadcasts when the content of a client has changed. */
		FSimpleMulticastDelegate OnCacheChangedDelegate;
		
		void OnOnlineCacheChanged(const FGuid&) const { BroadcastOnCacheChanged(); }
		void BroadcastOnCacheChanged() const { OnCacheChangedDelegate.Broadcast(); }
	};
}

