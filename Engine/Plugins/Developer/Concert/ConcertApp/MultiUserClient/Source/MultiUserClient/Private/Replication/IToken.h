// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::MultiUserClient::Replication
{
	/**
	 * The token "pattern" used by systems that start non-cancellable, latent operations (like TFutures from Concert responses):
	 * 1. The system creates a TSharedRef<FToken> and holds on to it for the system's entire lifetime.
	 * 2. Latent operation capture a TWeakPtr to the token and the this pointer of the system
	 * 3. When the latent operation finishes, it checks whether the token is still valid.
	 *	- If yes, that means the system is valid, too, so the captured this can be used safely.
	 *	- If no, discard the operation result.
	 *
	 * This pattern offers an alternative to forcing the system to be wrapped in a shared pointer itself and forcing it to inherit from TSharedFromThis.
	 * Often systems should not care about whether they are heap allocated or not - in such cases using TSharedFromThis would be an anti-pattern.
	 */
	class FToken : public TSharedFromThis<FToken>
	{
	public:
		static TSharedRef<FToken> Make() { return MakeShared<FToken>(); }
	};
}