// Copyright Epic Games, Inc. All Rights Reserved.
#include "AnalyticsHandler.h"

#include "EngineAnalytics.h"
#include "IConcertClient.h"
#include "Misc/EBreakBehavior.h"
#include "Replication/Client/Online/OnlineClientManager.h"

namespace UE::MultiUserClient::Replication
{
	FAnalyticsHandler::FAnalyticsHandler( IConcertClient& InClient, FOnlineClientManager& InOnlineClientManager )
		:
		  Client(InClient)
		, OnlineClientManager(InOnlineClientManager)
	{
		OnlineClientManager.OnPostRemoteClientAdded().AddRaw(this, &FAnalyticsHandler::OnClientAdded);
	}
	FAnalyticsHandler::~FAnalyticsHandler()
	{
		OnlineClientManager.OnPostRemoteClientAdded().RemoveAll(this);
		OnlineClientManager.ForEachClient(
			[this](FOnlineClient& OnlineClient)
			{
				OnlineClient.OnModelChanged().RemoveAll(this);
				return EBreakBehavior::Continue;
			});
	}

	void FAnalyticsHandler::OnClientContentChanged()
	{
		if (!bSentReplicationAnalyticsData)
		{
			if (!FEngineAnalytics::IsAvailable())
			{
				return;
			}

			TSharedPtr<IConcertClientSession> CurrentSession = Client.GetCurrentSession();
			check(CurrentSession.IsValid());

			const FConcertSessionInfo& SessionInfo = CurrentSession->GetSessionInfo();
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SessionID"), SessionInfo.SessionId.ToString()));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Usage.MultiUser.ReplicationUsed"), EventAttributes);
			bSentReplicationAnalyticsData = true;
		}
	}

	void FAnalyticsHandler::OnClientAdded(FRemoteClient& InRemoteClient)
	{
		InRemoteClient.OnModelChanged().AddRaw(this, &FAnalyticsHandler::OnClientContentChanged);
	}
}
