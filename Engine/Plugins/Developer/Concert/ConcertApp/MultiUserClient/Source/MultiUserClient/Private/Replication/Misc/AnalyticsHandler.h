// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

class IConcertClient;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
	class FRemoteClient;

	/**
	 * Handles any changes to the replication data model and send an analytics event with session id.
	 */
	class FAnalyticsHandler : public FNoncopyable
	{
	public:
		FAnalyticsHandler(IConcertClient& InClient UE_LIFETIMEBOUND,
						  FOnlineClientManager& InReplicationClientManager UE_LIFETIMEBOUND);
		~FAnalyticsHandler();

	private:
		/** Used to get client session information. */
		IConcertClient& Client;

		/** Used to detect when a client's state changes. */
		FOnlineClientManager& OnlineClientManager;

		/** Called when a remote client joins. */
		void OnClientAdded(FRemoteClient& InRemoteClient);

		/** Called when remote client changes the replication model. */
		void OnClientContentChanged();

		/** Have we sent our analytics event for replication. */
		bool bSentReplicationAnalyticsData = false;
	};
}
