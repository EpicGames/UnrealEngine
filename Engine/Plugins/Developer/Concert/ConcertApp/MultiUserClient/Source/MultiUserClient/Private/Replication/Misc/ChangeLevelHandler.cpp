// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChangeLevelHandler.h"

#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"

#include "Engine/Engine.h"
#include "Engine/World.h"

namespace UE::MultiUserClient::Replication
{
	FChangeLevelHandler::FChangeLevelHandler(
		IConcertSyncClient& Client,
		ConcertSharedSlate::IEditableReplicationStreamModel& UpdatedModel
		)
		: Client(Client)
		, UpdatedModel(UpdatedModel)
	{
		if (ensure(GEngine))
		{
			GEngine->OnWorldDestroyed().AddRaw(this, &FChangeLevelHandler::OnWorldDestroyed);
			GEngine->OnWorldAdded().AddRaw(this, &FChangeLevelHandler::OnWorldAdded);
		}
	}

	FChangeLevelHandler::~FChangeLevelHandler()
	{
		if (GEngine)
		{
			GEngine->OnWorldDestroyed().RemoveAll(this);
			GEngine->OnWorldAdded().RemoveAll(this);
		}
	}

	void FChangeLevelHandler::OnWorldDestroyed(UWorld* World)
	{
		if (IsValidWorldType(World) && !IsConcertHotReloadingWorld())
		{
			// Remember the current map so when the next world
			PreviousWorldPath = FSoftObjectPath(World);
		}
	}

	void FChangeLevelHandler::OnWorldAdded(UWorld* World) const
	{
		if (IsValidWorldType(World) && !IsConcertHotReloadingWorld()
			// If user reloaded the same map - keep around all settings
			&& PreviousWorldPath != World)
		{
			UpdatedModel.Clear();
		}
	}

	bool FChangeLevelHandler::IsConcertHotReloadingWorld() const
	{
		const TSharedPtr<IConcertClientWorkspace>& Workspace = Client.GetWorkspace();
		const FName PackagePath = PreviousWorldPath.GetLongPackageFName();
		const bool bIsReloadingWorld = Workspace && Workspace->IsReloadingPackage(PackagePath);
		return bIsReloadingWorld;
	}

	bool FChangeLevelHandler::IsValidWorldType(UWorld* World) const
	{
		return World->WorldType == EWorldType::Editor;
	}
}