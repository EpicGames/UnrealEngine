// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/SoftObjectPath.h"

class IConcertSyncClient;
class UWorld;

namespace UE::ConcertSharedSlate { class IEditableReplicationStreamModel; }

namespace UE::MultiUserClient::Replication
{
	/**
	 * Handles a client opening a new level.
	 * Upon leaving a level, all replicated objects assigned to that client in that level are removed.
	 */
	class FChangeLevelHandler : public FNoncopyable
	{
	public:
		
		/**
		 * @param Client The Concert client. Used to get the workspace for checking against hot reloading. The caller ensures that it outlives the constructed object.
		 * @param UpdatedModel The client model to update when the local editor changes maps. The caller ensures that it outlives the constructed object.
		 */
		FChangeLevelHandler(IConcertSyncClient& Client UE_LIFETIMEBOUND, ConcertSharedSlate::IEditableReplicationStreamModel& UpdatedModel UE_LIFETIMEBOUND);
		~FChangeLevelHandler();

	private:
		
		/** The Concert client. Used to get the workspace for checking against hot reloading. */
		IConcertSyncClient& Client;
		/** The client model to update when the local editor changes maps. */
		ConcertSharedSlate::IEditableReplicationStreamModel& UpdatedModel;

		/** Path to the previously open world */
		FSoftObjectPath PreviousWorldPath;
		
		void OnWorldDestroyed(UWorld* World);
		void OnWorldAdded(UWorld* World) const;

		/**
		 * Hot reload: When a remote user saves the world, Concert will reload the world's package on the other clients.
		 * In that case, a temporary world called "Untitled" is created. During hot reload, OnWorldDestroyed and OnWorldAdded calls should be ignored.
		 * 
		 * @return Whether concert is hot reloading the currently open world.
		 */
		bool IsConcertHotReloadingWorld() const;
		bool IsValidWorldType(UWorld* World) const;
	};
}
