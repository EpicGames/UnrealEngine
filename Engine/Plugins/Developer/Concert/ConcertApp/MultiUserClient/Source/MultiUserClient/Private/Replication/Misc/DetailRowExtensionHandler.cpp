// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailRowExtensionHandler.h"

#include "IConcertSyncClient.h"
#include "IDetailTreeNode.h"
#include "MultiUserReplicationStyle.h"
#include "Replication/Client/ClientUtils.h"
#include "Replication/Editor/View/IMultiReplicationStreamEditor.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"
#include "Replication/MultiUserReplicationManager.h"
#include "Replication/PropertyChainUtils.h"
#include "Widgets/ActiveSession/Replication/Client/Multi/Columns/AssignProperty/AssignPropertyModel.h"
#include "Widgets/ActiveSession/SActiveSessionRoot.h"

#include "Algo/RemoveIf.h"
#include "GameFramework/Actor.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "PropertyPath.h"

#define LOCTEXT_NAMESPACE "FDetailRowExtensionHandler"

namespace UE::MultiUserClient::Replication::DetailRowExtensionHandler
{
	static UStruct* GetOwningStruct(const FPropertyInfo& LeafPropertyInfo)
	{
		const FProperty* LeafProperty = LeafPropertyInfo.Property.Get();
		return LeafProperty ? Cast<UStruct>(LeafProperty->GetOwnerUObject()) : nullptr;
	}

	static int32 FindIndexOfRootPropertyOfLeafmostObject(const FPropertyPath& PropertyPath)
	{
		UObject* LeafOwningObject = GetOwningStruct(PropertyPath.GetLeafMostProperty());
		for (int32 i = 0; i < PropertyPath.GetNumProperties(); ++i)
		{
			UObject* CurrentObject = GetOwningStruct(PropertyPath.GetPropertyInfo(i));
			if (CurrentObject == LeafOwningObject)
			{
				return i;
			}
		}
		return INDEX_NONE;
	}

	struct FPathInfo
	{
		TArray<FName> Path;
		UStruct* LeafClass = nullptr;

		bool IsValid() const { return LeafClass != nullptr; }
	};
	
	/**
	 * Retrieves the property path leading to the deepest (leaf-most) property, which can be used to construct an FConcertPropertyChain.
	 * Example input: 
	 *  [0] { "StaticMeshComponent", Owner = AStaticMeshActor }
	 *  [1] { "RelativeLocation", Owner = UStaticMeshComponent }
	 *  [2] { "X", Owner = UStaticMeshComponent }
	 * 
	 * Example output: 
	 *  [0 ]"RelativeLocation",	
	 *  [1] "X"
	 * It skipped the StaticMeshComponent property.
	 */
	static FPathInfo GetPropertyPath(const TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		FPathInfo Result;
		
		const TSharedPtr<FPropertyPath> PropertyPath = PropertyHandle ? PropertyHandle->CreateFPropertyPath() : nullptr;
		if (!PropertyPath || PropertyPath->GetNumProperties() == 0)
		{
			return Result;
		}

		/**
		 * Depending on the selection in the details panel, the property path will vary. 
		 * The path originates from the root object in the details view and includes all intermediate properties leading to the selected property.
		 * 
		 * Example 1:
		 *  If you select a StaticMeshActor in the Outliner, and PropertyHandle refers to "RelativeLocation", the path would be:
		 *  { 
		 *    [0] { "StaticMeshComponent", Owner = AStaticMeshActor },
		 *    [1] { "RelativeLocation", Owner = UStaticMeshComponent }
		 *  }.
		 * 
		 * Example 2:
		 *  If you select the StaticMeshComponent directly from the component hierarchy, the path would be:
		 *  { 
		 *    [0] { "RelativeLocation", Owner = UStaticMeshComponent }
		 *  }.
		 * 
		 * This behavior also applies to nested subobjects.
		 * Summary: We adjust StartIndex to point to the root property of the deepest UObject in the property path.
		 */
		const int32 StartIndex = FindIndexOfRootPropertyOfLeafmostObject(*PropertyPath);
		if (const bool bIsInvalidIndex = StartIndex < 0 || StartIndex >= PropertyPath->GetNumProperties())
		{
			return Result;
		}

		UStruct* LeafObject = GetOwningStruct(PropertyPath->GetPropertyInfo(StartIndex));
		checkf(LeafObject, TEXT("Valid StartIndex was returned but the corresponding UObject is invalid!"));
		Result.LeafClass = LeafObject;
		
		for (int32 i = StartIndex; i < PropertyPath->GetNumProperties(); ++i)
		{
			if (FProperty* Property = PropertyPath->GetPropertyInfo(i).Property.Get(); ensure(Property))
			{
				Result.Path.Add(Property->GetFName());
			}
			else
			{
				return {};
			}
		}
		
		return Result;
	}
	
	static void SetMinimumRightColumnWidth(bool bIsConnectedToReplication)
	{
		const FName DetailsTabIdentifiers[] =
			{ "LevelEditorSelectionDetails", "LevelEditorSelectionDetails2", "LevelEditorSelectionDetails3", "LevelEditorSelectionDetails4" };
		
		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		for (const FName& DetailsTabIdentifier : DetailsTabIdentifiers)
		{
			if (const TSharedPtr<IDetailsView> DetailsView = PropertyEditor.FindDetailView(DetailsTabIdentifier))
			{
				const float DeltaChang = bIsConnectedToReplication ? 22 : -22;
				DetailsView->SetRightColumnMinWidth(DetailsView->GetRightColumnMinWidth() + DeltaChang);
				DetailsView->ForceRefresh();
			}
		}
	}
	
	static void OnConnectionStateChanged(EMultiUserReplicationConnectionState NewState)
	{
		const bool bIsConnected = NewState == EMultiUserReplicationConnectionState::Connected;
		SetMinimumRightColumnWidth(bIsConnected);
	}

	static TArray<UObject*> GetObjects(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		TArray<UObject*> Result;

		if (InPropertyHandle)
		{
			InPropertyHandle->GetOuterObjects(Result);
		}
		
		return Result;
	}

	static bool IsObjectHierarchyReplicated(
		const TWeakPtr<FMultiUserReplicationManager>& InWeakReplicationManager, TConstArrayView<UObject*> InObjects)
	{
		const TSharedPtr<FMultiUserReplicationManager> ReplicationManager = InWeakReplicationManager.Pin();
		const FOnlineClientManager* OnlineClientManager = ReplicationManager ? ReplicationManager->GetOnlineClientManager() : nullptr;
		if (!OnlineClientManager // Unset when not in any replicated session.
			|| InObjects.IsEmpty()) 
		{
			return false;
		}

		return Algo::AnyOf(InObjects, [OnlineClientManager](UObject* Object)
		{
			if (!Object)
			{
				return false;
			}
			AActor* OwningActor = Object->IsA<AActor>() ? Cast<AActor>(Object) : Object->GetTypedOuter<AActor>();
			return OwningActor && OnlineClientManager->GetAuthorityCache().IsObjectOrChildReferenced(OwningActor);
		});
	}
}

namespace UE::MultiUserClient::Replication
{
	FDetailRowExtensionHandler::FDetailRowExtensionHandler(
		const TSharedRef<IConcertSyncClient>& InClient,
		const TSharedRef<FMultiUserReplicationManager>& InReplicationManager,
		FGetConcertBrowserWidget InGetOrInvokeBrowserTabDelegate
		)
		: WeakClient(InClient)
		, WeakReplicationManager(InReplicationManager)
		, GetOrInvokeBrowserTabDelegate(MoveTemp(InGetOrInvokeBrowserTabDelegate))
		, OnReplicationConnectionChangedHandle(
			// Without this, the right column will be too small to host all icons (Reset to Default, MU, Sequencer, etc.) - so make it wider if needed.
			InReplicationManager->OnReplicationConnectionStateChanged().AddStatic(&DetailRowExtensionHandler::OnConnectionStateChanged)
			)
	{
		check(GetOrInvokeBrowserTabDelegate.IsBound());
		
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		EditModule.GetGlobalRowExtensionDelegate().AddRaw(
			this, &FDetailRowExtensionHandler::RegisterExtensionHandler
			);
	}

	FDetailRowExtensionHandler::~FDetailRowExtensionHandler()
	{
		if (FPropertyEditorModule* EditModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			EditModule->GetGlobalRowExtensionDelegate().RemoveAll(this);
		}

		if (const TSharedPtr<FMultiUserReplicationManager> ReplicationManager = WeakReplicationManager.Pin())
		{
			ReplicationManager->OnReplicationConnectionStateChanged().Remove(OnReplicationConnectionChangedHandle);
		}
	}

	void FDetailRowExtensionHandler::RegisterExtensionHandler(
		const FOnGenerateGlobalRowExtensionArgs& Args,
		TArray<FPropertyRowExtensionButton>& OutExtensionButtons
		)
	{
		const TSharedPtr<IPropertyHandle> PropertyHandle = Args.PropertyHandle;
		if (!PropertyHandle.IsValid())
		{
			return;
		}

		// Only add the buttons if the object or one of its outers is being replicated. 
		// Adding them to more than that is overkill and can even interfere with details panels where it is inappropriate (Blueprint Editor, etc.).
		const bool bIsReplicated = DetailRowExtensionHandler::IsObjectHierarchyReplicated(
			WeakReplicationManager, DetailRowExtensionHandler::GetObjects(Args.PropertyHandle)
			);
		if (!bIsReplicated)
		{
			return;
		}
		
		FPropertyRowExtensionButton& CreateKey = OutExtensionButtons.AddDefaulted_GetRef();
		CreateKey.Icon = FSlateIcon(FMultiUserReplicationStyle::GetStyleSetName(), TEXT("MultiUser.Icons.AddProperty.Small"));
		CreateKey.Label = LOCTEXT("AddProperty.Label", "Replicate property");
		CreateKey.ToolTip = TAttribute<FText>::CreateRaw(this, &FDetailRowExtensionHandler::GetToolTipText, PropertyHandle);
		CreateKey.UIAction = FUIAction(
			FExecuteAction::CreateRaw(this, &FDetailRowExtensionHandler::OnAssignPropertyClicked, PropertyHandle),
			FCanExecuteAction::CreateRaw(this, &FDetailRowExtensionHandler::CanExecuteAssignPropertyClick, PropertyHandle),
			FGetActionCheckState(),
			FIsActionButtonVisible::CreateRaw(this, &FDetailRowExtensionHandler::IsAssignPropertyButtonVisible, PropertyHandle)
		);
	}
	
	FText FDetailRowExtensionHandler::GetToolTipText(TSharedPtr<IPropertyHandle> PropertyHandle) const
	{
		FText Reason;
		return CanAssignPropertyWithReason(PropertyHandle, &Reason)
			? LOCTEXT("AddProperty.Label.Tooltip", "Assign the property to yourself for replication in Multi-User.")
			: Reason;
	}

	void FDetailRowExtensionHandler::OnAssignPropertyClicked(TSharedPtr<IPropertyHandle> PropertyHandle) const
	{
		using namespace DetailRowExtensionHandler;
		const TSharedPtr<FMultiUserReplicationManager> ReplicationManager = WeakReplicationManager.Pin();
		
		if (!ensure(ReplicationManager) || !CanExecuteAssignPropertyClick(PropertyHandle))
		{
			return;
		}
		
		const FPathInfo PathInfo = GetPropertyPath(PropertyHandle);
		const TOptional<FConcertPropertyChain> PropertyChain = PathInfo.IsValid()
			? FConcertPropertyChain::CreateFromPath(*PathInfo.LeafClass, PathInfo.Path)
			: TOptional<FConcertPropertyChain>{};

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		// Handle multi-edit: only assign to those objects that do not have the property owned by any client, yet.
		OuterObjects.SetNum(
			Algo::RemoveIf(OuterObjects, [this, &PathInfo](const UObject* Object){ return !IsPropertyNotYetAssigned(*Object, PathInfo.Path);})
		);
		
		if (PropertyChain
			&& !OuterObjects.IsEmpty()
			&& ensure(ReplicationManager->GetConnectionState() == EMultiUserReplicationConnectionState::Connected))
		{
			TArray<TSoftObjectPtr<>> ObjectsToAssignTo;
			Algo::Transform(OuterObjects, ObjectsToAssignTo, [](UObject* Object){ return Object; });
			
			MultiStreamColumns::FAssignPropertyModel::AssignPropertyTo(
				*ReplicationManager->GetUnifiedClientView(),
				ReplicationManager->GetOnlineClientManager()->GetLocalClient().GetEndpointId(),
				ObjectsToAssignTo,
				*PropertyChain
				);

			// To make the user aware of what they've just done, show the UI if it's closed.
			SelectObjectsInReplicationUI(OuterObjects);
		}
	}

	bool FDetailRowExtensionHandler::IsAssignPropertyButtonVisible(TSharedPtr<IPropertyHandle> PropertyHandle) const
	{
		const FProperty* Property = PropertyHandle->GetProperty();
		const bool bIsReplicatableProperty = Property && ConcertSyncCore::PropertyChain::IsReplicatableProperty(*Property);
		return bIsReplicatableProperty && IsConnectedToReplication();
	}

	bool FDetailRowExtensionHandler::IsConnectedToReplication() const
	{
		const TSharedPtr<FMultiUserReplicationManager> ReplicationManager = WeakReplicationManager.Pin();
		return ensure(ReplicationManager) && ReplicationManager->GetConnectionState() == EMultiUserReplicationConnectionState::Connected;
	}

	void FDetailRowExtensionHandler::SelectObjectsInReplicationUI(TConstArrayView<UObject*> ObjectsToSelect) const
	{
		TArray<TSoftObjectPtr<>> ActorsToSelect;
		Algo::Transform(ObjectsToSelect, ActorsToSelect, [](UObject* Object)
		{
			// IReplicationStreamViewer::SetSelectedObjects only allows selecting top-level objects, i.e. actors.
			// Subobjects will be shown implicitly by the bottom panel,.
			return Object->IsA<AActor>() ? Object : Object->GetTypedOuter<AActor>();
		});
		
		const TSharedPtr<SConcertBrowser> Browser = GetOrInvokeBrowserTabDelegate.Execute();
		if (!ensure(Browser))
		{
			return;
		}
		
		const TSharedPtr<SActiveSessionRoot> SessionRoot = GetActiveSessionWidgetFromBrowser(*Browser);
		const TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor> ReplicationEditor = GetReplicationStreamEditorWidgetFromBrowser(*Browser);
		if (SessionRoot && ReplicationEditor)
		{
			SessionRoot->OpenTab(EMultiUserTab::Replication);
			ReplicationEditor->GetEditorBase().SetSelectedObjects(ActorsToSelect);
		}
	}

#define SET_REASON(X) if (OutReason){ *OutReason = X; }
	bool FDetailRowExtensionHandler::CanAssignPropertyWithReason(
		TSharedPtr<IPropertyHandle> PropertyHandle,
		FText* OutReason
		) const
	{
		using namespace DetailRowExtensionHandler;
		
		const TSharedPtr<FMultiUserReplicationManager> ReplicationManager = WeakReplicationManager.Pin();
		if (ensure(ReplicationManager) && ReplicationManager->GetConnectionState() != EMultiUserReplicationConnectionState::Connected)
		{
			SET_REASON(LOCTEXT("Reason.NotInSession", "You're not in any Multi-User session."));
			return false;
		}

		// Do not construct FConcertPropertyChain: it's expensive since it iterates the property hierarchy.
		// This function is called every tick for every shown property. 
		const FPathInfo PathInfo = GetPropertyPath(PropertyHandle);
		if (!PathInfo.IsValid())
		{
			SET_REASON(LOCTEXT("Reason.InvalidProperty", "This property cannot be replicated"));
			return false;
		}

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		if (OuterObjects.IsEmpty())
		{
			// Not sure what would cause this to happen - never hit it while testing.
			SET_REASON(LOCTEXT("Reason.NoObjects", "That object in the details panel is outdated."));
			return false;
		}

		// Handle multi-edit: if at least one property can be assigned, return true...
		const bool bIsAnyUnowned = Algo::AnyOf(OuterObjects, [this, &PathInfo](const UObject* Object)
		{
			return IsPropertyNotYetAssigned(*Object, PathInfo.Path);
		});
		if (bIsAnyUnowned)
		{
			return true;
		}

		// ... at this point we know none of the properties can be assigned. We'll try to give the user a descriptive reason.
		if (OuterObjects.Num() > 1)
		{
			SET_REASON(LOCTEXT("OwnedByMultiple", "You're editing multiple objects and the property is being replicated on each object already."));
			return false;
		}

		// We can give a better reason text if there's only a single object to assign to.
		return IsPropertyNotYetAssignedWithReason(*OuterObjects[0], PathInfo.Path, OutReason);
	}
	
	bool FDetailRowExtensionHandler::IsPropertyNotYetAssignedWithReason(
		const UObject& AssignedToObject,
		const TArray<FName>& PropertyPath,
		FText* OutReason
		) const
	{
		const TSharedPtr<FMultiUserReplicationManager> ReplicationManager = WeakReplicationManager.Pin();
		
		const TOptional<FGuid> OwningClient = ensure(ReplicationManager)
			? ReplicationManager->GetOnlineClientManager()->GetAuthorityCache().GetClientWithAuthorityOverProperty(&AssignedToObject, PropertyPath)
			: TOptional<FGuid>{};
		if (OwningClient && *OwningClient == ReplicationManager->GetOnlineClientManager()->GetLocalClient().GetEndpointId())
		{
			SET_REASON(LOCTEXT("AlreadyOwnedLocally", "You are already replicating this property."));
			return false;
		}
		
		if (OwningClient)
		{
			// Use lambda to defer evaluation below.
			const auto GetClientName = [this, &OwningClient]
			{
				const TSharedPtr<IConcertSyncClient> Client = WeakClient.Pin();
				const TSharedPtr<IConcertClientSession> Session = Client ? Client->GetConcertClient()->GetCurrentSession() : nullptr;
				checkf(Session, TEXT("We checked for EMultiUserReplicationConnectionState::Connected above but have no valid session"));
				return ClientUtils::GetClientDisplayName(*Session, *OwningClient);
			};
			
			SET_REASON(
				FText::Format(
					LOCTEXT("Reason.OwnedByOtherClient", "This property is already owned by client {0}. Re-assign manually in replication view."),
					FText::FromString(GetClientName())
				)
			);
			return false;
		}
		
		return true;
	}
#undef SET_REASON
}

#undef LOCTEXT_NAMESPACE