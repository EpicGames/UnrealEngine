// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/WidgetGetters.h"

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;
class IPropertyHandle;
class UObject;

struct FOnGenerateGlobalRowExtensionArgs;
struct FPropertyRowExtensionButton;

namespace UE::MultiUserClient::Replication
{
	class FMultiUserReplicationManager;

	/** Adds a button to Details panel rows for quickly assigning the property to the local client for replication. */
	class FDetailRowExtensionHandler : public FNoncopyable
	{
	public:
		
		FDetailRowExtensionHandler(
			const TSharedRef<IConcertSyncClient>& InClient,
			const TSharedRef<FMultiUserReplicationManager>& InReplicationManager,
			FGetConcertBrowserWidget InGetOrInvokeBrowserTabDelegate
			);
		~FDetailRowExtensionHandler();

	private:

		/** Used to look up client names. */
		const TWeakPtr<IConcertSyncClient> WeakClient;
		/** Used to get the local client, to which properties will be assigned by default. */
		const TWeakPtr<FMultiUserReplicationManager> WeakReplicationManager;
		/** Gets the content of the Multi User browser tab invoking it if it is not open. */
		const FGetConcertBrowserWidget GetOrInvokeBrowserTabDelegate;
		/** Handle to FMultiUserReplicationManager::OnReplicationConnectionStateChanged. */
		const FDelegateHandle OnReplicationConnectionChangedHandle;

		/** Adds the assign button to valid property rows.  */
		void RegisterExtensionHandler(
			const FOnGenerateGlobalRowExtensionArgs& Args,
			TArray<FPropertyRowExtensionButton>& OutExtensionButtons
			);

		/** @return Displays the reason why the property cannot be assigned. */
		FText GetToolTipText(TSharedPtr<IPropertyHandle> PropertyHandle) const;

		/** Assigns the property to the local client. */
		void OnAssignPropertyClicked(TSharedPtr<IPropertyHandle> PropertyHandle) const;
		/** @return Whether the property can be assigned. */
		bool CanExecuteAssignPropertyClick(TSharedPtr<IPropertyHandle> PropertyHandle) const { return CanAssignProperty(PropertyHandle); }
		/** @return Whether the client is in a MU session and the property can ever be replicated. */
		bool IsAssignPropertyButtonVisible(TSharedPtr<IPropertyHandle> PropertyHandle) const;

		/** @return Whether in a MU session and completed the replication handshake. */
		bool IsConnectedToReplication() const;

		/** Shows the replication UI if hidden and selects Objects. */
		void SelectObjectsInReplicationUI(TConstArrayView<UObject*> ObjectsToSelect) const;

		/** @return Whether PropertyHandle can be assigned to its outer objects. */
		bool CanAssignPropertyWithReason(TSharedPtr<IPropertyHandle> PropertyHandle, FText* OutReason = nullptr) const;
		bool CanAssignProperty(TSharedPtr<IPropertyHandle> PropertyHandle) const { return CanAssignPropertyWithReason(PropertyHandle); }
		
		/** @return Whether the property identified by PropertyPath is not yet assigned to AssigedToObject by any client. */
		bool IsPropertyNotYetAssignedWithReason(const UObject& AssignedToObject, const TArray<FName>& PropertyPath, FText* OutReason = nullptr) const;
		bool IsPropertyNotYetAssigned(const UObject& Object, const TArray<FName>& PropertyChainPath) const
		{
			return IsPropertyNotYetAssignedWithReason(Object, PropertyChainPath);
		}
	};
}

