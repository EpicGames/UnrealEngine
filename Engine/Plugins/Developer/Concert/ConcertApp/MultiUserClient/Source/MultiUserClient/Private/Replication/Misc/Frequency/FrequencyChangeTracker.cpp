// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrequencyChangeTracker.h"

#include "Replication/Stream/IClientStreamSynchronizer.h"
#include "Settings/MultiUserReplicationSettings.h"

namespace UE::MultiUserClient::Replication
{
	FFrequencyChangeTracker::FFrequencyChangeTracker(
		IClientStreamSynchronizer& InStreamSynchronizer
		)
		: StreamSynchronizer(InStreamSynchronizer)
	{
		// Refresh locally recorded changes when the server state changes, e.g. after a request has been served.
		StreamSynchronizer.OnServerStreamChanged().AddRaw(this, &FFrequencyChangeTracker::RefreshChanges);
	}

	FFrequencyChangeTracker::~FFrequencyChangeTracker()
	{
		StreamSynchronizer.OnServerStreamChanged().RemoveAll(this);
	}

	void FFrequencyChangeTracker::AddOverride(FSoftObjectPath Object, FConcertObjectReplicationSettings NewSettings)
	{
		// TODO UE-219834: The override should be added to UMultiUserReplicationStream so it can be transacted.
		RecordedChanges.OverridesToAdd.Emplace(MoveTemp(Object), MoveTemp(NewSettings));
	}
	
	FFrequencyChangelist FFrequencyChangeTracker::BuildForSubmission(const FStreamChangelist& ObjectChanges)
	{
		// TODO UE-219834: A diff between UMultiUserReplicationStream and the server state should be generated instead
		FFrequencyChangelist Result = RecordedChanges;

		// For now we only support adding overrides.
		const FGuid StreamId = StreamSynchronizer.GetStreamId();
		const FConcertObjectReplicationMap& ReplicationMap = StreamSynchronizer.GetServerState();
		for (auto IteratorForAdds = Result.OverridesToAdd.CreateIterator(); IteratorForAdds; ++IteratorForAdds)
		{
			const TPair<FSoftObjectPath, FConcertObjectReplicationSettings>& Pair = *IteratorForAdds;
			if (!ReplicationMap.HasProperties(Pair.Key) && !ObjectChanges.ObjectsToPut.Contains({ StreamId, Pair.Key }))
			{
				IteratorForAdds.RemoveCurrent();
			}
		}

		// Set frequency defaults if either this is the first time we're submitting to the server or the user has changes the project settings since.
		const FConcertObjectReplicationSettings& DefaultsToSet = UMultiUserReplicationSettings::Get()->FrequencyRules.DefaultObjectFrequencySettings;
		if (DefaultsToSet != StreamSynchronizer.GetFrequencySettings().Defaults)
		{
			Result.NewDefaults = DefaultsToSet;
		}
		
		return Result;
	}

	void FFrequencyChangeTracker::RefreshChanges()
	{
		const FConcertObjectReplicationMap& ReplicationMap = StreamSynchronizer.GetServerState();
		for (auto IteratorForAdds = RecordedChanges.OverridesToAdd.CreateIterator(); IteratorForAdds; ++IteratorForAdds)
		{
			const TPair<FSoftObjectPath, FConcertObjectReplicationSettings>& Pair = *IteratorForAdds;
			if (!ReplicationMap.HasProperties(Pair.Key))
			{
				IteratorForAdds.RemoveCurrent();
			}
		}

		// In the future we must also cleans RecordedChanges.OverridesToRemove if we add a function like RemoveOverride
	}
}
