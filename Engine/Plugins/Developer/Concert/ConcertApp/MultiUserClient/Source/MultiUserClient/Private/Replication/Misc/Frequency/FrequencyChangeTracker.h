// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Misc/Util/StreamRequestUtils.h"

#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealType.h"

struct FConcertObjectReplicationSettings;
struct FSoftObjectPath;

namespace UE::MultiUserClient::Replication
{
	class IClientStreamSynchronizer;
	class FStreamChangeTracker;
	
	/** Used to queue up local made changes to object frequency settings. */
	class FFrequencyChangeTracker : public FNoncopyable
    {
    public:

		FFrequencyChangeTracker(
			IClientStreamSynchronizer& InStreamSynchronizer UE_LIFETIMEBOUND
			);
		~FFrequencyChangeTracker();

		/** Adds a frequency override for the given object. */
		void AddOverride(FSoftObjectPath Object, FConcertObjectReplicationSettings NewSettings);
		
		// In the future we could add RemoveOverride if needed

		/** Compares the recorded changes to about to be submitted ObjectChanges and strips any invalid changes. */
		FFrequencyChangelist BuildForSubmission(const FStreamChangelist& ObjectChanges);

		/** Compares the local changes against the server state and optionally removes changes. */
		void RefreshChanges();

		DECLARE_MULTICAST_DELEGATE(FOnFrequencySettingsChanged);
		/** Called when frequency settings have changed been recorded that need to be submitted to the server. */
		FOnFrequencySettingsChanged& OnFrequencySettingsChanged() { return OnFrequencySettingsChangedDelegate; } 

	private:

		/** Tells us when the server state changes so RecordedChanges can be cleansed of applied changes. */
		IClientStreamSynchronizer& StreamSynchronizer;

		/** The changes recorded thus far. */
		FFrequencyChangelist RecordedChanges;

		/** Called when frequency settings have changed been recorded that need to be submitted to the server. */
		FOnFrequencySettingsChanged OnFrequencySettingsChangedDelegate;
    };
}

