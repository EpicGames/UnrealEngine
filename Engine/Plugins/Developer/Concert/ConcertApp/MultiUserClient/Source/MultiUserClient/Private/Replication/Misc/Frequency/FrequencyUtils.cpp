// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrequencyUtils.h"

namespace UE::MultiUserClient::Replication::FrequencyUtils
{
	TSharedPtr<IParallelSubmissionOperation> SetFrequencySettingForClients(
		const FSoftObjectPath& ContextObject,
		const FInlineClientArray& Clients,
		FOnlineClientManager& InClientManager,
		TFunctionRef<void(FConcertObjectReplicationSettings&)> ApplySettingChange
		)
	{
		TMap<FGuid, FConcertReplication_ChangeStream_Request> ParallelOperations;
		
		for (const FGuid& ClientId : Clients)
		{
			FOnlineClient* Client = InClientManager.FindClient(ClientId);
			if (Client && CanChangeFrequencySettings(ContextObject, *Client))
			{
				const FGuid StreamId = Client->GetStreamSynchronizer().GetStreamId();
				const FConcertStreamFrequencySettings& AllSettings = Client->GetStreamSynchronizer().GetFrequencySettings();
				FConcertObjectReplicationSettings PerObjectOverride = AllSettings.GetSettingsFor(ContextObject);

				const bool bNeedsToApplyDefault = !AllSettings.ObjectOverrides.Contains(ContextObject);
				// TODO UE-203350: Specify default replication settings from Multi User project settings
				PerObjectOverride = bNeedsToApplyDefault ? FConcertObjectReplicationSettings{ EConcertObjectReplicationMode::SpecifiedRate, 30 } : PerObjectOverride;
				ApplySettingChange(PerObjectOverride);
				if (!ensure(PerObjectOverride.ReplicationRate > 0))
				{
					continue;
				}
				
				FConcertReplication_ChangeStream_Frequency FrequencyChange;
				FrequencyChange.OverridesToAdd.Add(ContextObject, PerObjectOverride);
				FConcertReplication_ChangeStream_Request StreamChangeRequest;
				StreamChangeRequest.FrequencyChanges.Add(StreamId, FrequencyChange);
				
				ParallelOperations.Add(ClientId, StreamChangeRequest);
			}
		}

		return ParallelOperations.IsEmpty()
			? nullptr
			: ExecuteParallelStreamChanges(InClientManager, ParallelOperations);
	}
	
}