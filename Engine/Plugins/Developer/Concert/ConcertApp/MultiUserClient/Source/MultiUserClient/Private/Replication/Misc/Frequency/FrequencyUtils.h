// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Data/ReplicationFrequencySettings.h"

#include "Algo/AllOf.h"
#include "Async/Future.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#include <type_traits>

namespace UE::MultiUserClient::Replication
{
	class IParallelSubmissionOperation;
}

namespace UE::MultiUserClient::Replication::FrequencyUtils
{
	// Low effort optimization to minimize heap allocations
	using FInlineClientArray = TArray<FGuid, TInlineAllocator<8>>;
	
	/** Gets the shared value from FConcertObjectReplicationSettings, if all clients share the same value. Transform selects the member. */
	template<typename TSubSetting, typename TSelectSubSetting>
	TOptional<TSubSetting> FindSharedFrequencySetting(
		const FSoftObjectPath& ContextObject,
		const FInlineClientArray& Clients,
		const FOnlineClientManager& InClientManager,
		TSelectSubSetting&& Transform
		)
	requires std::is_invocable_r_v<TSubSetting, TSelectSubSetting, const FConcertObjectReplicationSettings&>;

	enum class EChangeResult : uint8 { Success, Failure };
	/**
	 * Goes through all clients and sets a FConcertObjectReplicationSettings for ContextObject. The setting is applied by invoking ApplySettingChange.
	 * @return Operations object that informs you when the change operations have completed.
	 */
	TSharedPtr<IParallelSubmissionOperation> SetFrequencySettingForClients(
		const FSoftObjectPath& ContextObject,
		const FInlineClientArray& Clients,
		FOnlineClientManager& InClientManager,
		TFunctionRef<void(FConcertObjectReplicationSettings&)> ApplySettingChange
		);

	/** Gets the replication rate of all clients, if they share the same value. */
	inline TOptional<uint8> FindSharedFrequencyRate(const FSoftObjectPath& ContextObject, const FInlineClientArray& Clients, const FOnlineClientManager& InClientManager)
	{
		bool bHasRightMode = false;
		const TOptional<uint8> SharedMode = FindSharedFrequencySetting<uint8>(ContextObject, Clients, InClientManager,
			[&bHasRightMode](const FConcertObjectReplicationSettings& Setting)
			{
				bHasRightMode |= Setting.ReplicationMode == EConcertObjectReplicationMode::SpecifiedRate;
				return Setting.ReplicationRate;
			});
		return bHasRightMode ? SharedMode : TOptional<uint8>{};
	}

	/** @return The replication mode that all clients are using, if they share the same value. */
	inline TOptional<EConcertObjectReplicationMode> FindSharedReplicationMode(const FSoftObjectPath& ContextObject, const FInlineClientArray& Clients, const FOnlineClientManager& InClientManager)
	{
		return FindSharedFrequencySetting<EConcertObjectReplicationMode>(ContextObject, Clients, InClientManager,
			[](const FConcertObjectReplicationSettings& Setting)
			{
				return Setting.ReplicationMode;
			});
	}
	
	/** @return Whether all clients have the given replication mode */
	inline bool AllClientsHaveMode(const EConcertObjectReplicationMode SearchedMode, const FSoftObjectPath& ContextObject, const FInlineClientArray& Clients, const FOnlineClientManager& InClientManager) 
	{
		return SearchedMode == FindSharedReplicationMode(ContextObject, Clients, InClientManager);
	}

	/** @return Whether ContextObject's frequency settings can be changed for Client. */
	inline bool CanChangeFrequencySettings(const FSoftObjectPath& ContextObject, const FOnlineClient& Client)
	{
		const bool bHasProperties = Client.GetStreamSynchronizer().GetServerState().HasProperties(ContextObject);
		const bool bNotProcessingOtherRequests = Client.GetSubmissionWorkflow().CanSubmit();
		return bHasProperties && bNotProcessingOtherRequests;
	}

	/** @return Whether ContextObject's frequency settings can be changed for all given Clients. */
	inline bool CanChangeFrequencySettings(const FSoftObjectPath& ContextObject, const FInlineClientArray& Clients, const FOnlineClientManager& InClientManager)
	{
		return Algo::AllOf(Clients, [ContextObject, &InClientManager](const FGuid& ClientId)
		{
			const FOnlineClient* Client = InClientManager.FindClient(ClientId);
			return Client && CanChangeFrequencySettings(ContextObject, *Client);
		});
	}
}

namespace UE::MultiUserClient::Replication::FrequencyUtils
{
	template<typename TSubSetting, typename TSelectSubSetting>
	TOptional<TSubSetting> FindSharedFrequencySetting(
		const FSoftObjectPath& ContextObject,
		const FInlineClientArray& Clients,
		const FOnlineClientManager& InClientManager,
		TSelectSubSetting&& Transform
		)
	requires std::is_invocable_r_v<TSubSetting, TSelectSubSetting, const FConcertObjectReplicationSettings&>
	{
		TOptional<TSubSetting> SharedSetting;
		for (const FGuid& ClientId : Clients)
		{
			const FOnlineClient* Client = InClientManager.FindClient(ClientId);
			if (!Client)
			{
				continue;
			}

			const FConcertObjectReplicationSettings& ObjectSettings = Client->GetStreamSynchronizer().GetFrequencySettings().GetSettingsFor(ContextObject);
			const TSubSetting SubSetting = Transform(ObjectSettings);
			if (SharedSetting && SharedSetting != SubSetting)
			{
				return {};
			}
			else
			{
				SharedSetting = SubSetting;
			}
		}
		return SharedSetting;
	}
}
