// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalAuthorityCache.h"

#include "Replication/AuthorityConflictSharedUtils.h"
#include "Replication/Authority/EAuthorityMutability.h"
#include "Replication/Client/Online/RemoteClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"

namespace UE::MultiUserClient::Replication
{
	FGlobalAuthorityCache::FGlobalAuthorityCache(FOnlineClientManager& InClientManager)
		: ClientManager(InClientManager)
	{}

	void FGlobalAuthorityCache::RegisterEvents()
	{
		RegisterForClientEvents(ClientManager.GetLocalClient());

		ClientManager.OnPostRemoteClientAdded().AddRaw(this, &FGlobalAuthorityCache::OnPostRemoteClientAdded);
		ClientManager.OnPreRemoteClientRemoved().AddRaw(this, &FGlobalAuthorityCache::OnPreRemoteClientRemoved);
	}

	void FGlobalAuthorityCache::ForEachClientWithObjectInStream(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback) const
	{
		const TSet<FGuid>* Clients = StreamObjectsToClients.Find(Object);
		if (!Clients)
		{
			return;
		}

		for (const FGuid& ClientId : *Clients)
		{
			if (Callback(ClientId) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	bool FGlobalAuthorityCache::IsObjectOrChildReferenced(const FSoftObjectPath& Object) const
	{
		const FString ObjectPathString = Object.ToString();
		for (const TPair<FSoftObjectPath, TSet<FGuid>>& Pair : StreamObjectsToClients)
		{
			if (Pair.Key.ToString().Contains(ObjectPathString))
			{
				return true;
			}
		}
		return false;
	}

	void FGlobalAuthorityCache::ForEachClientWithAuthorityOverObject(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback) const
	{
		const TSet<FGuid>* Clients = AuthorityObjectsToClients.Find(Object);
		if (!Clients)
		{
			return;
		}

		for (const FGuid& ClientId : *Clients)
		{
			if (Callback(ClientId) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	TArray<FGuid> FGlobalAuthorityCache::GetClientsWithAuthorityOverObject(const FSoftObjectPath& Object) const
	{
		TArray<FGuid> Result;
		ForEachClientWithAuthorityOverObject(Object, [&Result](const FGuid& ClientId)
		{
			Result.Add(ClientId);
			return EBreakBehavior::Continue;
		});
		return Result;
	}

	bool FGlobalAuthorityCache::HasAuthorityOverObject(const FSoftObjectPath& Object, const FGuid& ClientId) const
	{
		bool bHasAuthority = false;
		ForEachClientWithAuthorityOverObject(Object, [&ClientId, &bHasAuthority](const FGuid& OtherClientId)
		{
			bHasAuthority = ClientId == OtherClientId;
			return bHasAuthority ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		return bHasAuthority;
	}

	EAuthorityMutability FGlobalAuthorityCache::CanClientTakeAuthorityAfterSubmission(const FSoftObjectPath& Object, const FGuid& ClientId, FProcessPropertyConflict ProcessConflict) const
	{
		const FOnlineClient* Client = ClientManager.FindClient(ClientId);
		if (!ensure(Client))
		{
			return EAuthorityMutability::NotApplicable;
		}

		// Important: Get server state with local changes applied to it! CanClientTakeAuthority answers: "Can the client take authority after submitting?" 
		const FConcertPropertySelection* PropertySelection = Client->GetStreamDiffer().GetPropertiesAfterSubmit(Object);
		if (!PropertySelection)
		{
			// Nothing to take authority over
			return EAuthorityMutability::NotApplicable;
		}
		
		using namespace ConcertSyncCore::Replication::AuthorityConflictUtils;
		const EAuthorityConflict Conflict = EnumerateAuthorityConflicts(
			ClientId,
			Object,
			PropertySelection->ReplicatedProperties,
			*this,
			[&ProcessConflict](const FGuid& ClientId, const FGuid&, const FConcertPropertyChain& ConflictingProperty)
			{
				return ProcessConflict(ClientId, ConflictingProperty);
			});
		return Conflict == EAuthorityConflict::Allowed ? EAuthorityMutability::Allowed : EAuthorityMutability::Conflict;
	}

	bool FGlobalAuthorityCache::CanClientAddProperty(const FSoftObjectPath& Object, const FGuid& ClientId, const FConcertPropertyChain& Chain) const
	{
		const FOnlineClient* Client = ClientManager.FindClient(ClientId);
		if (!ensure(Client))
		{
			return false;
		}
		
		using namespace ConcertSyncCore::Replication::AuthorityConflictUtils;
		const EAuthorityConflict Conflict = EnumerateAuthorityConflicts(
			ClientId,
			Object,
			{ Chain },
			*this
			);
		return Conflict == EAuthorityConflict::Allowed;
	}

	void FGlobalAuthorityCache::ForEachClientReferencingProperty(const FSoftObjectPath& Object, const FConcertPropertyChain& Property, TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback) const
	{
		const TSet<FGuid>* ClientsWithStreams = StreamObjectsToClients.Find(Object);
		if (!ClientsWithStreams)
		{
			return;
		}

		for (const FGuid& ClientId : *ClientsWithStreams)
		{
			const FOnlineClient* Client = ClientManager.FindClient(ClientId);
			if (!ensure(Client))
			{
				continue;
			}

			const FConcertReplicatedObjectInfo* ObjectInfo = Client->GetStreamSynchronizer().GetServerState().ReplicatedObjects.Find(Object);
			const bool bContainsProperty = ensureMsgf(ObjectInfo, TEXT("RegisteredObjectsToClients lied. Investigate."))
				&& ObjectInfo->PropertySelection.ReplicatedProperties.Contains(Property);
			if (bContainsProperty && Callback(ClientId) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	bool FGlobalAuthorityCache::IsPropertyReferencedByAnyClientStream(const FSoftObjectPath& Object, const FConcertPropertyChain& Property) const
	{
		bool bIsReferenced = false;
		ForEachClientReferencingProperty(Object, Property, [&](const FGuid& ClientId)
		{
			bIsReferenced = true;
			return EBreakBehavior::Continue;
		});
		return bIsReferenced;
	}

	TOptional<FGuid> FGlobalAuthorityCache::GetClientWithAuthorityOverProperty(const FSoftObjectPath& Object, const TArray<FName>& PropertyChain) const
	{
		TOptional<FGuid> Result;
		const uint32 Hash = ConcertSyncCore::ComputeHashForPropertyChainContent(PropertyChain);
		
		ForEachClientWithAuthorityOverObject(Object, [this, &Object, &PropertyChain, &Result, Hash](const FGuid& ClientId)
		{
			const FOnlineClient* Client = ClientManager.FindClient(ClientId);
			if (!ensureMsgf(Client, TEXT("OnPreRemoteClientRemoved should have updated OwnedObjectsToClients")))
			{
				return EBreakBehavior::Continue;
			}
			
			const FConcertReplicatedObjectInfo* ObjectInfo = Client->GetStreamSynchronizer().GetServerState().ReplicatedObjects.Find(Object);
			const bool bHasObjectRegistered = ensureMsgf(ObjectInfo, TEXT("OnStreamChanged should have updated OwnedObjectsToClients"))
				&& ObjectInfo->PropertySelection.ReplicatedProperties.ContainsByHash(Hash, PropertyChain);
			const bool bHasAuthority = Client->GetAuthoritySynchronizer().HasAuthorityOver(Object);
			if (bHasObjectRegistered && bHasAuthority)
			{
				Result = ClientId;
				return EBreakBehavior::Break;
			}
			return EBreakBehavior::Continue;
		});
		return Result;
	}

	void FGlobalAuthorityCache::CleanseConflictsFromAuthorityRequest(FConcertReplication_ChangeAuthority_Request& Request, const FGuid& SendingClient) const
	{
		ConcertSyncCore::Replication::AuthorityConflictUtils::CleanseConflictsFromAuthorityRequest(Request, SendingClient, *this);
	}

	void FGlobalAuthorityCache::CleanseConflictsFromStreamRequest(FConcertReplication_ChangeStream_Request& Request, const FGuid& SendingClient) const
	{
		ConcertSyncCore::Replication::AuthorityConflictUtils::CleanseConflictsFromStreamRequest(Request, SendingClient, *this);
	}

	void FGlobalAuthorityCache::RegisterForClientEvents(const FOnlineClient& Client)
	{
		const FGuid& ClientEndpointId = Client.GetEndpointId();
		Client.GetAuthoritySynchronizer().OnServerAuthorityChanged().AddRaw(this, &FGlobalAuthorityCache::OnPostAuthorityChanged, ClientEndpointId);
		Client.GetStreamSynchronizer().OnServerStreamChanged().AddRaw(this, &FGlobalAuthorityCache::OnStreamChanged, ClientEndpointId);
	}

	void FGlobalAuthorityCache::UnregisterFromClientEvents(const FOnlineClient& Client) const
	{
		Client.GetAuthoritySynchronizer().OnServerAuthorityChanged().RemoveAll(this);
		Client.GetStreamSynchronizer().OnServerStreamChanged().RemoveAll(this);
	}

	void FGlobalAuthorityCache::AddClient(const FGuid& ClientId)
	{
		const FOnlineClient* Client = ClientManager.FindClient(ClientId);
		if (!ensure(Client))
		{
			return;
		}
		
		const FConcertObjectReplicationMap& ClientObjectMap = Client->GetStreamSynchronizer().GetServerState();
		const IClientAuthoritySynchronizer& ClientAuthority = Client->GetAuthoritySynchronizer();
		for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& StreamContents : ClientObjectMap.ReplicatedObjects)
		{
			const FSoftObjectPath& Object = StreamContents.Key;
			StreamObjectsToClients.FindOrAdd(Object).Add(ClientId);
			StreamObjectHierarchy.AddObject(Object);
			
			if (ClientAuthority.HasAuthorityOver(Object))
			{
				AuthorityObjectsToClients.FindOrAdd(Object).Add(ClientId);
			}
		}
	}

	void FGlobalAuthorityCache::RemoveClient(const FGuid& ClientId)
	{
		for (auto It = AuthorityObjectsToClients.CreateIterator(); It; ++It)
		{
			It->Value.Remove(ClientId);
			if (It->Value.IsEmpty())
			{
				It.RemoveCurrent();
			}
		}

		for (auto It = StreamObjectsToClients.CreateIterator(); It; ++It)
		{
			It->Value.Remove(ClientId);
			if (It->Value.IsEmpty())
			{
				StreamObjectHierarchy.RemoveObject(It->Key);
				It.RemoveCurrent();
			}
		}
	}
	
	void FGlobalAuthorityCache::ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)> Callback) const
	{
		const FOnlineClient* Client = ClientManager.FindClient(ClientEndpointId);
		if (!ensure(Client))
		{
			return;
		}

		const IClientStreamSynchronizer& StreamSynchronizer = Client->GetStreamSynchronizer();
		Callback(StreamSynchronizer.GetStreamId(), StreamSynchronizer.GetServerState());
	}

	void FGlobalAuthorityCache::ForEachClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const
	{
		auto ProcessClient = [&Callback](const FOnlineClient& Client)
		{
			return Client.GetAuthoritySynchronizer().HasAnyAuthority()
				? Callback(Client.GetEndpointId())
				:EBreakBehavior::Continue;
		};

		if (ProcessClient(ClientManager.GetLocalClient()) == EBreakBehavior::Break)
		{
			return;
		}
		for (const TNonNullPtr<FRemoteClient>& RemoteClient : ClientManager.GetRemoteClients())
		{
			if (ProcessClient(*RemoteClient) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	bool FGlobalAuthorityCache::HasAuthority(const FGuid& ClientId, const FGuid& StreamId, const FSoftObjectPath& ObjectPath) const
	{
		return GetClientsWithAuthorityOverObject(ObjectPath).Contains(ClientId);
	}
}
