// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Client/Online/RemoteClient.h"

#include "Replication/AuthorityConflictSharedUtils.h"
#include "Replication/Misc/IReplicationGroundTruth.h"

#include "Delegates/Delegate.h"
#include "Misc/ObjectPathHierarchy.h"
#include "Templates/Function.h"

struct FConcertPropertyChain;

namespace UE::MultiUserClient::Replication
{
	class FRemoteClient;
	class FOnlineClient;
	class FOnlineClientManager;
}

namespace UE::MultiUserClient::Replication
{
	/**
	 * Allows efficient look-up of which objects and properties are owned by which clients.
	 * This class efficiently answers the question: "Which clients own this object?"
	 */
	class FGlobalAuthorityCache
		: public FNoncopyable
		, ConcertSyncCore::Replication::IReplicationGroundTruth
	{
	public:

		using FProcessPropertyConflict = TFunctionRef<EBreakBehavior(const FGuid& ConflictingClientId, const FConcertPropertyChain& Property)>;
		
		FGlobalAuthorityCache(FOnlineClientManager& InClientManager);
		/** Called when the local client has been created and it is safe to register client events with FReplicationClientManager. */
		void RegisterEvents();
		
		/** Iterates every client that Object in its stream. */
		void ForEachClientWithObjectInStream(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback) const;

		/** @return Whether the object is referenced by at least one client stream. */
		bool IsObjectOrChildReferenced(const FSoftObjectPath& Object) const;

		/** Iterates every client that has authority over Object. */
		void ForEachClientWithAuthorityOverObject(const FSoftObjectPath& Object, TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback) const;
		/** Util that uses ForEachClientWithAuthorityOverObject to make an array. */
		TArray<FGuid> GetClientsWithAuthorityOverObject(const FSoftObjectPath& Object) const;
		/** @return Returns whether the client has (partial) authority over the object. */
		bool HasAuthorityOverObject(const FSoftObjectPath& Object, const FGuid& ClientId) const;
		
		/** @return Whether the given client can take authority over the object without causing any conflicts. This also considers the changes made to the stream after submission. */
		EAuthorityMutability CanClientTakeAuthorityAfterSubmission(const FSoftObjectPath& Object, const FGuid& ClientId, FProcessPropertyConflict ProcessConflict = [](auto&, auto&){ return EBreakBehavior::Break; }) const;
		/** @return Whether the given client add the given property to the object without causing any conflicts. */
		bool CanClientAddProperty(const FSoftObjectPath& Object, const FGuid& ClientId, const FConcertPropertyChain& Chain) const;

		/** Iterates through every client whose stream is referencing Object's Property. */
		void ForEachClientReferencingProperty(const FSoftObjectPath& Object, const FConcertPropertyChain& Property, TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback) const;
		/** @return Whether any client is referencing Object in a stream. */
		bool IsPropertyReferencedByAnyClientStream(const FSoftObjectPath& Object, const FConcertPropertyChain& Property) const;

		/** Gets the client that has authority over the given property, if there is any. */
		TOptional<FGuid> GetClientWithAuthorityOverProperty(const FSoftObjectPath& Object, const FConcertPropertyChain& Property) const { return GetClientWithAuthorityOverProperty(Object, Property.GetPathToProperty()); }
		/** Gets the client that has authority over the given property, if there is any. */
		TOptional<FGuid> GetClientWithAuthorityOverProperty(const FSoftObjectPath& Object, const TArray<FName>& PropertyChain) const;
		
		/**
		 * Removes entries from Request that would generate conflicts.
		 * 
		 * @param Request The request to clense
		 * @param SendingClient The client that will send the request
		 */
		void CleanseConflictsFromAuthorityRequest(FConcertReplication_ChangeAuthority_Request& Request, const FGuid& SendingClient) const;
		/**
		 * Removes entries from Request that would generate conflicts.
		 * 
		 * @param Request The request to clense
		 * @param SendingClient The client that will send the request
		 */
		void CleanseConflictsFromStreamRequest(FConcertReplication_ChangeStream_Request& Request, const FGuid& SendingClient) const;

		/** Allows efficient retrieving of registered child objects. */
		const ConcertSyncCore::FObjectPathHierarchy& GetStreamObjectHierarchy() const { return StreamObjectHierarchy; }
		
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnCacheChanged, const FGuid& ClientId);
		/** Called when the cache changes for a specific client. */
		FOnCacheChanged& OnCacheChanged() { return OnCacheChangedDelegate; }
		
	private:

		/** Used to obtain the clients and their states */
		FOnlineClientManager& ClientManager;
		
		/** Maps objects that are owned to the clients that have them in the stream */
		TMap<FSoftObjectPath, TSet<FGuid>> StreamObjectsToClients;
		/** Maps objects that are owned to the clients that own them (have authority) */
		TMap<FSoftObjectPath, TSet<FGuid>> AuthorityObjectsToClients;
		/** The hierarchy of objects that have been registered by all clients. Allows efficient retrieving of child objects. */
		ConcertSyncCore::FObjectPathHierarchy StreamObjectHierarchy;
		
		/** Called when the cache changes for a specific client. */
		FOnCacheChanged OnCacheChangedDelegate;

		/** Registers for authority and stream changes */
		void RegisterForClientEvents(const FOnlineClient& Client);
		void UnregisterFromClientEvents(const FOnlineClient& Client) const;

		/** Adds the client to OwnedObjectsToClients */
		void AddClient(const FGuid& ClientId);
		void RemoveClient(const FGuid& ClientId);

		// Respond to remote client registration
		void OnPostRemoteClientAdded(FRemoteClient& RemoteClient)
		{
			RebuildClient(RemoteClient.GetEndpointId());
			RegisterForClientEvents(RemoteClient);
		}
		void OnPreRemoteClientRemoved(FRemoteClient& RemoteClient)
		{
			RemoveClient(RemoteClient.GetEndpointId());
			UnregisterFromClientEvents(RemoteClient);
		}

		// Rebuild client when their authority changes
		void OnPostAuthorityChanged(const FGuid ClientId) { RebuildClient(ClientId); }
		void OnStreamChanged(const FGuid ClientId){ RebuildClient(ClientId); }
		void RebuildClient(const FGuid ClientId)
		{
			RemoveClient(ClientId);
			AddClient(ClientId);
			OnCacheChangedDelegate.Broadcast(ClientId);
		}

		//~ Begin IReplicationGroundTruth Interface
		virtual void ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)> Callback) const override;
		virtual void ForEachClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const override;
		virtual bool HasAuthority(const FGuid& ClientId, const FGuid& StreamId, const FSoftObjectPath& ObjectPath) const override;
		//~ End IReplicationGroundTruth Interface
	};
}

