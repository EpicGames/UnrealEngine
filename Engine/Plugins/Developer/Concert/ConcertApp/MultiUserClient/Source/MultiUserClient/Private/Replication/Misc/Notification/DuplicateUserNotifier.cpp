// Copyright Epic Games, Inc. All Rights Reserved.

#include "DuplicateUserNotifier.h"

#include "ConcertLogGlobal.h"
#include "ConcertMessageData.h"
#include "IConcertClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Messages/RestoreContent.h"
#include "Replication/Misc/GlobalAuthorityCache.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

#include "HAL/PlatformMisc.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FDuplicateUserNotifier"

namespace UE::MultiUserClient::Replication::PrivateDuplicateUserNotifier
{
	static void ShowNotification(const FConcertClientInfo& ClientInfo)
	{
		check(IsInGameThread());
		if (!FSlateApplication::IsInitialized())
		{
			return;
		}
			
		FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
		FNotificationInfo NotificationInfo(LOCTEXT("DuplicateClient.Main", "Duplicate Client Name"));
		NotificationInfo.SubText = FText::Format(
			LOCTEXT("DuplicateClient.SubTextFmt", "There are 2 clients with display name {0} and device name {1} in the session.\n"
				"Some replication features will not work as intended.\n\n"
				"Most likely, you have launched 2 editors on the same machine.\n"
				"Try setting a display name using different -CONCERTDISPLAYNAME flag values for each instance."
			),
			FText::FromString(ClientInfo.DisplayName),
			FText::FromString(ClientInfo.DeviceName)
			);
		NotificationInfo.bFireAndForget = true;
		NotificationInfo.ExpireDuration = 8.f;
		NotificationManager.AddNotification(NotificationInfo)
			->SetCompletionState(SNotificationItem::CS_Fail);
	}

	
	static bool HasNameConflictWithLogicallyDuplicateClient(
		const FOnlineClientManager& ClientManager,
		const IConcertClientSession& Session
		)
	{
		const FConcertClientInfo& ClientInfo = Session.GetLocalClientInfo();
		const FGuid& LocalId = Session.GetSessionClientEndpointId();

		bool bLocalClientHasContent = !ClientManager.GetLocalClient().GetStreamSynchronizer().GetServerState().IsEmpty();
		bool bHasDuplicateClientName = false;
		ClientManager.ForEachClient(
			[&Session, &ClientInfo, &LocalId, &bHasDuplicateClientName, bLocalClientHasContent]
			(const FOnlineClient& Client)
			{
				FConcertSessionClientInfo OtherInfo;
				const bool bSuccess = Session.FindSessionClient(Client.GetEndpointId(), OtherInfo);
				
				bHasDuplicateClientName |= bSuccess
					&& Client.GetEndpointId() != LocalId
					&& ConcertSyncCore::Replication::AreLogicallySameClients(ClientInfo, OtherInfo.ClientInfo)
					// Only warn if either I or the other client is using replication.
					&& (bLocalClientHasContent || !Client.GetStreamSynchronizer().GetServerState().IsEmpty());
				return bHasDuplicateClientName ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});

		return bHasDuplicateClientName;
	}
}

namespace UE::MultiUserClient::Replication
{
	FDuplicateUserNotifier::FDuplicateUserNotifier(
		IConcertClient& InClient, 
		FOnlineClientManager& InOnlineClientManager 
		)
		: Client(InClient)
		, OnlineClientManager(InOnlineClientManager)
	{
		OnlineClientManager.GetAuthorityCache().OnCacheChanged().AddRaw(this, &FDuplicateUserNotifier::OnClientContentChanged);
	}

	FDuplicateUserNotifier::~FDuplicateUserNotifier()
	{
		OnlineClientManager.GetAuthorityCache().OnCacheChanged().RemoveAll(this);
	}

	void FDuplicateUserNotifier::OnClientContentChanged(const FGuid&)
	{
		const TSharedPtr<IConcertClientSession> Session = Client.GetCurrentSession();
		if (ensureMsgf(Session, TEXT("We are already supposed to have been destroyed."))
			&& PrivateDuplicateUserNotifier::HasNameConflictWithLogicallyDuplicateClient(OnlineClientManager, *Session))
		{
			ShowWarning();
		}
	}

	void FDuplicateUserNotifier::ShowWarning()
	{
		if (!bHasWarnedUser)
		{
			bHasWarnedUser = true;
			OnlineClientManager.GetAuthorityCache().OnCacheChanged().RemoveAll(this);
		
			const FConcertClientInfo& ClientInfo = Client.GetClientInfo();
			UE_LOGF(LogConcert, Error,
				"There are 2 clients with display name %ls and device name %ls in the session. Some replication features won't work as intended.",
				*ClientInfo.DisplayName,
				*ClientInfo.DeviceName
				);
			PrivateDuplicateUserNotifier::ShowNotification(ClientInfo);
		}
	}
}

#undef LOCTEXT_NAMESPACE