// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Replication/Client/Online/RemoteClient.h"
#include "Templates/UnrealTemplate.h"

class IConcertClient;
struct FConcertReplication_RestoreContent_Response;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
	
	/**
	 * Informs the user if they are using the same name as another client.
	 * The message is only shown if the local or other, duplicate client use replication, i.e. register properties.
	 * 
	 * Certain features, like restoring the content that the client had when they last left, will not working correctly with non-unique display names.
	 */
	class FDuplicateUserNotifier : public FNoncopyable
	{
	public:
		
		FDuplicateUserNotifier(
			IConcertClient& InClient UE_LIFETIMEBOUND,
			FOnlineClientManager& InOnlineClientManager UE_LIFETIMEBOUND
			);
		~FDuplicateUserNotifier();

	private:

		/** Used to get client display information. */
		IConcertClient& Client;
		/** Used to detect when a client's state changes. */
		FOnlineClientManager& OnlineClientManager; 

		/** Whether the user was already warned in this session (will re-warn if they leave and join another session). */
		bool bHasWarnedUser = false;

		/** Invokes when any client's content changes. */
		void OnClientContentChanged(const FGuid& ClientId);

		/** Shows the warning and prevents it from being shown again in the active session. */
		void ShowWarning();
	};
}

