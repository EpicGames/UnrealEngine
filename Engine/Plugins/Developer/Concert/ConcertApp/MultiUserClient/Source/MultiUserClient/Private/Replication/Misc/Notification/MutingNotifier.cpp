// Copyright Epic Games, Inc. All Rights Reserved.

#include "MutingNotifier.h"

#include "Replication/Muting/MuteStateManager.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FMutingNotifier"

namespace UE::MultiUserClient::Replication
{
	FMutingNotifier::FMutingNotifier(FMuteStateManager& InMuteManager)
		: MuteManager(InMuteManager)
	{
		MuteManager.OnMuteRequestFailure().AddRaw(this, &FMutingNotifier::OnMuteRequestFailed);
	}
	
	FMutingNotifier::~FMutingNotifier()
	{
		MuteManager.OnMuteRequestFailure().RemoveAll(this);
	}

	void FMutingNotifier::OnMuteRequestFailed(const FConcertReplication_ChangeMuteState_Request& Request, const FConcertReplication_ChangeMuteState_Response& Response)
	{
		if (FSlateApplication::IsInitialized())
		{
			FNotificationInfo Info(LOCTEXT("MuteRequestFailed.Title", "Pause / resume rejected by server."));
			Info.SubText = LOCTEXT("MuteRequestFailed.Subtext", "Server-client state was likely de-synched.\nTry again.");
			Info.bFireAndForget = true;
			Info.ExpireDuration = 4.f;
			Info.bUseSuccessFailIcons = true;
			FSlateNotificationManager::Get().AddNotification(Info)
				->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

#undef LOCTEXT_NAMESPACE