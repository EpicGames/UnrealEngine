// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Replication/Messages/Muting.h"
#include "Templates/UnrealTemplate.h"

struct FConcertReplication_ChangeMuteState_Request;
struct FConcertReplication_ChangeMuteState_Response;

namespace UE::MultiUserClient::Replication
{
	class FMuteStateManager;

	/** Informs the user of things that went wrong with muting requests. */
	class FMutingNotifier : public FNoncopyable
	{
	public:
		FMutingNotifier(FMuteStateManager& InMuteManager UE_LIFETIMEBOUND);
		~FMutingNotifier();

	private:

		/** Informs us when a request goes wrong. */
		FMuteStateManager& MuteManager;
		
		void OnMuteRequestFailed(const FConcertReplication_ChangeMuteState_Request& Request, const FConcertReplication_ChangeMuteState_Response& Response);
	};
}

