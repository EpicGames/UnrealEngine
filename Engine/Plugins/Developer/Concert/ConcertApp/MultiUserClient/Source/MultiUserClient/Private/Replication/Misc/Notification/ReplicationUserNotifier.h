// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DuplicateUserNotifier.h"
#include "MutingNotifier.h"
#include "Replication/Submission/Notification/SubmissionNotifier.h"

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

namespace UE::MultiUserClient::Replication
{
	class FMuteStateManager;

	/** This informs the user of things that went wrong during replication. */
	class FReplicationUserNotifier : public FNoncopyable
	{
	public:

		FReplicationUserNotifier(
			IConcertClient& InClient UE_LIFETIMEBOUND,
			FOnlineClientManager& InReplicationClientManager UE_LIFETIMEBOUND,
			FMuteStateManager& InMuteManager UE_LIFETIMEBOUND
			);

	private:
		
		/** Notifies the user when stream or authority submission to the server fails. */
		const FSubmissionNotifier SubmissionNotifier;
		
		/** Informs the user of things that went wrong with muting requests. */
		const FMutingNotifier MutingNotifier;
		/** Informs the user if they are using the same name as another client. */
		const FDuplicateUserNotifier DuplicateClientNameNotifier;
	};
}

