// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreventReplicatedPropertyTransaction.h"

#include "IConcertSyncClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Misc/GlobalAuthorityCache.h"
#include "Replication/Muting/MuteStateManager.h"

#include "Algo/AllOf.h"
#include "HAL/IConsoleManager.h"
#include "UObject/SoftObjectPath.h"

namespace UE::MultiUserClient::Replication
{
	TAutoConsoleVariable<bool> CVarAllowTransactionsOnReplicatedProperties(
		TEXT("MultiUser.AllowTransactionsOnReplicatedProperties"),
		false,
		TEXT("Whether to stop disallowing transactions on replicated properties."),
		ECVF_Default
		);
	
	namespace Private
	{
		FORCEINLINE FName GetFilterName() { return TEXT("MultiUserReplicationFilter"); }
	}
	
	FPreventReplicatedPropertyTransaction::FPreventReplicatedPropertyTransaction(
		IConcertSyncClient& InSyncClient,
		FOnlineClientManager& InClientManager,
		FMuteStateManager& InMuteManager
		)
		: SyncClient(InSyncClient)
		, ClientManager(InClientManager)
		, MuteManager(InMuteManager)
	{
		SyncClient.GetTransactionBridge()->RegisterTransactionFilter(
			Private::GetFilterName(),
			FOnFilterTransactionDelegate::CreateRaw(this, &FPreventReplicatedPropertyTransaction::FilterTransactionAffectedByReplication)
			);
	}

	FPreventReplicatedPropertyTransaction::~FPreventReplicatedPropertyTransaction()
	{
		SyncClient.GetTransactionBridge()->UnregisterTransactionFilter(Private::GetFilterName());
	}

	ETransactionFilterResult FPreventReplicatedPropertyTransaction::FilterTransactionAffectedByReplication(const FConcertTransactionFilterArgs& FilterArgs)
	{
		if (CVarAllowTransactionsOnReplicatedProperties.GetValueOnAnyThread())
		{
			return ETransactionFilterResult::UseDefault;
		}
		
		// 1. If an object is muted, we can save ourselves the work of analysing client streams.
		// 2. We do not know whether remote clients have sync control (see below) but we can approximate that they don't have it if the object is muted.
		const bool bIsMuted = MuteManager.GetSynchronizer().IsMuted(FilterArgs.ObjectToFilter);

		const TArray<FName>& Properties = FilterArgs.TransactionEvent.GetChangedProperties();
		// FTransactionObjectEvent::GetChangedProperties() only contains root properties, like RelativeLocation; sub-properties, like RelativeLocation.X, are not listed.
		// Luckily, replication streams list every parent property (so if RelativeLocation.X is being replicated then RelativeLocation is also in the stream).
		const bool bOnlyHasReplicatedProperties = !bIsMuted && !Properties.IsEmpty() && Algo::AllOf(Properties, [this, &FilterArgs](const FName& RootProperty)
		{
			const FGlobalAuthorityCache& AuthorityCache = ClientManager.GetAuthorityCache();
			const TOptional<FGuid> AuthoringClient = AuthorityCache.GetClientWithAuthorityOverProperty(FilterArgs.ObjectToFilter, { RootProperty });
			const bool bHasAuthoringClient = AuthoringClient.IsSet();
			const bool bIsLocalClient = bHasAuthoringClient && *AuthoringClient == ClientManager.GetLocalClient().GetEndpointId();
			if (bIsLocalClient)
			{
				const IConcertClientReplicationManager* ReplicationManager = SyncClient.GetReplicationManager();
				check(ReplicationManager);
				
				// Just because a client has authority, it does not mean they're replicating.
				// Replication is only happening when a client also has sync control.
				// Since we only query sync control of the local client, we only know for our local client whether it has sync control...
				const FGuid ClientStreamId = ClientManager.GetLocalClient().GetStreamSynchronizer().GetStreamId();
				return ReplicationManager->HasSyncControl({ ClientStreamId, FilterArgs.ObjectToFilter});
			}
			// ... and for remote clients we'll just assume they're replicating when they have authority.
			return bHasAuthoringClient;
		});
		
		// If the transaction contains only properties that are being replicated, skip the object as the two systems may interfere.
		// If the transaction contains a property that is not replicated, the transaction should be allowed.
		return bOnlyHasReplicatedProperties
			? ETransactionFilterResult::ExcludeObject
			: ETransactionFilterResult::UseDefault;
	}
}
