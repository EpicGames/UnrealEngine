// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertClientTransactionBridge.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"


class IConcertSyncClient;
enum class ETransactionFilterResult : uint8;
struct FConcertTransactionFilterArgs;

namespace UE::MultiUserClient::Replication
{
	class FMuteStateManager;
	class FOnlineClientManager;
	
	/** Prevents generating transactions with properties that are being replicated by any other client. */
	class FPreventReplicatedPropertyTransaction : public FNoncopyable
	{
	public:
		
		FPreventReplicatedPropertyTransaction(
			IConcertSyncClient& InSyncClient UE_LIFETIMEBOUND,
			FOnlineClientManager& InClientManager UE_LIFETIMEBOUND,
			FMuteStateManager& InMuteManager UE_LIFETIMEBOUND
			);
		~FPreventReplicatedPropertyTransaction();

	private:

		/** Used to unsubscribe from the transaction bridge. */
		IConcertSyncClient& SyncClient;

		/** Used to obtain up-to-date registered properties from all clients. */
		FOnlineClientManager& ClientManager;
		/** Used to approximate whether remote clients may have sync control */
		FMuteStateManager& MuteManager;
		
		ETransactionFilterResult FilterTransactionAffectedByReplication(const FConcertTransactionFilterArgs& FilterArgs);
	};
}

