// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoPropertyOwnershipTaker.h"

#include "UserPropertySelector.h"
#include "Replication/Client/Online/LocalClient.h"
#include "Replication/Misc/GlobalAuthorityCache.h"

namespace UE::MultiUserClient::Replication
{
	FAutoPropertyOwnershipTaker::FAutoPropertyOwnershipTaker(
		FUserPropertySelector& InPropertySelector,
		FLocalClient& InLocalClient,
		FGlobalAuthorityCache& InReplicationCache
		)
		: PropertySelector(InPropertySelector)
		, LocalClient(InLocalClient)
		, ReplicationCache(InReplicationCache)
	{
		PropertySelector.OnPropertiesAddedByUser().AddRaw(this, &FAutoPropertyOwnershipTaker::OnPropertiesAddedByUser);
		PropertySelector.OnPropertiesRemovedByUser().AddRaw(this, &FAutoPropertyOwnershipTaker::OnPropertiesRemovedByUser);
	}

	FAutoPropertyOwnershipTaker::~FAutoPropertyOwnershipTaker()
	{
		PropertySelector.OnPropertiesAddedByUser().RemoveAll(this);
		PropertySelector.OnPropertiesRemovedByUser().RemoveAll(this);
	}

	void FAutoPropertyOwnershipTaker::OnPropertiesAddedByUser(UObject* Object, TArrayView<const FConcertPropertyChain> Properties) const
	{
		TArray<FConcertPropertyChain> AddedProperties;
		Algo::TransformIf(Properties, AddedProperties,
			[this, Object](const FConcertPropertyChain& Property){ return !ReplicationCache.IsPropertyReferencedByAnyClientStream(Object, Property); },
			[](const FConcertPropertyChain& Property){ return Property; }
			);
		
		if (!AddedProperties.IsEmpty())
		{
			const TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel>& EditModel = LocalClient.GetClientEditModel();
			EditModel->AddObjects({ Object });
			EditModel->AddProperties(Object, AddedProperties);
		}
	}

	void FAutoPropertyOwnershipTaker::OnPropertiesRemovedByUser(UObject* Object, TArrayView<const FConcertPropertyChain> Properties) const
	{
		const TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel>& EditModel = LocalClient.GetClientEditModel();
		EditModel->RemoveProperties(Object, Properties);
	}
}
