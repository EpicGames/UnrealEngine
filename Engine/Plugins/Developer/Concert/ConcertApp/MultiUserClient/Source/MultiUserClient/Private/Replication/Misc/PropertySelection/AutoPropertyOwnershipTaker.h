// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

class UObject;
struct FConcertPropertyChain;

namespace UE::MultiUserClient::Replication
{
	class FGlobalAuthorityCache;
	class FLocalClient;
	class FUserPropertySelector;

	/** Automatically takes and releases ownership over properties the user adds to the set of selected properties. */
	class FAutoPropertyOwnershipTaker : public FNoncopyable
	{
	public:
		
		FAutoPropertyOwnershipTaker(
			FUserPropertySelector& InPropertySelector UE_LIFETIMEBOUND,
			FLocalClient& InLocalClient UE_LIFETIMEBOUND,
			FGlobalAuthorityCache& InReplicationCache UE_LIFETIMEBOUND
			);
		~FAutoPropertyOwnershipTaker();

	private:

		/** Used to detect when a property is added or removed from the selected set of properties. */
		FUserPropertySelector& PropertySelector;
		/** The property ownership is changed for this client. */
		FLocalClient& LocalClient;
		/** Used to quickly check whether user selected properties are already owned. */
		FGlobalAuthorityCache& ReplicationCache;
		
		void OnPropertiesAddedByUser(UObject* Object, TArrayView<const FConcertPropertyChain> Properties) const;
		void OnPropertiesRemovedByUser(UObject* Object, TArrayView<const FConcertPropertyChain> ConcertPropertyChains) const;
	};
}

