// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserPropertySelectionSource.h"

#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Replication/Editor/Model/Property/IPropertySource.h"

namespace UE::MultiUserClient::Replication
{
	namespace Private
	{
		class FUserPropertySource
			: public ConcertSharedSlate::IPropertySource
			, public FNoncopyable
		{
		public:
			
			FUserPropertySource(
				FSoftObjectPath Object,
				const ConcertSharedSlate::IReplicationStreamModel& InUserSelection UE_LIFETIMEBOUND,
				const FOnlineClientManager& InClientManager UE_LIFETIMEBOUND
				)
				: Object(MoveTemp(Object))
				, UserSelection(InUserSelection)
				, ClientManager(InClientManager)
			{}

			//~ Begin IPropertySource Interface
			virtual void EnumerateProperties(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FPropertyInfo& Property)> Delegate) const override
			{
				// The UI should list the properties the user explicitly selected...
				EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
				UserSelection.ForEachProperty(Object, [&Delegate, &BreakBehavior](const FConcertPropertyChain& Property)
				{
					BreakBehavior = Delegate(ConcertSharedSlate::FPropertyInfo(Property));
					return BreakBehavior;
				});
				if (BreakBehavior == EBreakBehavior::Break)
				{
					return;
				}

				// ... and the properties that are in use
				
				// The idea behind this is to avoid making unnecessary FConcertPropertyChain copies (clutters heap memory)
				using FHash = uint32;
				using FPropertyPtrArray = TArray<const FConcertPropertyChain*, TInlineAllocator<4>>;
				TMap<FHash, FPropertyPtrArray, TInlineSetAllocator<512>> VisitedProperties;
				
				ClientManager.ForEachClient([this, &Delegate, &VisitedProperties](const FOnlineClient& Client)
				{
					const FConcertReplicatedObjectInfo* ObjectInfo = Client.GetStreamSynchronizer()
						.GetServerState()
						.ReplicatedObjects.Find(Object);
					if (!ObjectInfo)
					{
						return EBreakBehavior::Continue;
					}

					for (const FConcertPropertyChain& PropertyChain : ObjectInfo->PropertySelection.ReplicatedProperties)
					{
						const FHash Hash = GetTypeHash(PropertyChain);
						FPropertyPtrArray& PropertyArray = VisitedProperties.FindOrAdd(Hash);
						const bool bListedByOtherStream = PropertyArray.ContainsByPredicate([&PropertyChain](const FConcertPropertyChain* Visited){ return PropertyChain == *Visited; });
						
						// Properties should not be listed multiple times
						const bool bAlreadyListed = bListedByOtherStream
							|| UserSelection.ContainsProperties(Object, { PropertyChain });
						if (bAlreadyListed)
						{
							continue;
						}

						if (Delegate(ConcertSharedSlate::FPropertyInfo(PropertyChain)) == EBreakBehavior::Break)
						{
							return EBreakBehavior::Break;
						}
						PropertyArray.Add(&PropertyChain);
					}
					return EBreakBehavior::Continue;
				});
			}
			//~ End IPropertySource Interface

		private:

			/** The object for which the properties are being displayed. */
			const FSoftObjectPath Object;

			/** Used to get the properties the user has selected. */
			const ConcertSharedSlate::IReplicationStreamModel& UserSelection;
			/** Used to get client stream content. */
			const FOnlineClientManager& ClientManager;
		};
	}
	
	FUserPropertySelectionSource::FUserPropertySelectionSource(
		const ConcertSharedSlate::IReplicationStreamModel& InUserSelection,
		const FOnlineClientManager& InClientManager
		)
		: UserSelection(InUserSelection)
		, ClientManager(InClientManager)
	{}

	void FUserPropertySelectionSource::ProcessPropertySource(
		const ConcertSharedSlate::FPropertySourceContext& Context,
		TFunctionRef<void(const ConcertSharedSlate::IPropertySource& Model)> Processor
		) const
	{
		const Private::FUserPropertySource UserPropertySource(Context.Object.GetUniqueID(), UserSelection, ClientManager);
		Processor(UserPropertySource);
	}
}
