// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Property/IPropertySourceProcessor.h"

#include "HAL//Platform.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ConcertSharedSlate { class IReplicationStreamModel; }

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
	
	/**
	 * Injected into UI causing it to only display the properties that
	 * - have been selected by the user
	 * - are referenced by any client streams
	 */
	class FUserPropertySelectionSource
		: public ConcertSharedSlate::IPropertySourceProcessor
		, public FNoncopyable
	{
	public:

		FUserPropertySelectionSource(
			const ConcertSharedSlate::IReplicationStreamModel& InUserSelection UE_LIFETIMEBOUND,
			const FOnlineClientManager& InClientManager UE_LIFETIMEBOUND
			);

		//~ Begin IPropertySourceProcessor Interface
		virtual void ProcessPropertySource(
			const ConcertSharedSlate::FPropertySourceContext& Context,
			TFunctionRef<void(const ConcertSharedSlate::IPropertySource& Model)> Processor
			) const override;
		//~ End IPropertySourceProcessor Interface

	private:

		/** This is used to read the properties the user has selected, which is represented by a stream. */
		const ConcertSharedSlate::IReplicationStreamModel& UserSelection;
		/** Used to get client stream content and subscribe to changes. */
		const FOnlineClientManager& ClientManager;
	};
}
