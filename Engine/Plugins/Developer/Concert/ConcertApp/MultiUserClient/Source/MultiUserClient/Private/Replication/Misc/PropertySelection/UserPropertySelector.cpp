// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserPropertySelector.h"

#include "UserPropertySelectionSource.h"
#include "Replication/Client/Offline/OfflineClientManager.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Replication/ReplicationWidgetFactories.h"

#include "ScopedTransaction.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "FUserPropertySelector"

namespace UE::MultiUserClient::Replication
{
	FUserPropertySelector::FUserPropertySelector(
		FOnlineClientManager& InOnlineClientManager,
		FOfflineClientManager& InOfflineClientManger
		)
		: OnlineClientManager(InOnlineClientManager)
		, OfflineClientManager(InOfflineClientManger)
		, PropertySelection(NewObject<UMultiUserReplicationStream>(GetTransientPackage(), NAME_None, RF_Transient | RF_Transactional))
		, SelectionEditModel(ConcertSharedSlate::CreateBaseStreamModel(PropertySelection->MakeReplicationMapGetterAttribute()))
		, PropertyProcessor(MakeShared<FUserPropertySelectionSource>(*SelectionEditModel, InOnlineClientManager))
	{
		OnlineClientManager.OnPostRemoteClientAdded().AddRaw(this, &FUserPropertySelector::OnClientAdded);
		OnlineClientManager.ForEachClient([this](FOnlineClient& Client)
		{
			RegisterOnlineClient(Client);
			return EBreakBehavior::Continue;
		});

		OfflineClientManager.OnPostClientAdded().AddRaw(this, &FUserPropertySelector::RegisterOfflineClient);
		OfflineClientManager.ForEachClient([this](FOfflineClient& Client)
		{
			RegisterOfflineClient(Client);
			return EBreakBehavior::Continue;
		});
		
		FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FUserPropertySelector::OnObjectTransacted);
	}

	FUserPropertySelector::~FUserPropertySelector()
	{
		OnlineClientManager.OnPostRemoteClientAdded().RemoveAll(this);
		OnlineClientManager.ForEachClient([this](FOnlineClient& Client)
		{
			Client.GetStreamSynchronizer().OnServerStreamChanged().RemoveAll(this);
			return EBreakBehavior::Continue;
		});

		OfflineClientManager.OnPostClientAdded().RemoveAll(this);
		OfflineClientManager.ForEachClient([this](FOfflineClient& Client)
		{
			Client.OnStreamPredictionChanged().RemoveAll(this);
			return EBreakBehavior::Continue;
		});
		
		FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	}

	void FUserPropertySelector::AddUserSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddSelectedProperties", "Select replicated property"));
		PropertySelection->Modify();
		
		InternalAddSelectedProperties(Object, Properties);
		OnPropertiesAddedByUserDelegate.Broadcast(Object, Properties);
	}
	
	void FUserPropertySelector::RemoveUserSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties)
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveSelectedProperties", "Deselect replicated property"));
		PropertySelection->Modify();
		
		InternalRemoveSelectedProperties(Object, Properties);
		OnPropertiesRemovedByUserDelegate.Broadcast(Object, Properties);
	}

	bool FUserPropertySelector::IsPropertySelected(const FSoftObjectPath& Object, const FConcertPropertyChain& Property) const
	{
		return OnlineClientManager.GetAuthorityCache().IsPropertyReferencedByAnyClientStream(Object, Property)
			|| SelectionEditModel->HasProperty(Object, Property);
	}

	TSharedRef<ConcertSharedSlate::IPropertySourceProcessor> FUserPropertySelector::GetPropertySourceProcessor() const
	{
		return PropertyProcessor;
	}

	void FUserPropertySelector::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(PropertySelection);
	}

	void FUserPropertySelector::OnClientAdded(FRemoteClient& Client)
	{
		RegisterOnlineClient(Client);
	}

	void FUserPropertySelector::RegisterOnlineClient(FOnlineClient& Client)
	{
		IClientStreamSynchronizer& StreamSynchronizer = Client.GetStreamSynchronizer();
		TrackProperties(StreamSynchronizer.GetServerState());
		
		StreamSynchronizer.OnServerStreamChanged().AddRaw(this, &FUserPropertySelector::OnOnlineClientContentChanged, Client.GetEndpointId());
	}

	void FUserPropertySelector::RegisterOfflineClient(FOfflineClient& Client)
	{
		TrackProperties(Client.GetPredictedStream().ReplicationMap);
		
		const TNonNullPtr<const FOfflineClient>& NonNullClient = &Client;
		Client.OnStreamPredictionChanged().AddRaw(this, &FUserPropertySelector::OnOfflineClientContentChanged, NonNullClient);
	}

	void FUserPropertySelector::OnOnlineClientContentChanged(const FGuid ClientId)
	{
		const FOnlineClient* Client = OnlineClientManager.FindClient(ClientId);
		if (ensure(Client))
		{
			TrackProperties(Client->GetStreamSynchronizer().GetServerState());
		}
	}

	void FUserPropertySelector::OnOfflineClientContentChanged(const TNonNullPtr<const FOfflineClient> Client)
	{
		TrackProperties(Client->GetPredictedStream().ReplicationMap);
	}

	void FUserPropertySelector::TrackProperties(const FConcertObjectReplicationMap& ReplicationMap)
	{
		for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : ReplicationMap.ReplicatedObjects)
		{
			UObject* Object = Pair.Key.ResolveObject();
			// The object may come from a remote client that is in a different world than the local application
			if (!Object)
			{
				continue;
			}

			for (const FConcertPropertyChain& Property : Pair.Value.PropertySelection.ReplicatedProperties)
			{
				// Do not transact this change: the user did not actively add these properties, so it should not show up in the undo history.
				InternalAddSelectedProperties(Object, { Property });
			}
		}
	}

	void FUserPropertySelector::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent&) const
	{
		if (Object == PropertySelection)
		{
			// Refreshes UI
			OnPropertySelectionChangedDelegate.Broadcast();
		}
	}
	
	void FUserPropertySelector::InternalAddSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties)
	{
		SelectionEditModel->AddObjects({ Object });
		SelectionEditModel->AddProperties({ Object }, Properties);

		OnPropertySelectionChangedDelegate.Broadcast();
	}
	
	void FUserPropertySelector::InternalRemoveSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties)
	{
		SelectionEditModel->RemoveProperties({ Object }, Properties);
		if (!SelectionEditModel->HasAnyPropertyAssigned(Object))
		{
			SelectionEditModel->RemoveObjects({ Object });
		}

		const TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> ClientEditModel = OnlineClientManager.GetLocalClient().GetClientEditModel();
		ClientEditModel->RemoveProperties({ Object }, Properties);
		
		OnPropertySelectionChangedDelegate.Broadcast();
	}
}

#undef LOCTEXT_NAMESPACE