// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MultiUserReplicationStream.h"

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectPtr.h"



class FReferenceCollector;
class FTransactionObjectEvent;
class UObject;

namespace UE::ConcertSharedSlate
{
	class IEditableReplicationStreamModel;
	class IPropertySourceProcessor;
}

namespace UE::MultiUserClient::Replication
{
	class FOfflineClient;
	class FOfflineClientManager;
	class FOnlineClientManager;
	class FOnlineClient;
	class FRemoteClient;
	class FUserPropertySelectionSource;
	
	/**
	 * Manages the properties the user is iterating on in the replication session.
	 * The bottom-half property section in the replication UI uses this to keep track of which properties the user has selected for which properties.
	 *
	 * Whenever any client adds a property to its stream, we'll assume the user is iterating on that property.
	 * For this reason, we automatically will track the property as user-selected.
	 */
	class FUserPropertySelector
		: public FNoncopyable
		, public FGCObject
	{
	public:
		
		FUserPropertySelector(
			FOnlineClientManager& InOnlineClientManager UE_LIFETIMEBOUND,
			FOfflineClientManager& InOfflineClientManger UE_LIFETIMEBOUND
			);
		virtual ~FUserPropertySelector() override;

		/** Add Properties to the user's selection for Object. These properties were purposefully selected added by the user. */
		void AddUserSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties);
		/** Removes Properties from the user's selection for Object. These properties were purposefully selected added by the user. */
		void RemoveUserSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties);
		/** @return Whether Property is selected for Object. */
		bool IsPropertySelected(const FSoftObjectPath& Object, const FConcertPropertyChain& Property) const;

		TSharedRef<ConcertSharedSlate::IPropertySourceProcessor> GetPropertySourceProcessor() const;

		DECLARE_MULTICAST_DELEGATE(FOnPropertySelectionChanged)
		/** @return Event that broadcasts when the user property selection changes. */
		FOnPropertySelectionChanged& OnPropertySelectionChanged() { return OnPropertySelectionChangedDelegate; }

		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPropertiesChangedByUser, UObject* /*Object*/, TConstArrayView<FConcertPropertyChain> /*Properties*/);
		/** @return Event that broadcasts when the user adds a property manually (through UI). */
		FOnPropertiesChangedByUser& OnPropertiesAddedByUser() { return OnPropertiesAddedByUserDelegate; }
		/** @return Event that broadcasts when the user removes a property manually (through UI). */
		FOnPropertiesChangedByUser& OnPropertiesRemovedByUser() { return OnPropertiesRemovedByUserDelegate; }
		
		//~ Begin FGCObject Interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FUserPropertySelector"); }
		//~ Begin FGCObject Interface

	private:

		/** Used to remove deselected properties from local client's stream and auto-add properties from remote clients to the user selection. */
		FOnlineClientManager& OnlineClientManager;
		/**
		 * When an online client turns into an offline client, we need to subscribe to changes made to that client,
		 * e.g. preset could change what client gets when it rejoins.
		 */
		FOfflineClientManager& OfflineClientManager;

		/** This underlying object saves the properties that user has selected. It allows for transactions. */
		TObjectPtr<UMultiUserReplicationStream> PropertySelection;
		/** This logic modifies PropertySelection. */
		const TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> SelectionEditModel;

		/** Getter for UI to determine which properties to display. */
		const TSharedRef<FUserPropertySelectionSource> PropertyProcessor;

		/** Broadcasts when the user property selection changes. */
		FOnPropertySelectionChanged OnPropertySelectionChangedDelegate;
		/** Broadcasts when the user adds a property manually (through UI). */
		FOnPropertiesChangedByUser OnPropertiesAddedByUserDelegate;
		/** Broadcasts when the user removes a property manually (through UI). */
		FOnPropertiesChangedByUser OnPropertiesRemovedByUserDelegate;

		/** Called when a remote client joins. */
		void OnClientAdded(FRemoteClient& Client);
		/** Ensures that whenever the client's server state changes, its properties are tracked as user selected. */
		void RegisterOnlineClient(FOnlineClient& Client);
		void RegisterOfflineClient(FOfflineClient& Client);

		/** Tracks all properties of the client as user selected. */
		void OnOnlineClientContentChanged(const FGuid ClientId);
		/** Tracks all properties the offline client will get upon rejoining as user selected. */
		void OnOfflineClientContentChanged(const TNonNullPtr<const FOfflineClient> Client);
		/** Adds all properties in the replication map as user selected. */
		void TrackProperties(const FConcertObjectReplicationMap& ReplicationMap);

		/** If PropertySelection is transacted, broadcast OnPropertySelectionChangedDelegate. */
		void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent&) const;
		
		void InternalAddSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties);
		void InternalRemoveSelectedProperties(UObject* Object, TConstArrayView<FConcertPropertyChain> Properties);
	};
}

