// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuteStateQueryService.h"

#include "IConcertSyncClient.h"
#include "Replication/IConcertClientReplicationManager.h"

namespace UE::MultiUserClient::Replication
{
	FMuteStateQueryService::FMuteStateQueryService(TWeakPtr<FToken> InToken, const IConcertSyncClient& InOwningClient)
		: Token(MoveTemp(InToken))
		, OwningClient(InOwningClient)
	{}

	void FMuteStateQueryService::SendQueryEvent()
	{
		// Avoid network calls when nobody is subscribed.
		IConcertClientReplicationManager* ReplicationManager = OwningClient.GetReplicationManager();
		if (!OnMuteStateQueriedDelegate.IsBound() || !ensure(ReplicationManager))
		{
			return;
		}

		// For now, we'll query for ALL objects - in the future we could restrict it to the local application's GWorld
		ReplicationManager->QueryMuteState()
			.Next([this, WeakToken = Token](FConcertReplication_QueryMuteState_Response&& Response)
			{
				// If WeakToken is not valid, our this pointer has been destroyed (there's currently no API for cancelling started requests).
				if (WeakToken.IsValid())
				{
					OnMuteStateQueriedDelegate.Broadcast(Response);
				}
			});
	}
}
