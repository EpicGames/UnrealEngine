// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IToken.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;
struct FConcertReplication_QueryMuteState_Response;

namespace UE::MultiUserClient::Replication
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMuteStateQueried, const FConcertReplication_QueryMuteState_Response& NewMuteState);
	
	/** Sends regular FConcertReplication_QueryMuteState_Request to endpoints and publishes the results. */
	class FMuteStateQueryService : public FNoncopyable
	{
		friend class FRegularQueryService;
	public:
		
		/**
		 * @param InToken Used to check whether this instance was destroyed after a response is received.
		 * @param InOwningClient The client through which queries are made. The caller ensures it outlasts the constructed instance.
		 */
		FMuteStateQueryService(TWeakPtr<FToken> InToken, const IConcertSyncClient& InOwningClient UE_LIFETIMEBOUND);

		/** @return Sends a request to the server immediately */
		void RequestInstantRefresh() { SendQueryEvent(); }

		FOnMuteStateQueried& OnMuteStateQueried() { return OnMuteStateQueriedDelegate; }
		
	private:

		/** Used to check whether we were destroyed after a response is received. */
		const TWeakPtr<FToken> Token;
		
		/**
		 * Used to send queries.
		 * 
		 * This FRegularQueryService's owner is supposed to make sure this FRegularQueryService is destroyed
		 * when the client shuts down.
		 */
		const IConcertSyncClient& OwningClient;

		/** Broadcasts when the mute state has been queried. */
		FOnMuteStateQueried OnMuteStateQueriedDelegate;
		
		/** Queries the server for the client's current state. */
		void SendQueryEvent();
	};
}

