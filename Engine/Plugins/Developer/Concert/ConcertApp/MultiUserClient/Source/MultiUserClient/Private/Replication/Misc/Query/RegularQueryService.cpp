// Copyright Epic Games, Inc. All Rights Reserved.

#include "RegularQueryService.h"

namespace UE::MultiUserClient::Replication
{
	FRegularQueryService::FRegularQueryService(const IConcertSyncClient& InOwningClient, float InInterval)
		: OwningClient(InOwningClient)
		, TickerDelegateHandle(FTSTicker::GetCoreTicker().AddTicker(
			TEXT("Multi-User Replication Query"),
			InInterval,
			[this](float)
			{
				Tick();
				return true;
			}))
		, StreamAndAuthorityQueryService(Token, OwningClient)
		, MuteStateQueryService(Token, OwningClient)
	{}

	FRegularQueryService::~FRegularQueryService()
	{
		FTSTicker::RemoveTicker(TickerDelegateHandle);
	}

	void FRegularQueryService::Tick()
	{
		StreamAndAuthorityQueryService.SendQueryEvent();
		MuteStateQueryService.SendQueryEvent();
	}
}
