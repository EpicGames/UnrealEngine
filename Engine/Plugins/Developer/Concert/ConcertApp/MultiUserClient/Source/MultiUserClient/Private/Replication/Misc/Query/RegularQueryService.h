// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuteStateQueryService.h"
#include "StreamAndAuthorityQueryService.h"
#include "Replication/IToken.h"

#include "Containers/Ticker.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;

namespace UE::MultiUserClient::Replication
{
	/**
	 * This service manages multiple sub-services that query the server state, e.g. stream, authority, etc.
	 * Its responsibility is to tick the sub-services and manage access to them. 
	 */
	class FRegularQueryService : public FNoncopyable
	{
	public:
		
		/**
		 * @param InOwningClient The client through which queries are made. The caller ensures it outlasts the constructed instance.
		 * @param InInterval The interval at which queries are to be made.
		 */
		FRegularQueryService(const IConcertSyncClient& InOwningClient UE_LIFETIMEBOUND, float InInterval = 1.f);
		~FRegularQueryService();

		/** @return The service for querying stream and authority state. */
		FStreamAndAuthorityQueryService& GetStreamAndAuthorityQueryService() { return StreamAndAuthorityQueryService; }
		/** @return The service for querying global mute state. */
		FMuteStateQueryService& GetMuteStateQueryService() { return MuteStateQueryService; }

	private:
		
		/** Passed to sub-services to check whether they were destroyed after a response is received. */
		const TSharedRef<FToken> Token = FToken::Make();
		
		/**
		 * Used to send queries.
		 * 
		 * This FRegularQueryService's owner is supposed to make sure this FRegularQueryService is destroyed
		 * when the client shuts down.
		 */
		const IConcertSyncClient& OwningClient;

		/** Used to unregister the ticker. */
		const FTSTicker::FDelegateHandle TickerDelegateHandle;

		/** Queries stream and authority changes. */
		FStreamAndAuthorityQueryService StreamAndAuthorityQueryService;
		/** Queries global mute state at regular intervals. */
		FMuteStateQueryService MuteStateQueryService;

		/** Ticks the sub-services when it is time to send a new request. */
		void Tick();
	};
}

