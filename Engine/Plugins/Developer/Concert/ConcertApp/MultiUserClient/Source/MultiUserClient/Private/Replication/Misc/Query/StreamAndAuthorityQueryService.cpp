// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamAndAuthorityQueryService.h"

#include "IConcertSyncClient.h"
#include "Replication/IConcertClientReplicationManager.h"

#include "Templates/UnrealTemplate.h"

namespace UE::MultiUserClient::Replication
{
	FStreamAndAuthorityQueryService::FStreamAndAuthorityQueryService(TWeakPtr<FToken> InToken, const IConcertSyncClient& InOwningClient)
		: Token(MoveTemp(InToken))
		, OwningClient(InOwningClient)
	{}

	FDelegateHandle FStreamAndAuthorityQueryService::RegisterStreamQuery(const FGuid& EndpointId, FStreamQueryDelegate Delegate)
	{
		FStreamQueryInfo& Info = StreamQueryInfos.FindOrAdd(EndpointId);
		const FDelegateHandle DelegateHandle = Info.Delegate.AddLambda([Delegate = MoveTemp(Delegate)](const TArray<FConcertBaseStreamInfo>& Descriptions)
		{
			Delegate.Execute(Descriptions);
		});
		Info.Handles.Add(DelegateHandle);
		return DelegateHandle;
	}

	FDelegateHandle FStreamAndAuthorityQueryService::RegisterAuthorityQuery(const FGuid& EndpointId, FAuthorityQueryDelegate Delegate)
	{
		FAuthorityQueryInfo& Info = AuthorityQueryInfos.FindOrAdd(EndpointId);
		const FDelegateHandle DelegateHandle = Info.Delegate.AddLambda([Delegate = MoveTemp(Delegate)](const TArray<FConcertAuthorityClientInfo>& Infos)
		{
			Delegate.Execute(Infos);
		});
		Info.Handles.Add(DelegateHandle);
		return DelegateHandle;
	}

	void FStreamAndAuthorityQueryService::UnregisterStreamQuery(const FDelegateHandle& Handle)
	{
		for (auto StreamIt = StreamQueryInfos.CreateIterator(); StreamIt; ++StreamIt)
		{
			FStreamQueryInfo& Info = StreamIt->Value;
			if (Info.Handles.Contains(Handle))
			{
				Info.Handles.Remove(Handle);
				Info.Delegate.Remove(Handle);
				
				// If bIsHandlingQueryResponse, then HandleQueryResponse is executing delegates and a delegate is unregistering.
				// Removing now would cause a crash since Broadcast will read some bookkeeping memory after Broadcast.
				// Memory is cleaned up later by HandleQueryResponse.
				if (!bIsHandlingQueryResponse && Info.Handles.IsEmpty())
				{
					StreamIt.RemoveCurrent();
				}
				
				return;
			}
		}
	}

	void FStreamAndAuthorityQueryService::UnregisterAuthorityQuery(const FDelegateHandle& Handle)
	{
		for (auto AuthorityIt = AuthorityQueryInfos.CreateIterator(); AuthorityIt; ++AuthorityIt)
		{
			FAuthorityQueryInfo& Info = AuthorityIt->Value;
			if (Info.Handles.Contains(Handle))
			{
				Info.Handles.Remove(Handle);
				Info.Delegate.Remove(Handle);
				
				// If bIsHandlingQueryResponse, then HandleQueryResponse is executing delegates and a delegate is unregistering.
				// Removing now would cause a crash since Broadcast will read some bookkeeping memory after Broadcast.
				// Memory is cleaned up later by HandleQueryResponse.
				if (!bIsHandlingQueryResponse && Info.Handles.IsEmpty())
				{
					AuthorityIt.RemoveCurrent();
				}
				
				return;
			}
		}
	}

	void FStreamAndAuthorityQueryService::SendQueryEvent()
	{
		IConcertClientReplicationManager* ReplicationManager = OwningClient.GetReplicationManager();
		if (!ensure(ReplicationManager))
		{
			return;
		}
		
		FConcertReplication_QueryReplicationInfo_Request Request;
		BuildStreamRequest(Request);
		BuildAuthorityRequest(Request);

		if (!Request.ClientEndpointIds.IsEmpty())
		{
			ReplicationManager->QueryClientInfo({ Request })
				.Next([this, WeakToken = Token](FConcertReplication_QueryReplicationInfo_Response&& Response)
				{
					const bool bCanProcessRequest = WeakToken.Pin().IsValid();
					if (bCanProcessRequest)
					{
						HandleQueryResponse(Response);
					}
				});
		}
	}

	void FStreamAndAuthorityQueryService::BuildStreamRequest(FConcertReplication_QueryReplicationInfo_Request& Request) const
	{
		if (StreamQueryInfos.IsEmpty())
		{
			Request.QueryFlags |= EConcertQueryClientStreamFlags::SkipStreamInfo;
		}
		else
		{
			for (const TPair<FGuid, FStreamQueryInfo>& Query : StreamQueryInfos)
			{
				Request.ClientEndpointIds.Add(Query.Key);
			}
		}
	}

	void FStreamAndAuthorityQueryService::BuildAuthorityRequest(FConcertReplication_QueryReplicationInfo_Request& Request) const
	{
		if (AuthorityQueryInfos.IsEmpty())
		{
			Request.QueryFlags |= EConcertQueryClientStreamFlags::SkipAuthority;
		}
		else
		{
			for (const TPair<FGuid, FAuthorityQueryInfo>& Query : AuthorityQueryInfos)
			{
				Request.ClientEndpointIds.Add(Query.Key);
			}
		}
	}

	void FStreamAndAuthorityQueryService::HandleQueryResponse(
		const FConcertReplication_QueryReplicationInfo_Response& Response
		)
	{
		TGuardValue<bool> Guard(bIsHandlingQueryResponse, true);
		
		for (const TPair<FGuid, FConcertQueriedClientInfo>& StreamQueryPair : Response.ClientInfo)
		{
			const FGuid StreamId = StreamQueryPair.Key;
			
			// Note that the delegates may have been unsubscribed since request was sent, which is why we need to Find() them
			if (const FStreamQueryInfo* QueryInfo = StreamQueryInfos.Find(StreamId))
			{
				QueryInfo->Delegate.Broadcast(StreamQueryPair.Value.Streams);
			}

			// Note that the delegates may have been unsubscribed since request was sent, which is why we need to Find() them
			if (const FAuthorityQueryInfo* QueryInfo = AuthorityQueryInfos.Find(StreamId))
			{
				QueryInfo->Delegate.Broadcast(StreamQueryPair.Value.Authority);
			}
		}

		// Delegates may have removed themselves up above.
		CompactDelegates();
	}

	void FStreamAndAuthorityQueryService::CompactDelegates()
	{
		for (auto StreamIt = StreamQueryInfos.CreateIterator(); StreamIt; ++StreamIt)
		{
			if (StreamIt->Value.Handles.IsEmpty())
			{
				StreamIt.RemoveCurrent();
			}
		}

		for (auto Authority = AuthorityQueryInfos.CreateIterator(); Authority; ++Authority)
		{
			if (Authority->Value.Handles.IsEmpty())
			{
				Authority.RemoveCurrent();
			}
		}
	}
}
