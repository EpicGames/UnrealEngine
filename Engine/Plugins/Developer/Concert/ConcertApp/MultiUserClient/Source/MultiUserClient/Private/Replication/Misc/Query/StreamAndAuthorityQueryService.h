// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Messages/ClientQuery.h"
#include "Replication/IToken.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;

namespace UE::MultiUserClient::Replication
{
	DECLARE_DELEGATE_OneParam(FStreamQueryDelegate, const TArray<FConcertBaseStreamInfo>&);
	DECLARE_DELEGATE_OneParam(FAuthorityQueryDelegate, const TArray<FConcertAuthorityClientInfo>&);
	
	/** Sends regular FConcertReplication_QueryReplicationInfo_Request to endpoints and publishes the results. */
	class FStreamAndAuthorityQueryService : public FNoncopyable
	{
		friend class FRegularQueryService;
	public:
		
		/**
		 * @param InToken Used to check whether this instance was destroyed after a response is received.
		 * @param InOwningClient The client through which queries are made. The caller ensures it outlasts the constructed instance.
		 */
		FStreamAndAuthorityQueryService(TWeakPtr<FToken> InToken, const IConcertSyncClient& InOwningClient UE_LIFETIMEBOUND);

		/** Registers a delegate to invoke for querying an endpoint about its registered streams. */
		FDelegateHandle RegisterStreamQuery(const FGuid& EndpointId, FStreamQueryDelegate Delegate);
		/** Registers a delegate to invoke for querying an endpoint about its authority. */
		FDelegateHandle RegisterAuthorityQuery(const FGuid& EndpointId, FAuthorityQueryDelegate Delegate);

		void UnregisterStreamQuery(const FDelegateHandle& Handle);
		void UnregisterAuthorityQuery(const FDelegateHandle& Handle);
		
	private:

		/** Used to check whether we were destroyed after a response is received. */
		const TWeakPtr<FToken> Token;
		
		/**
		 * Used to send queries.
		 * 
		 * This FRegularQueryService's owner is supposed to make sure this FRegularQueryService is destroyed
		 * when the client shuts down.
		 */
		const IConcertSyncClient& OwningClient;

		// We use a multicast delegate here because it handles unsubscribing while being Broadcast
		DECLARE_MULTICAST_DELEGATE_OneParam(FMulticastStreamQueryDelegate, const TArray<FConcertBaseStreamInfo>&);
		DECLARE_MULTICAST_DELEGATE_OneParam(FMulticastAuthorityQueryDelegate, const TArray<FConcertAuthorityClientInfo>&);
		struct FStreamQueryInfo
		{
			TSet<FDelegateHandle> Handles;
			FMulticastStreamQueryDelegate Delegate;
		};
		struct FAuthorityQueryInfo
		{
			TSet<FDelegateHandle> Handles;
			FMulticastAuthorityQueryDelegate Delegate;
		};
		TMap<FGuid, FStreamQueryInfo> StreamQueryInfos;
		TMap<FGuid, FAuthorityQueryInfo> AuthorityQueryInfos;

		bool bIsHandlingQueryResponse = false;
		
		/** Queries the server for the client's current state. */
		void SendQueryEvent();
		
		void BuildStreamRequest(FConcertReplication_QueryReplicationInfo_Request& Request) const;
		void BuildAuthorityRequest(FConcertReplication_QueryReplicationInfo_Request& Request) const;
		
		/** Handles the server's response. */
		void HandleQueryResponse(const FConcertReplication_QueryReplicationInfo_Response& Response);
		void CompactDelegates();
	};
}

