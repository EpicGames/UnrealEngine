// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamRequestUtils.h"

#include "Replication/Stream/StreamChangeTracker.h"

#include "Containers/UnrealString.h"

namespace UE::MultiUserClient::Replication::StreamRequestUtils
{
	FConcertReplication_ChangeStream_Request BuildChangeRequest_CreateNewStream(
		const FGuid& StreamId,
		const FStreamChangelist& ObjectChanges,
		FFrequencyChangelist FrequencyChanges
		)
	{
		const TMap<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& ObjectsToPut = ObjectChanges.ObjectsToPut;
		
		FConcertReplication_ChangeStream_Request Request;
		Request.StreamsToAdd.Emplace();
		FConcertReplicationStream& NewStream = Request.StreamsToAdd[0];
		NewStream.BaseDescription.Identifier = StreamId;
			
		// If creating a new stream, the objects must be supplied in the description instead of in PutObjects!
		FConcertObjectReplicationMap& ReplicationMap = NewStream.BaseDescription.ReplicationMap;
		ReplicationMap.ReplicatedObjects.Reserve(ObjectsToPut.Num());
		for (const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& PutObjectPair : ObjectsToPut)
		{
			const TOptional<FConcertReplicatedObjectInfo> NewObjectInfo = PutObjectPair.Value.MakeObjectInfoIfValid();
			// The editing UI allows adding objects without properties (to make UX easier) - do not submit those to the server.
			if (!NewObjectInfo)
			{
				continue;
			}
			checkf(PutObjectPair.Key.StreamId == StreamId, TEXT("BuildChangeRequest should always use LocalClientStreamId ID!"));
				
			ReplicationMap.ReplicatedObjects.Add(PutObjectPair.Key.Object, *NewObjectInfo);
		}
		
		FConcertStreamFrequencySettings& FrequencySettings = NewStream.BaseDescription.FrequencySettings;
		if (FrequencyChanges.NewDefaults)
		{
			FrequencySettings.Defaults = MoveTemp(*FrequencyChanges.NewDefaults);
		}
		for (TPair<FSoftObjectPath, FConcertObjectReplicationSettings>& Overrides : FrequencyChanges.OverridesToAdd)
		{
			FrequencySettings.ObjectOverrides.Emplace(MoveTemp(Overrides.Key), MoveTemp(Overrides.Value));
		}
		
		return Request;
	}
		
	FConcertReplication_ChangeStream_Request BuildChangeRequest_UpdateExistingStream(
		const FGuid& StreamId,
		FStreamChangelist ObjectChanges,
		FFrequencyChangelist FrequencyChanges
		)
	{
		FConcertReplication_ChangeStream_Request Result { MoveTemp(ObjectChanges.ObjectsToRemove), MoveTemp(ObjectChanges.ObjectsToPut) };
		
		FConcertReplication_ChangeStream_Frequency& FrequencyChangeRequest = Result.FrequencyChanges.Add(StreamId);
		if (FrequencyChanges.NewDefaults)
		{
			FrequencyChangeRequest.NewDefaults = MoveTemp(*FrequencyChanges.NewDefaults); 
			FrequencyChangeRequest.Flags = EConcertReplicationChangeFrequencyFlags::SetDefaults; 
		}
		FrequencyChangeRequest.OverridesToAdd = MoveTemp(FrequencyChanges.OverridesToAdd);
		FrequencyChangeRequest.OverridesToRemove = MoveTemp(FrequencyChanges.OverridesToRemove);

		return Result;
	}
}
