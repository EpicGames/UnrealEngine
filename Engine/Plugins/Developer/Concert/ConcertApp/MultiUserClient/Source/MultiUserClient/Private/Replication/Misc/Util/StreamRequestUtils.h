// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IConcertClientReplicationManager.h"

class IConcertClientSession;

struct FConcertReplication_ChangeStream_Request;

namespace UE::MultiUserClient::Replication
{
	/** Describes changes that MU clients make to the objects in a stream. */
	struct FStreamChangelist
	{
		TSet<FConcertObjectInStreamID> ObjectsToRemove;
		TMap<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject> ObjectsToPut;
	};
	
	/** Describes changes that MU clients make to the frequency settings in a stream. */
	struct FFrequencyChangelist
	{
		TSet<FSoftObjectPath> OverridesToRemove;
		TMap<FSoftObjectPath, FConcertObjectReplicationSettings> OverridesToAdd;
		TOptional<FConcertObjectReplicationSettings> NewDefaults;

		bool IsEmpty() const { return OverridesToRemove.IsEmpty() && OverridesToAdd.IsEmpty() && !NewDefaults.IsSet(); }
	};
}

namespace UE::MultiUserClient::Replication::StreamRequestUtils
{
	/**
	 * Builds a request for creating a new stream.
	 * @param StreamId The stream that should be modified
	 * @param ObjectChanges The object changes to be made
	 * @param FrequencyChanges The frequency changes to be made
	 */
	FConcertReplication_ChangeStream_Request BuildChangeRequest_CreateNewStream(
		const FGuid& StreamId,
		const FStreamChangelist& ObjectChanges,
		FFrequencyChangelist FrequencyChanges = FFrequencyChangelist()
		);
	
	/**
	 * Builds a request for updating a preexisting stream.
	 * @param StreamId The stream that should be modified
	 * @param ObjectChanges The object changes to be made
	 * @param FrequencyChanges The frequency changes to be made
	 */
	FConcertReplication_ChangeStream_Request BuildChangeRequest_UpdateExistingStream(
		const FGuid& StreamId,
		FStreamChangelist ObjectChanges,
		FFrequencyChangelist FrequencyChanges = FFrequencyChangelist()
		);
}
