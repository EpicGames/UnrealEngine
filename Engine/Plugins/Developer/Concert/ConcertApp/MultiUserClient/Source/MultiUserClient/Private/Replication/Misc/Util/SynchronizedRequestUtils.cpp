// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynchronizedRequestUtils.h"

#include "ConcertLogGlobal.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Submission/Data/AuthoritySubmission.h"
#include "Replication/Submission/Data/StreamSubmission.h"
#include "Replication/Submission/ISubmissionOperation.h"

#include "Algo/AllOf.h"
#include "Containers/UnrealString.h"
#include <atomic>

namespace UE::MultiUserClient::Replication
{
	namespace ParallelSubmission::Private
	{
		static void RemoveInvalidClientEntries(const FOnlineClientManager& ClientManager, TMap<FGuid, FSubmissionParams> ParallelOperations)
		{
			for (auto It = ParallelOperations.CreateIterator(); It; ++It)
			{
				const TPair<FGuid, FSubmissionParams>& Operation =*It;
				if (!ensure(ClientManager.FindClient(Operation.Key)))
				{
					It.RemoveCurrent();
				}
			}
		}

		class FSyncOperation;

		/** Adapter for FSubmissionQueue. Handles submitting for one client and notifying FSyncOperation if completions. */
		class FDeferredSubmitter
			: public FNoncopyable
			, public IDeferredSubmitter
		{
			FSyncOperation& Owner;
			FOnlineClientManager& ClientManager;
			FSubmissionQueue& SubmissionQueue;
			const FGuid ClientId;
			const FSubmissionParams SubmissionParams;
			
		public:

			FDeferredSubmitter(
				FSyncOperation& InOwner,
				FOnlineClientManager& InClientManager,
				FSubmissionQueue& InSubmissionQueue,
				const FGuid& InClientId,
				FSubmissionParams&& InSubmissionParams
				)
				: Owner(InOwner)
				, ClientManager(InClientManager)
				, SubmissionQueue(InSubmissionQueue)
				, ClientId(InClientId)
				, SubmissionParams(MoveTemp(InSubmissionParams))
			{}

			virtual ~FDeferredSubmitter() override
			{
				checkf(IsInGameThread(), TEXT("You must destroy IParallelSubmissionOperation on the game thread"));
				if (FOnlineClient* Client = ClientManager.FindClient(ClientId))
				{
					Client->GetSubmissionQueue().Dequeue_GameThread(*this);
				}
			}

			void EnqueueOrSubmit()
			{
				SubmissionQueue.SubmitNowOrEnqueue_GameThread(*this);
			}
			
			const FGuid& GetClientId() const { return ClientId; }

			virtual void PerformSubmission_GameThread(ISubmissionWorkflow& Workflow) override;
		};

		/** Starts all submissions and synchronizes their concurrent completion. */
		class FSyncOperation : public IParallelSubmissionOperation, public TSharedFromThis<FSyncOperation>
		{
		public:
			
			FSyncOperation(FOnlineClientManager& ClientManager, TMap<FGuid, FSubmissionParams>&& Operations)
			{
				const int32 NumOperations = Operations.Num();
				IntermediateResults.StreamResponses.Reserve(NumOperations);
				IntermediateResults.AuthorityResponses.Reserve(NumOperations);
				SubOperations.Reserve(NumOperations);

				// Prepare the intermediate slots in advance to handle concurrent writing
				for (TPair<FGuid, FSubmissionParams>& Pair : Operations)
				{
					// Allocate slots
					const FGuid& ClientId = Pair.Key;
					IntermediateResults.StreamResponses.Add(ClientId);
					IntermediateResults.AuthorityResponses.Add(ClientId);

					FOnlineClient* Client = ClientManager.FindClient(ClientId);
					checkf(Client, TEXT("Was supposed to have been validated ExecuteParallelStreamChanges."))
					SubOperations.Emplace(*this, ClientManager, ClientId, MoveTemp(Pair.Value), Client->GetSubmissionQueue());
				}
			}

			virtual ~FSyncOperation() override
			{
				if (!bPromiseWasSet)
				{
					bPromiseWasSet = true;
					constexpr bool bWasCancelled = true;
					Promise.EmplaceValue(FParallelExecutionResult{ bWasCancelled });
				}
			}

			void StartSubmissions()
			{
				// FDeferredSubmitter needs access to SharedThis(), so it must be called outside of constructor.
				for (FPendingOperation& PendingOperation : SubOperations)
				{
					FDeferredSubmitter& Submitter = PendingOperation.Submitter;
					Submitter.EnqueueOrSubmit();
				}
			}

			//~ Begin IParallelSubmissionOperation Interface
			virtual TFuture<FParallelExecutionResult> OnCompletedFuture_AnyThread() override { return Promise.GetFuture(); }
			//~ End IParallelSubmissionOperation Interface

			void OnCompleteStream(const FGuid& ClientId, FSubmitStreamChangesResponse&& Response)
			{
				IntermediateResults.StreamResponses[ClientId] = MoveTemp(Response);
				FindOperationByClient(ClientId)->bHasCompletedStreamChanges.store(true, std::memory_order_release);

				FinishIfDone();
			}

			void OnCompleteAuthority(const FGuid& ClientId, FSubmitAuthorityChangesResponse&& Response)
			{
				IntermediateResults.AuthorityResponses[ClientId] = MoveTemp(Response);
				FindOperationByClient(ClientId)->bHasCompletedAuthority.store(true, std::memory_order_release);
				
				const bool bIsFailure = Response.ErrorCode == EAuthoritySubmissionResponseErrorCode::Timeout || (Response.Response && !Response.Response->RejectedObjects.IsEmpty());
				UE_CLOGF(bIsFailure, LogConcert, Warning, "Remote authority change to client %ls failed", *ClientId.ToString());
				FinishIfDone();
			}

			void OnSubmissionFailure(const FGuid& ClientId)
			{
				FPendingOperation* PendingOperation = FindOperationByClient(ClientId);
				PendingOperation->bHasCompletedStreamChanges.store(true, std::memory_order_release);
				PendingOperation->bHasCompletedAuthority.store(true, std::memory_order_release);
				FinishIfDone();
			}

		private:
			
			TPromise<FParallelExecutionResult> Promise;
			/** Must be atomic because FinishIfDone may be executed in parallel by the game thread (completing an operation) and the UDP (timing out an operation). */
			std::atomic<bool> bPromiseWasSet = false;

			struct FPendingOperation
			{
				/** Handles the submission. Unregisters from the client queue (if we're destroyed early). */
				FDeferredSubmitter Submitter;
			
				/**
				 * Whether the stream change has completed, i.e. OnCompleteStream has been called.
				 * 
				 * Must be atomic because FinishIfDone() may read while OnCompleteStream() / OnCompleteAuthority() are writing.
				 * The (unlikely) scenario is that the game thread is finishing an operation while the UDP thread is timing out a task.
				 */
				std::atomic<bool> bHasCompletedStreamChanges = false;
				/**
				 * Whether the authority change has completed, i.e. OnCompleteAuthority has been called.
				 * 
				 * Must be atomic because FinishIfDone() may read while OnCompleteStream() / OnCompleteAuthority() are writing.
				 * The (unlikely) scenario is that the game thread is finishing an operation while the UDP thread is timing out a task.
				 */
				std::atomic<bool> bHasCompletedAuthority = false;
				
				FPendingOperation(
					FSyncOperation& InOwner,
					FOnlineClientManager& InClientManager,
					const FGuid& InClientId,
					FSubmissionParams&& InSubmissionParams,
					FSubmissionQueue& InSubmissionQueue
					)
					: Submitter(InOwner, InClientManager, InSubmissionQueue, InClientId, MoveTemp(InSubmissionParams))
				{}
			};
			/**
			 * Each latent tasks has its own, pre-allocated slot which effectively handles potentially concurrent writes.
			 *
			 * Initially every task value is initialized to 2: one for each operation (stream, authority).
			 * Every time a sub-operation finishes, the value decrements.
			 * Once all values have reached 0, that means Operations can be Emplace()'ed with IntermediateResults.
			 */
			TArray<FPendingOperation> SubOperations;

			/** Holds intermediate results as they come in. */
			FParallelExecutionResult IntermediateResults;

			void FinishIfDone()
			{
				const bool bAllDone = Algo::AllOf(SubOperations, [](const FPendingOperation& OperationInfo)
				{
					// This is called right after OnCompleteStream or OnCompleteAuthority.
					// This may run concurrently: e.g. the game thread is completing an operation and the UDP is timing out an operation.
					// In that case, both threads wrote to these bools with memory_order_release (in OnCompleteStream and OnCompleteAuthority),
					// meaning the writes happen-before these reads.
					return OperationInfo.bHasCompletedStreamChanges.load(std::memory_order_acquire)
						&& OperationInfo.bHasCompletedAuthority.load(std::memory_order_acquire);
				});

				// In the above scenario, both game thread and UDP thread may have bAllDone. Using exchange, the first thread to write will Emplace the promise.
				if (bAllDone && !bPromiseWasSet.exchange(true))
				{
					Promise.EmplaceValue(MoveTemp(IntermediateResults));
				}
			}

			FPendingOperation* FindOperationByClient(const FGuid& ClientId)
			{
				return SubOperations.FindByPredicate([&ClientId](const FPendingOperation& Operation)
				{
					return Operation.Submitter.GetClientId() == ClientId;
				});
			}
		};

		void FDeferredSubmitter::PerformSubmission_GameThread(ISubmissionWorkflow& Workflow)
		{
			const TSharedPtr<ISubmissionOperation> SubmissionOperation = Workflow.SubmitChanges(SubmissionParams);
			if (ensure(SubmissionOperation))
			{
				SubmissionOperation->OnCompleteStreamChangesFuture_AnyThread()
					.Next([this, WeakOwner = Owner.AsWeak()](FSubmitStreamChangesResponse&& Response)
					{
						if (const TSharedPtr<FSyncOperation> SyncOperation = WeakOwner.Pin())
						{
							SyncOperation->OnCompleteStream(ClientId, MoveTemp(Response));
						}
					});
				SubmissionOperation->OnCompleteAuthorityChangeFuture_AnyThread()
					.Next([this, WeakOwner = Owner.AsWeak()](FSubmitAuthorityChangesResponse&& Response)
					{
						if (const TSharedPtr<FSyncOperation> SyncOperation = WeakOwner.Pin())
						{
							SyncOperation->OnCompleteAuthority(ClientId, MoveTemp(Response));
						}
					});
			}
			else
			{
				Owner.OnSubmissionFailure(ClientId);
			}
		}
	}
	
	TSharedPtr<IParallelSubmissionOperation> ExecuteParallelStreamChanges(FOnlineClientManager& ClientManager, TMap<FGuid, FSubmissionParams> ParallelOperations)
	{
		ParallelSubmission::Private::RemoveInvalidClientEntries(ClientManager, ParallelOperations);
		if (!ensure(!ParallelOperations.IsEmpty()))
		{
			return nullptr;
		}

		using namespace ParallelSubmission::Private;
		TSharedRef<FSyncOperation> Operation = MakeShared<FSyncOperation>(ClientManager, MoveTemp(ParallelOperations));
		Operation->StartSubmissions();
		return Operation;
	}

	TSharedPtr<IParallelSubmissionOperation> ExecuteParallelStreamChanges(FOnlineClientManager& ClientManager, TMap<FGuid, FConcertReplication_ChangeStream_Request> ParallelOperations)
	{
		if (!ensure(!ParallelOperations.IsEmpty()))
		{
			return nullptr;
		}
		
		TMap<FGuid, FSubmissionParams> Transformed;
		Transformed.Reserve(ParallelOperations.Num()); // check()s if Num() == 0
		for (TPair<FGuid, FConcertReplication_ChangeStream_Request>& RequestPair : ParallelOperations)
		{
			Transformed.Emplace(RequestPair.Key, FSubmissionParams{ MoveTemp(RequestPair.Value) });
		}
		
		return ExecuteParallelStreamChanges(ClientManager, MoveTemp(Transformed));
	}
}
