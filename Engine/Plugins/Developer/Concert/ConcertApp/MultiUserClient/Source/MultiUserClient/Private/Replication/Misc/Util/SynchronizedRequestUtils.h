// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Submission/Data/AuthoritySubmission.h"
#include "Replication/Submission/Data/StreamSubmission.h"

#include "Async/Future.h"
#include "Templates/UnrealTemplate.h"

struct FConcertReplication_ChangeStream_Request;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
	struct FSubmissionParams;

	struct FParallelExecutionResult
	{
		/** True if the TSharedPtr<IParallelSubmissionOperation> was destroyed before completion of submissions. */
		bool bWasCancelled = false;

		/** Maps client IDs to responses from the submissions. Empty if bWasCancelled. */
		TMap<FGuid, FSubmitStreamChangesResponse> StreamResponses;
		/** Maps client IDs to responses from the submissions. Empty if bWasCancelled. */
		TMap<FGuid, FSubmitAuthorityChangesResponse> AuthorityResponses;
	};

	/** Result of ExecuteParallelStreamChanges. Handles correctly unregistering when the object is destroyed. */
	class IParallelSubmissionOperation
	{
	public:

		/**
		 * @return Future completed when all parallel submissions have been completed.
		 * @note This can complete on any thread. Usually this finishes on the game thread but timeouts usually occur on the messaging (UDP) thread.
		 */
		virtual TFuture<FParallelExecutionResult> OnCompletedFuture_AnyThread() = 0;

		virtual ~IParallelSubmissionOperation() = default;
	};

	// TODO UE-201136: This code can be removed once a dedicated server operation for changing client streams has been implemented.
	
	/**
	 * Executes the stream changes in parallel on the remote clients.
	 *
	 * @param ClientManager Used to get the clients to send to
	 * @param ParallelOperations The operations to complete
	 * 
	 * @return Object you can get future through. This object also handles correct unregistering from other systems when you destroy it.
	 * @note The returned object MUST be destroyed on the game thread.
	 */
	TSharedPtr<IParallelSubmissionOperation> ExecuteParallelStreamChanges(FOnlineClientManager& ClientManager, TMap<FGuid, FSubmissionParams> ParallelOperations);
	/** Util for transforming into FSubmissionParams */
	TSharedPtr<IParallelSubmissionOperation> ExecuteParallelStreamChanges(FOnlineClientManager& ClientManager, TMap<FGuid, FConcertReplication_ChangeStream_Request> ParallelOperations);
}

