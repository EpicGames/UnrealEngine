// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/MultiUserReplicationCVars.h"

namespace UE::MultiUserClient::Replication
{
TAutoConsoleVariable<bool> CVarLogRemoteChangeRequestsAndResponses(
	TEXT("MultiUser.Replication.LogRemoteChangeRequestsAndResponses"),
	false,
	TEXT("Whether to log FMultiUser_ChangeRemote_Request & FMultiUser_ChangeRemote_Response received from other MU clients.")
	);
}
