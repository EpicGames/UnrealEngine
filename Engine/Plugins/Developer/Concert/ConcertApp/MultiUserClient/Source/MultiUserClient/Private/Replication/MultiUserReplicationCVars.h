// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertLogGlobal.h"
#include "HAL/IConsoleManager.h"
#include "JsonObjectConverter.h"

namespace UE::MultiUserClient::Replication
{
/** Whether to log FMultiUser_ChangeRemote_Request & FMultiUser_ChangeRemote_Response received from other MU clients. */
extern TAutoConsoleVariable<bool> CVarLogRemoteChangeRequestsAndResponses;

/** Logs a Concert network message, which is USTRUCT(). */
template<typename TMessage>
static void LogNetworkMessage(const TAutoConsoleVariable<bool>& ShouldLog, const TMessage& Message)
{
	if (ShouldLog.GetValueOnAnyThread())
	{
		FString JsonString;
		FJsonObjectConverter::UStructToJsonObjectString(TMessage::StaticStruct(), &Message, JsonString, 0, 0);
		UE_LOGF(LogConcert, Log, "%ls\n%ls", *TMessage::StaticStruct()->GetName(), *JsonString);
	}
}
}
