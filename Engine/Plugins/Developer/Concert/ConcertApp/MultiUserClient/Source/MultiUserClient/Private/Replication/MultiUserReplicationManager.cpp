// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserReplicationManager.h"

#include "ConcertLogGlobal.h"
#include "IConcertSyncClient.h"
#include "Misc/AnalyticsHandler.h"
#include "Replication/ChangeOperationTypes.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/IOfflineReplicationClient.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

#include "Containers/Ticker.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "FMultiUserReplicationManager"

namespace UE::MultiUserClient::Replication::PrivateReplicationManager
{
	class FOfflineClientAdapter : public IOfflineReplicationClient
	{
		const FOfflineClient& Client;
	public:
		
		FOfflineClientAdapter(const FOfflineClient& Client)
			: Client(Client)
		{}

		//~ Begin IOfflineReplicationClient Interface
		virtual const FConcertClientInfo& GetClientInfo() const override { return Client.GetClientInfo(); }
		virtual const FGuid& GetLastAssociatedEndpoint() const override { return Client.GetLastAssociatedEndpoint(); }
		virtual const FConcertBaseStreamInfo& GetPredictedStream() const override { return Client.GetPredictedStream(); }
		//~ End IOfflineReplicationClient Interface
	};
}

namespace UE::MultiUserClient::Replication
{
	FMultiUserReplicationManager::FMultiUserReplicationManager(TSharedRef<IConcertSyncClient> InClient)
		: Client(MoveTemp(InClient))
	{
		Client->GetConcertClient()->OnSessionConnectionChanged().AddRaw(
			this,
			&FMultiUserReplicationManager::OnSessionConnectionChanged
			);
	}

	FMultiUserReplicationManager::~FMultiUserReplicationManager()
	{
		Client->GetConcertClient()->OnSessionConnectionChanged().RemoveAll(this);
	}

	void FMultiUserReplicationManager::JoinReplicationSession()
	{
		IConcertClientReplicationManager* Manager = Client->GetReplicationManager();
		if (!ensure(ConnectionState == EMultiUserReplicationConnectionState::Disconnected)
			|| !ensure(Manager))
		{
			return;
		}

		ConnectionState = EMultiUserReplicationConnectionState::Connecting;
		Manager->JoinReplicationSession({})
			.Next([WeakThis = AsWeak()](ConcertSyncClient::Replication::FJoinReplicatedSessionResult&& JoinSessionResult)
			{
				// The future can execute on any thread
				ExecuteOnGameThread(TEXT("JoinReplicationSession"), [WeakThis, JoinSessionResult = MoveTemp(JoinSessionResult)]()
				{
					// Shutting down engine?
					if (const TSharedPtr<FMultiUserReplicationManager> ThisPin = WeakThis.Pin())
					{
						ThisPin->HandleReplicationSessionJoined(JoinSessionResult);
					}
				});
			});
	}

	void FMultiUserReplicationManager::OnSessionConnectionChanged(
		IConcertClientSession& ConcertClientSession,
		EConcertConnectionStatus ConcertConnectionStatus
		)
	{
		switch (ConcertConnectionStatus)
		{
		case EConcertConnectionStatus::Connecting:
			break;
		case EConcertConnectionStatus::Connected:
			JoinReplicationSession();
			break;
		case EConcertConnectionStatus::Disconnecting:
			break;
		case EConcertConnectionStatus::Disconnected:
			OnLeaveSession(ConcertClientSession);
			break;
		default: ;
		}
	}

	void FMultiUserReplicationManager::OnLeaveSession(IConcertClientSession&)
	{
		// This destroys the UI and tells any other potential system to stop referencing anything in ConnectedState (such as shared ptrs)...
		SetConnectionStateAndBroadcast(EMultiUserReplicationConnectionState::Disconnected);
		// ... so now it is safe to destroy ConnectedState.
		ConnectedState.Reset();
	}

	void FMultiUserReplicationManager::HandleReplicationSessionJoined(const ConcertSyncClient::Replication::FJoinReplicatedSessionResult& JoinSessionResult)
	{
		const bool bSuccess = JoinSessionResult.ErrorCode == EJoinReplicationErrorCode::Success;
		if (bSuccess)
		{
			ConnectedState.Emplace(Client, DiscoveryContainer);
			SetupClientConnectionEvents();
			SetConnectionStateAndBroadcast(EMultiUserReplicationConnectionState::Connected);

			// For convenience, the client should attempt to restore the content when they last left.
			RestoreContentFromLastTime();
		}
		else
		{
			SetConnectionStateAndBroadcast(EMultiUserReplicationConnectionState::Disconnected);
		}
	}

	void FMultiUserReplicationManager::SetConnectionStateAndBroadcast(EMultiUserReplicationConnectionState NewState)
	{
		ConnectionState = NewState;
		OnReplicationConnectionStateChangedDelegate.Broadcast(ConnectionState);
	}

	void FMultiUserReplicationManager::RestoreContentFromLastTime()
	{
		Client->GetReplicationManager()->RestoreContent(
			{
				.Flags = EConcertReplicationRestoreContentFlags::All | EConcertReplicationRestoreContentFlags::ValidateUniqueClient
			}
		);
	}

	void FMultiUserReplicationManager::SetupClientConnectionEvents()
	{
		FOnlineClientManager& OnlineClientManager = ConnectedState->OnlineClientManager;
		OnlineClientManager.ForEachClient([this](FOnlineClient& InClient){ SetupClientDelegates(InClient); return EBreakBehavior::Continue; });
		OnlineClientManager.OnPostRemoteClientAdded().AddRaw(this, &FMultiUserReplicationManager::OnReplicationClientConnected);

		FOfflineClientManager& OfflineClientManager = ConnectedState->OfflineClientManager;
		OfflineClientManager.OnClientsChanged().AddRaw(this, &FMultiUserReplicationManager::OnInternalOfflineClientsChanged);
		OfflineClientManager.OnClientContentChanged().AddRaw(this, &FMultiUserReplicationManager::OnInternalOfflineClientContentChanged);
	}

	void FMultiUserReplicationManager::SetupClientDelegates(FOnlineClient& InClient) const
	{
		InClient.GetStreamSynchronizer().OnServerStreamChanged().AddRaw(this, &FMultiUserReplicationManager::OnClientStreamServerStateChanged, InClient.GetEndpointId());
		InClient.GetAuthoritySynchronizer().OnServerAuthorityChanged().AddRaw(this, &FMultiUserReplicationManager::OnClientAuthorityServerStateChanged, InClient.GetEndpointId());
	}

	void FMultiUserReplicationManager::OnClientStreamServerStateChanged(const FGuid EndpointId) const
	{
		UE_LOGF(LogConcert, Verbose, "Client %ls stream changed", *EndpointId.ToString());
		OnStreamServerStateChangedDelegate.Broadcast(EndpointId);
	}

	void FMultiUserReplicationManager::OnClientAuthorityServerStateChanged(const FGuid EndpointId) const
	{
		UE_LOGF(LogConcert, Verbose, "Client %ls authority changed", *EndpointId.ToString());
		OnAuthorityServerStateChangedDelegate.Broadcast(EndpointId);
	}

	const FConcertObjectReplicationMap* FMultiUserReplicationManager::FindReplicationMapForClient(const FGuid& ClientId) const
	{
		if (ConnectedState && ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed.")))
		{
			if (const FOnlineClient* OnlineClient = ConnectedState->OnlineClientManager.FindClient(ClientId))
			{
				return &OnlineClient->GetStreamSynchronizer().GetServerState();
			}

			if (const FOfflineClient* OfflineClient = ConnectedState->OfflineClientManager.FindClient(ClientId))
			{
				return &OfflineClient->GetPredictedStream().ReplicationMap;
			}

			return nullptr;
		}
		
		return nullptr;
	}

	const FConcertStreamFrequencySettings* FMultiUserReplicationManager::FindReplicationFrequenciesForClient(const FGuid& ClientId) const
	{
		if (ConnectedState && ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed.")))
		{
			if (const FOnlineClient* OnlineClient = ConnectedState->OnlineClientManager.FindClient(ClientId))
			{
				return &OnlineClient->GetStreamSynchronizer().GetFrequencySettings();
			}

			if (const FOfflineClient* OfflineClient = ConnectedState->OfflineClientManager.FindClient(ClientId))
			{
				return &OfflineClient->GetPredictedStream().FrequencySettings;
			}

			return nullptr;
		}
		return nullptr;
	}

	bool FMultiUserReplicationManager::IsReplicatingObject(const FGuid& ClientId, const FSoftObjectPath& ObjectPath) const 
	{
		if (ConnectedState && ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed.")))
		{
			const FOnlineClient* ReplicationClient = ConnectedState->OnlineClientManager.FindClient(ClientId);
			return ReplicationClient && ReplicationClient->GetAuthoritySynchronizer().HasAuthorityOver(ObjectPath);
		}
		return false;
	}

	void FMultiUserReplicationManager::RegisterReplicationDiscoverer(TSharedRef<IReplicationDiscoverer> Discoverer)
	{
		if (ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed.")))
		{
			DiscoveryContainer.AddDiscoverer(Discoverer);
		}
	}

	void FMultiUserReplicationManager::RemoveReplicationDiscoverer(const TSharedRef<IReplicationDiscoverer>& Discoverer)
	{
		if (ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed.")))
		{
			DiscoveryContainer.RemoveDiscoverer(Discoverer);
		}
	}

	TSharedRef<IClientChangeOperation> FMultiUserReplicationManager::EnqueueChanges(const FGuid& ClientId, TAttribute<FChangeClientReplicationRequest> SubmissionParams)
	{
		if (!ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed.")))
		{
			return FExternalClientChangeRequestHandler::MakeFailedOperation(EChangeStreamOperationResult::NotOnGameThread, EChangeAuthorityOperationResult::NotOnGameThread);
		}
		
		if (ConnectedState)
		{
			FOnlineClient* ReplicationClient = ConnectedState->OnlineClientManager.FindClient(ClientId);
			return ReplicationClient
				? ReplicationClient->GetExternalRequestHandler().HandleRequest(MoveTemp(SubmissionParams))
				: FExternalClientChangeRequestHandler::MakeFailedOperation(EChangeStreamOperationResult::UnknownClient, EChangeAuthorityOperationResult::UnknownClient);
		}
		return FExternalClientChangeRequestHandler::MakeFailedOperation(EChangeStreamOperationResult::NotInSession, EChangeAuthorityOperationResult::NotInSession);
	}
	
	void FMultiUserReplicationManager::ForEachOfflineClient(TFunctionRef<EBreakBehavior(const IOfflineReplicationClient&)> Callback) const
	{
		if (ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed."))
			&& ConnectedState)
		{
			ConnectedState->OfflineClientManager.ForEachClient([&Callback](const FOfflineClient& OfflineClient)
			{
				const PrivateReplicationManager::FOfflineClientAdapter OfflineClientAdapter(OfflineClient);
				return Callback(OfflineClientAdapter);
			});
		}
	}

	bool FMultiUserReplicationManager::FindOfflineClient(const FGuid& ClientId, TFunctionRef<void(const IOfflineReplicationClient&)> Callback) const
	{
		if (!ensureMsgf(IsInGameThread(), TEXT("To simplify implementation, only calls from game thread are allowed."))
			|| !ConnectedState)
		{
			return false;
		}

		const FOfflineClient* OfflineClient = ConnectedState->OfflineClientManager.FindClient(ClientId);
		if (OfflineClient)
		{
			Callback(PrivateReplicationManager::FOfflineClientAdapter(*OfflineClient));
			return true;
		}
		
		return false;
	}

	FMultiUserReplicationManager::FConnectedState::FConnectedState(
		TSharedRef<IConcertSyncClient> InClient, FReplicationDiscoveryContainer& InDiscoveryContainer
	)
		: Client(InClient)
		, QueryService(*InClient)
		, OnlineClientManager(
			  InClient,
			  InClient->GetConcertClient()->GetCurrentSession().ToSharedRef(),
			  InDiscoveryContainer,
			  QueryService.GetStreamAndAuthorityQueryService()
		  )
		, OfflineClientManager(*InClient, OnlineClientManager)
		, UnifiedClientView(*InClient, OnlineClientManager, OfflineClientManager)
		, MuteManager(*InClient, QueryService.GetMuteStateQueryService(), OnlineClientManager.GetAuthorityCache())
		, PresetManager(*InClient, OnlineClientManager, MuteManager.GetSynchronizer())
		, PropertySelector(OnlineClientManager, OfflineClientManager)
		, AutoPropertyOwnershipTaker(PropertySelector, OnlineClientManager.GetLocalClient(), OnlineClientManager.GetAuthorityCache())
		, ChangeLevelHandler(*InClient, OnlineClientManager.GetLocalClient().GetClientEditModel().Get())
		, PreventReplicatedPropertyTransaction(*InClient, OnlineClientManager, MuteManager)
		, UserNotifier(*InClient->GetConcertClient(), OnlineClientManager, MuteManager)
		, AnalyticsHandler(*InClient->GetConcertClient(), OnlineClientManager)
	{}
}

#undef LOCTEXT_NAMESPACE
