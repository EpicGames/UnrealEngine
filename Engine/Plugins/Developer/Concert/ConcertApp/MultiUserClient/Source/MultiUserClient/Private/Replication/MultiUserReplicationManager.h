// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Client/Online/OnlineClientManager.h"
#include "IConcertSession.h"
#include "Client/UnifiedClientView.h"
#include "Client/Offline/OfflineClientManager.h"
#include "Misc/AnalyticsHandler.h"
#include "Misc/ChangeLevelHandler.h"
#include "Misc/Notification/ReplicationUserNotifier.h"
#include "Misc/PreventReplicatedPropertyTransaction.h"
#include "Misc/PropertySelection/AutoPropertyOwnershipTaker.h"
#include "Misc/PropertySelection/UserPropertySelector.h"
#include "Misc/Query/RegularQueryService.h"
#include "Muting/MuteStateManager.h"
#include "Preset/PresetManager.h"
#include "Replication/IMultiUserReplication.h"
#include "Replication/Stream/Discovery/ReplicationDiscoveryContainer.h"

#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IConcertClientSession;
class IConcertSyncClient;
enum class EConcertConnectionStatus : uint8;

namespace UE::ConcertSharedSlate { class IEditableReplicationStreamModel; }
namespace UE::ConcertSyncClient { struct FJoinReplicatedSessionResult; }

namespace UE::MultiUserClient::Replication
{
	class FOnlineClient;
	
	enum class EMultiUserReplicationConnectionState : uint8
	{
		Connecting,
		Connected,
		Disconnected
	};
	
	/**
	 * Interacts with the replication system on behalf of Multi-User to execute actions specific to Multi-User workflows;
	 * this is opposed to other uses of the replication API, e.g. users using the system in a shipped game.
	 *
	 * This class implements the Fence design pattern. All knowledge Multi-User might need should be encapsulated by this class.
	 */
	class FMultiUserReplicationManager
		: public TSharedFromThis<FMultiUserReplicationManager>
		, public FNoncopyable
		, public IMultiUserReplication
	{
	public:
		
		FMultiUserReplicationManager(TSharedRef<IConcertSyncClient> InClient);
		virtual ~FMultiUserReplicationManager() override;

		/**
		 * Joins the replication session.
		 *
		 * Joining occurs automatically after successful connection to the Concert session. However the request can be
		 * rejected by the server. In that case, the user can manually attempt to connect again, which is what this
		 * is exposed publicly for.
		 */
		void JoinReplicationSession();

		/** @note This pointer is only valid while connected and can become stale. Listen for OnReplicationConnectionStateChanged. */
		FOnlineClientManager* GetOnlineClientManager() { return ConnectedState ? &ConnectedState->OnlineClientManager : nullptr; }
		const FOnlineClientManager* GetOnlineClientManager() const { return ConnectedState ? &ConnectedState->OnlineClientManager : nullptr; }
		
		/** @note This pointer is only valid while connected and can become stale. Listen for OnReplicationConnectionStateChanged. */
		FOfflineClientManager* GetOfflineClientManager() { return ConnectedState ? &ConnectedState->OfflineClientManager : nullptr; }
		const FOfflineClientManager* GetOfflineClientManager() const { return ConnectedState ? &ConnectedState->OfflineClientManager : nullptr; }
		
		/** @note This pointer is only valid while connected and can become stale. Listen for OnReplicationConnectionStateChanged. */
		FUnifiedClientView* GetUnifiedClientView() { return ConnectedState ? &ConnectedState->UnifiedClientView : nullptr; }
		const FUnifiedClientView* GetUnifiedClientView() const { return ConnectedState ? &ConnectedState->UnifiedClientView : nullptr; }
		
		/** @note This pointer is only valid while connected and can become stale. Listen for OnReplicationConnectionStateChanged. */
		FMuteStateManager* GetMuteManager() { return ConnectedState ? &ConnectedState->MuteManager : nullptr; }
		const FMuteStateManager* GetMuteManager() const { return ConnectedState ? &ConnectedState->MuteManager : nullptr; }

		/** @note This pointer is only valid while connected and can become stale. Listen for OnReplicationConnectionStateChanged. */
		FPresetManager* GetPresetManager() { return ConnectedState ? &ConnectedState->PresetManager : nullptr; }
		const FPresetManager* GetPresetManager() const { return ConnectedState ? &ConnectedState->PresetManager : nullptr; }

		/** @note This pointer is only valid while connected and can become stale. Listen for OnReplicationConnectionStateChanged. */
		FUserPropertySelector* GetUserPropertySelector() { return ConnectedState ? &ConnectedState->PropertySelector : nullptr; }
		const FUserPropertySelector* GetUserPropertySelector() const { return ConnectedState ? &ConnectedState->PropertySelector : nullptr; }

		/** Called when the connection to the replication system changes. */
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnReplicationConnectionStateChanged, EMultiUserReplicationConnectionState /*NewState*/);
		FOnReplicationConnectionStateChanged& OnReplicationConnectionStateChanged() { return OnReplicationConnectionStateChangedDelegate; }
		EMultiUserReplicationConnectionState GetConnectionState() const { return ConnectionState; }

		//~ Begin IMultiUserReplication Interface
		virtual const FConcertObjectReplicationMap* FindReplicationMapForClient(const FGuid& ClientId) const override;
		virtual const FConcertStreamFrequencySettings* FindReplicationFrequenciesForClient(const FGuid& ClientId) const override;
		virtual bool IsReplicatingObject(const FGuid& ClientId, const FSoftObjectPath& ObjectPath) const override;
		virtual void RegisterReplicationDiscoverer(TSharedRef<IReplicationDiscoverer> Discoverer) override;
		virtual void RemoveReplicationDiscoverer(const TSharedRef<IReplicationDiscoverer>& Discoverer) override;
		virtual TSharedRef<IClientChangeOperation> EnqueueChanges(const FGuid& ClientId, TAttribute<FChangeClientReplicationRequest> SubmissionParams) override;
		virtual void ForEachOfflineClient(TFunctionRef<EBreakBehavior(const IOfflineReplicationClient&)> Callback) const override;
		virtual bool FindOfflineClient(const FGuid& ClientId, TFunctionRef<void(const IOfflineReplicationClient&)> Callback) const override;
		virtual FOnServerStateChanged& OnStreamServerStateChanged() override { return OnStreamServerStateChangedDelegate; }
		virtual FOnServerStateChanged& OnAuthorityServerStateChanged() override { return OnAuthorityServerStateChangedDelegate; }
		virtual FOnOfflineClientsChanged& OnOfflineClientsChanged() override { return OnOfflineClientsChangedDelegate; }
		virtual FOnServerStateChanged& OnOfflineClientContentChanged() override { return OnOfflineClientContentChangedDelegate; }
		//~ End IMultiUserReplication Interface

	private:

		/** Client through which the replication bridge is accessed. */
		const TSharedRef<IConcertSyncClient> Client;
		
		/** Reflects the current connection state to the replication system (note: this does not reflect the state to the concert session). */
		EMultiUserReplicationConnectionState ConnectionState = EMultiUserReplicationConnectionState::Disconnected;

		struct FConnectedState
		{
			// The order of the below members matters, so they are destroyed in the right order!
			// Rule: Lower systems can only reference higher systems (reminder: C++ destroys in reverse declaration order).

			/** Keep the client alive because it is referenced by the lower systems. */
			TSharedRef<IConcertSyncClient> Client;
			
			/**
			 * Regularly queries server state (only if a system is subscribed to it), such as remote client streams, authority, and global mute state.
			 * Shared by subsystems, such as remote clients. This ensures requests are bundled, reducing the number of network requests. 
			 */
			FRegularQueryService QueryService;
			
			/**
			 * Synchronizes replication settings of clients connected to the session.
			 * Creates UMultiUserReplicationSessionPreset, which is displayed by UI.
			 * Keeps the preset in sync with the state on the server.
			 *
			 * Only valid when ConnectionState == EMultiUserReplicationConnectionState::Connected.
			 */
			FOnlineClientManager OnlineClientManager;
			/** Keeps track of clients that were once in the session but are no longer to be able to display their old settings in the UI. */
			FOfflineClientManager OfflineClientManager;
			/** Adapter abstraction that allows systems to query online and offline clients with a unified interface. */
			FUnifiedClientView UnifiedClientView;
			
			/** Interacts with the mute global server mute system. */
			FMuteStateManager MuteManager;
			/** Saves and loads presets for the session. Accessed by UI. */
			FPresetManager PresetManager;

			/**
			 * Manages the properties the user is iterating on in the replication session.
			 * The bottom-half property section in the replication UI uses this to keep track of which properties the user has selected for which properties.
			 */
			FUserPropertySelector PropertySelector;
			/** Automatically takes and releases ownership over properties the user adds to the set of selected properties. */
			FAutoPropertyOwnershipTaker AutoPropertyOwnershipTaker;
			
			/** Clears local client's registered objects when leaving map. */
			FChangeLevelHandler ChangeLevelHandler;
			/** Prevents recording of transactions that change properties that are being replicated by a client. */
			FPreventReplicatedPropertyTransaction PreventReplicatedPropertyTransaction;
			/** This system notifies users when requests go wrong */
			FReplicationUserNotifier UserNotifier;
			/** Sends analytics data about replication use. */
			FAnalyticsHandler AnalyticsHandler;

			FConnectedState(TSharedRef<IConcertSyncClient> InClient, FReplicationDiscoveryContainer& InDiscoveryContainer);
		};
		/** Set when connected to a replication session. */
		TOptional<FConnectedState> ConnectedState;

		/** Allows external modules to register discoverers for adding properties, etc. */
		FReplicationDiscoveryContainer DiscoveryContainer;

		/** Called when ConnectionState changes. */
		FOnReplicationConnectionStateChanged OnReplicationConnectionStateChangedDelegate;
		
		/** Triggers when a client's known server state has changed. */
		FOnServerStateChanged OnStreamServerStateChangedDelegate;
		/** Triggers when a client's known server state has changed. */
		FOnServerStateChanged OnAuthorityServerStateChangedDelegate;
		/** Triggers when the offline clients have changed. */
		FOnOfflineClientsChanged OnOfflineClientsChangedDelegate;
		/** Triggers when the content of an offline client has changed. Also triggered as part of OnOfflineClientsChangedDelegate. */
		FOnServerStateChanged OnOfflineClientContentChangedDelegate;

		/** Callback into Concert for when client connection has changed. */
		void OnSessionConnectionChanged(IConcertClientSession& ConcertClientSession, EConcertConnectionStatus ConcertConnectionStatus);
		/** Leaves the current replication session */
		void OnLeaveSession(IConcertClientSession&);

		/** Handles server response for joining replication session */
		void HandleReplicationSessionJoined(const ConcertSyncClient::Replication::FJoinReplicatedSessionResult& JoinSessionResult);
		/** Sets the current connection state and triggers OnReplicationConnectionStateChangedDelegate. */
		void SetConnectionStateAndBroadcast(EMultiUserReplicationConnectionState NewState);
		/** Restores the content the client had when they last left the session. */
		void RestoreContentFromLastTime();

		/** Sets up delegates for implementing the broadcasting of OnStreamServerStateChangedDelegate and OnAuthorityServerStateChangedDelegate. */
		void SetupClientConnectionEvents();
		void OnInternalOfflineClientsChanged() const { OnOfflineClientsChangedDelegate.Broadcast(); }
		void OnInternalOfflineClientContentChanged(FOfflineClient& OfflineClient) const { OnOfflineClientContentChangedDelegate.Broadcast(OfflineClient.GetLastAssociatedEndpoint()); }
		void OnReplicationClientConnected(FRemoteClient& RemoteClient) const { SetupClientDelegates(RemoteClient); }

		/** Sets up delegates for reacting to content changes. */
		void SetupClientDelegates(FOnlineClient& InClient) const;
		void OnClientStreamServerStateChanged(const FGuid EndpointId) const;
		void OnClientAuthorityServerStateChanged(const FGuid EndpointId) const;
	};
}

