// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuteChangeTracker.h"

#include "Replication/MultiUserReplicationManager.h"

#include "UObject/SoftObjectPath.h"

namespace UE::MultiUserClient::Replication
{
	FMuteChangeTracker::FMuteChangeTracker(FMuteStateSynchronizer& InMuteStateSynchronizer, const FGlobalAuthorityCache& InAuthorityCache)
		: MuteStateSynchronizer(InMuteStateSynchronizer)
		, AuthorityCache(InAuthorityCache)
	{
		MuteStateSynchronizer.OnMuteStateChanged().AddRaw(this, &FMuteChangeTracker::RefreshChanges);
	}

	FMuteChangeTracker::~FMuteChangeTracker()
	{
		MuteStateSynchronizer.OnMuteStateChanged().RemoveAll(this);
	}

	void FMuteChangeTracker::ToggleMuteState(const FSoftObjectPath& ObjectPath)
	{
		if (CanChangeMuteState(ObjectPath))
		{
			const bool bIsCurrentlyMuted = MuteStateSynchronizer.IsMuted(ObjectPath);
			LocalChanges.Add(ObjectPath, !bIsCurrentlyMuted);
			OnLocalMuteStateOverridenDelegate.Broadcast();
		}
	}

	bool FMuteChangeTracker::CanChangeMuteState(const FSoftObjectPath& ObjectPath) const
	{
		return AuthorityCache.IsObjectOrChildReferenced(ObjectPath);
	}

	bool FMuteChangeTracker::IsMuted(const FSoftObjectPath& ObjectPath) const
	{
		return MuteStateSynchronizer.IsMuted(ObjectPath);
	}

	void FMuteChangeTracker::RefreshChanges()
	{
		for (auto It = LocalChanges.CreateIterator(); It; ++It)
		{
			const FSoftObjectPath& ObjectPath = It->Key;
			const bool bWillBeMuted = It->Value;
		
			if (!CanChangeMuteState(ObjectPath) || bWillBeMuted == MuteStateSynchronizer.IsMuted(ObjectPath))
			{
				It.RemoveCurrent();
			}
		}
	}

	FConcertReplication_ChangeMuteState_Request FMuteChangeTracker::BuildChangeRequest() const
	{
		FConcertReplication_ChangeMuteState_Request Request;
		for (const TPair<FSoftObjectPath, bool>& Change : LocalChanges)
		{
			// May have changed since local mute change was made
			if (!CanChangeMuteState(Change.Key))
			{
				continue;
			}

			TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& MuteMap = Change.Value ? Request.ObjectsToMute : Request.ObjectsToUnmute;
			MuteMap.Add(Change.Key, FConcertReplication_ObjectMuteSetting{ EConcertReplicationMuteOption::ObjectAndSubobjects });
		}
		return Request;
	}
}
