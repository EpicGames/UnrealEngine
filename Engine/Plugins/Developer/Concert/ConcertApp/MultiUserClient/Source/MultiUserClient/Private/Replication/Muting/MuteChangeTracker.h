// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/SoftObjectPath.h"

struct FConcertReplication_ChangeMuteState_Request;

namespace UE::MultiUserClient::Replication
{
	class FGlobalAuthorityCache;
	class FMuteStateSynchronizer;

	DECLARE_MULTICAST_DELEGATE(FOnLocalMuteStateOverriden);

	/** Keeps track of changes to the mute state the local application has made but not yet submitted. */
	class FMuteChangeTracker : public FNoncopyable
	{
	public:
		FMuteChangeTracker(FMuteStateSynchronizer& InMuteStateSynchronizer UE_LIFETIMEBOUND, const FGlobalAuthorityCache& InAuthorityCache UE_LIFETIMEBOUND);
		~FMuteChangeTracker();
		
		/** @return Whether the mute state of this object can be changed. True if the object is at least in one client stream. */
		bool CanChangeMuteState(const FSoftObjectPath& ObjectPath) const;
		/** @return Returns whether ObjectPath according to the local, unsubmitted changes or returns the state on the server if no local changes have been made. */
		bool IsMuted(const FSoftObjectPath& ObjectPath) const;
		
		/** Toggles the mute state of ObjectPath if allowed. The mute setting will apply to all subobjects as well (unless they are overriden, too). */
		void ToggleMuteState(const FSoftObjectPath& ObjectPath);

		/** Clears all local changes, so they do not show up the next time BuildChangeRequest is called. */
		void ClearLocalMuteOverrides() { LocalChanges.Reset(); }
		/** Compares the local changes against the server state and kicks out any (now) invalid changes. */
		void RefreshChanges();

		/** @return A request contains all the pending local changes. */
		FConcertReplication_ChangeMuteState_Request BuildChangeRequest() const;

		/** Broadcasts when there a local change has been made that can be submitted. */
		FOnLocalMuteStateOverriden& OnLocalMuteStateOverriden() { return OnLocalMuteStateOverridenDelegate; }

	private:

		/** Knows of the server state. Used to diff state. */
		FMuteStateSynchronizer& MuteStateSynchronizer;

		/** Used to check whether an object is referenced in any stream. Needed so we don't build requests containing objects the server does not know about. */
		const FGlobalAuthorityCache& AuthorityCache;
		
		/** Maps objects to whether they should be muted. */
		TMap<FSoftObjectPath, bool> LocalChanges;

		/** Broadcasts when there a local change has been made that can be submitted. */
		FOnLocalMuteStateOverriden OnLocalMuteStateOverridenDelegate;
	};
}

