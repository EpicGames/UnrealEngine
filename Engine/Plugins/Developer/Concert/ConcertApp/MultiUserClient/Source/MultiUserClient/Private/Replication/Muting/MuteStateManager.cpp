// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuteStateManager.h"

#include "IConcertSyncClient.h"
#include "Misc/CoreDelegates.h"
#include "Replication/MultiUserReplicationManager.h"

namespace UE::MultiUserClient::Replication
{
	FMuteStateManager::FMuteStateManager(const IConcertSyncClient& InClient, FMuteStateQueryService& InMuteQueryService, const FGlobalAuthorityCache& InAuthorityCache)
		: Client(InClient)
		, MuteQueryService(InMuteQueryService)
		, MuteStateSynchronizer(InMuteQueryService)
		, ChangeTracker(MuteStateSynchronizer, InAuthorityCache)
	{
		ChangeTracker.OnLocalMuteStateOverriden().AddLambda([this]()
		{
			if (!FCoreDelegates::OnEndFrame.IsBoundToObject(this))
			{
				FCoreDelegates::OnEndFrame.AddRaw(this, &FMuteStateManager::OnEndOfFrame);
			}
		});
	}

	FMuteStateManager::~FMuteStateManager()
	{
		FCoreDelegates::OnEndFrame.RemoveAll(this);
	}

	void FMuteStateManager::OnEndOfFrame()
	{
		if (!bIsMuteChangeInProgress)
		{
			FCoreDelegates::OnEndFrame.RemoveAll(this);
			SendChangeRequest();
		}
	}

	void FMuteStateManager::SendChangeRequest()
	{
		IConcertClientReplicationManager* ReplicationManager = Client.GetReplicationManager();
		const FConcertReplication_ChangeMuteState_Request Request = ChangeTracker.BuildChangeRequest();
		if (Request.IsEmpty() || !ensure(ReplicationManager))
		{
			return;
		}

		bIsMuteChangeInProgress = true;
		ReplicationManager->ChangeMuteState(Request)
			.Next([this, Request, WeakToken = Token.ToWeakPtr()](FConcertReplication_ChangeMuteState_Response&& Response)
			{
				if (!WeakToken.IsValid())
				{
					return;
				}
					
				bIsMuteChangeInProgress = false;
				if (Response.IsSuccess())
				{
					// This will implicitly refresh ChangeTracker's local state because FMuteStateSynchronizer::OnMuteStateChanged is broadcast.
					MuteStateSynchronizer.UpdateStateFromSuccessfulChange(Request);
				}
				else
				{
					// If the response fails, keep the local changes. The state will be cleansed next time we attempt to submit or the FMuteStateQueryService receives the new server state.

					// This can e.g. notify the user of the failure
					OnMuteRequestFailureDelegate.Broadcast(Request, Response);
				}
			});

		// This is a hacky way of instantly getting the UI to refresh.
		// It's a hack because it generates another network request.
		// Instead, the local application should predict the new mute state locally.
		MuteQueryService.RequestInstantRefresh();
	}
}
