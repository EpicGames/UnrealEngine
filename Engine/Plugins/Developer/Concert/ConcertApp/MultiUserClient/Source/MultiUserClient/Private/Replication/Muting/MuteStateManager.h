// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuteChangeTracker.h"
#include "MuteStateSynchronizer.h"
#include "Replication/IToken.h"

#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;
struct FConcertReplication_ChangeMuteState_Request;
struct FConcertReplication_ChangeMuteState_Response;

namespace UE::MultiUserClient::Replication
{
	class FGlobalAuthorityCache;
	class FMuteStateQueryService;

	/** Manages all interaction with the application instance to the server regarding muting. */
	class FMuteStateManager : public FNoncopyable
	{
	public:

		FMuteStateManager(const IConcertSyncClient& InClient UE_LIFETIMEBOUND, FMuteStateQueryService& InMuteQueryService UE_LIFETIMEBOUND, const FGlobalAuthorityCache& InAuthorityCache UE_LIFETIMEBOUND);
		~FMuteStateManager();

		/** @return Object with which the objects' mute state can be changed and queried. */
		FMuteChangeTracker& GetChangeTracker() { return ChangeTracker; }
		/** @return Object with which you can query whether an object is muted. */
		const FMuteStateSynchronizer& GetSynchronizer() const { return MuteStateSynchronizer; }
		
		/** Broadcasts when a mute request fails. */
		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMuteRequestFailure, const FConcertReplication_ChangeMuteState_Request&, const FConcertReplication_ChangeMuteState_Response&);
		FOnMuteRequestFailure& OnMuteRequestFailure() { return OnMuteRequestFailureDelegate; }

	private:

		/** When the mute request is answered, we use this to detect whether we've been destroyed. */
		const TSharedRef<FToken> Token = FToken::Make();

		/** Used to send submit mute changes to the server. */
		const IConcertSyncClient& Client;
		/** We'll request an instant refresh of the server's mute state after submitting a request.. */
		FMuteStateQueryService& MuteQueryService; 
		
		/** Knows of the current mute state on the server. */
		FMuteStateSynchronizer MuteStateSynchronizer;
		/** Tracks locally made changes that still need to be submitted to the server. */
		FMuteChangeTracker ChangeTracker;

		/** Broadcasts when a mute request fails. */
		FOnMuteRequestFailure OnMuteRequestFailureDelegate;

		/** Whether we're currently waiting for a response to a mute change. */
		bool bIsMuteChangeInProgress = false;

		/** When a change is made to ChangeTracker, we subscribe FCoreDelegates::OnEndFrame to this function. */
		void OnEndOfFrame();
		/** Checks the changes pending in ChangeTracker and sends them to the server, if any. */
		void SendChangeRequest();
	};
}

