// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuteStateSynchronizer.h"

#include "Replication/MultiUserReplicationManager.h"

#include "UObject/SoftObjectPath.h"

namespace UE::MultiUserClient::Replication
{
	FMuteStateSynchronizer::FMuteStateSynchronizer(FMuteStateQueryService& InMuteQueryService)
		: MuteQueryService(InMuteQueryService)
	{
		MuteQueryService.OnMuteStateQueried().AddRaw(this, &FMuteStateSynchronizer::UpdateFromServerState);
	}

	FMuteStateSynchronizer::~FMuteStateSynchronizer()
	{
		MuteQueryService.OnMuteStateQueried().RemoveAll(this);
	}

	void FMuteStateSynchronizer::UpdateStateFromSuccessfulChange(const FConcertReplication_ChangeMuteState_Request& Request)
	{
		if (!ensure(!Request.IsEmpty()))
		{
			return;
		}

		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& MutedObject : Request.ObjectsToMute)
		{
			MutedObjects.Add(MutedObject.Key);
			ExplicitlyMutedObjects.Add(MutedObject.Key, MutedObject.Value);
		}
		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& UnmutedObject : Request.ObjectsToUnmute)
		{
			MutedObjects.Remove(UnmutedObject.Key);
			ExplicitlyUnmutedObjects.Add(UnmutedObject.Key, UnmutedObject.Value);
		}
		
		OnMuteStateChangedDelegate.Broadcast();
	}

	void FMuteStateSynchronizer::UpdateFromServerState(const FConcertReplication_QueryMuteState_Response& NewMuteState)
	{
		const int32 NumExpectedElements = NewMuteState.ExplicitlyMutedObjects.Num() + NewMuteState.ImplicitlyMutedObjects.Num();
		MutedObjects.Empty(NumExpectedElements);

		Algo::Transform(NewMuteState.ExplicitlyMutedObjects, MutedObjects, [](const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& Pair){ return Pair.Key; });
		MutedObjects.Append(NewMuteState.ImplicitlyMutedObjects);
		ExplicitlyMutedObjects = NewMuteState.ExplicitlyMutedObjects;
		ExplicitlyUnmutedObjects = NewMuteState.ExplicitlyUnmutedObjects;

		OnMuteStateChangedDelegate.Broadcast();
	}
}
