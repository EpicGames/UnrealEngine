// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Messages/Muting.h"

#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/SoftObjectPath.h"

struct FConcertReplication_ChangeMuteState_Request;
struct FConcertReplication_QueryMuteState_Response;

namespace UE::MultiUserClient::Replication
{
	class FMuteStateQueryService;

	DECLARE_MULTICAST_DELEGATE(FOnMuteStateChanged);

	/**
	 * Responsible for answering questions about whether an object is muted.
	 * In the future, this may be extended with reasons (e.g. object Actor.Component is muted because Actor is muted).
	 */
	class FMuteStateSynchronizer : public FNoncopyable
	{
	public:

		FMuteStateSynchronizer(FMuteStateQueryService& InMuteQueryService UE_LIFETIMEBOUND);
		~FMuteStateSynchronizer();

		/** @return Whether ObjectPath is muted. */
		bool IsMuted(const FSoftObjectPath& ObjectPath) const { return MutedObjects.Contains(ObjectPath); }

		const TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& GetExplicitlyMutedObjects() const { return ExplicitlyMutedObjects; }
		const TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& GetExplicitlyUnmutedObjects() const { return ExplicitlyUnmutedObjects; }

		/**
		 * Update the mute state after the local application has successfully changed mute state.
		 * FMuteStateQueryService will eventually notify us of the change but this applies it instantaneously.
		 */
		void UpdateStateFromSuccessfulChange(const FConcertReplication_ChangeMuteState_Request& Request);

		/** Broadcasts whent the mute state changes (either because FMuteStateQueryService received updated state or because the local application has successfully made a change request). */
		FOnMuteStateChanged& OnMuteStateChanged() { return OnMuteStateChangedDelegate; }
		
	private:

		/** Updates us with the new mute state from the server regularily. */
		FMuteStateQueryService& MuteQueryService;
		
		/** All explicitly muted objects on the server. */
		TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> ExplicitlyMutedObjects;
		/** All explicitly unmuted objects on the server.*/
		TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> ExplicitlyUnmutedObjects;
		/** A cached, final view of muted objects that combines ExplicitlyMutedObjects and ExplicitlyUnmutedObjects. */
		TSet<FSoftObjectPath> MutedObjects;

		/** Broadcasts whent the mute state changes (either because FMuteStateQueryService received updated state or because the local application has successfully made a change request). */
		FOnMuteStateChanged OnMuteStateChangedDelegate;

		/** Updates MutedObjects from the mute state on the server. */
		void UpdateFromServerState(const FConcertReplication_QueryMuteState_Response& NewMuteState);
	};
}

