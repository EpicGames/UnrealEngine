// Copyright Epic Games, Inc. All Rights Reserved.

#include "PresetManager.h"

#include "Assets/MultiUserReplicationClientPreset.h"
#include "Assets/MultiUserReplicationSessionPreset.h"
#include "Assets/MultiUserReplicationStream.h"
#include "ConcertLogGlobal.h"
#include "IConcertSyncClient.h"
#include "Replication/Client/ClientUtils.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Misc/ActorLabelRemappingEditor.h"
#include "Replication/Misc/ReplicationStreamUtils.h"
#include "Replication/Muting/MuteStateManager.h"
#include "Replication/Stream/MultiUserStreamId.h"

#include "Engine/World.h"
#include "FileHelpers.h"

namespace UE::MultiUserClient::Replication::Private
{
	enum class EPresetState : uint8
	{
		/** The preset is valid to apply. */
		Valid,
		/** Preset contains changes but none could be mapped to objects. */
		FailedToMapObjects
	};
	
	EPresetState Combine(EPresetState Base, EPresetState New)
	{
		return Base == EPresetState::FailedToMapObjects && New == EPresetState::FailedToMapObjects
			? EPresetState::FailedToMapObjects : EPresetState::Valid;
	}
	
	static void RemoveEmptyObjectsFromRequest(FConcertReplicationStream& Stream)
	{
		TMap<FSoftObjectPath, FConcertReplicatedObjectInfo>& ReplicationMap = Stream.BaseDescription.ReplicationMap.ReplicatedObjects;
		for (auto It = ReplicationMap.CreateIterator(); It; ++It)
		{
			const bool bIsEmpty = It->Value.PropertySelection.ReplicatedProperties.IsEmpty();
			if (bIsEmpty)
			{
				It.RemoveCurrent();
			}
		}
	}

	static void ProcessNonEmptyRequest(const FConcertSessionClientInfo& ClientSessionInfo, FConcertReplication_PutState_Request& Request, const FMultiUserReplicationClientPreset* ClientSessionContent, const FConcertObjectReplicationMap& OriginalReplicationMap)
	{
		const FGuid& StreamId = MultiUserStreamID;
		FConcertReplicationStream Stream { { .Identifier = StreamId, .ReplicationMap = OriginalReplicationMap } };
		// Empty objects will be rejected by the server
		RemoveEmptyObjectsFromRequest(Stream);
		Stream.BaseDescription.FrequencySettings = ClientSessionContent->FrequencySettings;
		Request.NewStreams.Add(ClientSessionInfo.ClientEndpointId, { TArray{ Stream } });
			
		// MU automatically requests authority when it adds an object.
		// We'll assume that that authority was granted when the preset was created - if it actually was not, our request may fail due to overlapping authority.
		TArray<FConcertObjectInStreamID>& OwnedObjects = Request.NewAuthorityState.Add(ClientSessionInfo.ClientEndpointId).Objects;
		Algo::Transform(Stream.BaseDescription.ReplicationMap.ReplicatedObjects, OwnedObjects, [&StreamId](const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair)
		{
			return FConcertObjectInStreamID{ StreamId, Pair.Key };
		});
	}

	static EPresetState AddClientToRequest(
		const FConcertSessionClientInfo& ClientSessionInfo,
		FConcertReplication_PutState_Request& Request,
		const UMultiUserReplicationSessionPreset& Preset,
		bool bClearUnreferencedClients
	)
	{
		const auto AddRequestToClearClient = [&Request, &ClientSessionInfo]()
		{
			Request.NewStreams.Add(ClientSessionInfo.ClientEndpointId, {});
		};
		
		const FMultiUserReplicationClientPreset* ClientSessionContent = Preset.GetClientContent(ClientSessionInfo.ClientInfo);
		if (!ClientSessionContent)
		{
			if (bClearUnreferencedClients)
			{
				AddRequestToClearClient();
			}
			return EPresetState::Valid;
		}

		const FConcertObjectReplicationMap& OriginalReplicationMap = ClientSessionContent->ReplicationMap;
		if (OriginalReplicationMap.IsEmpty())
		{
			AddRequestToClearClient();
			return EPresetState::Valid;
		}

		check(GWorld);
		const FConcertObjectReplicationMap TranslatedReplicationMap = ConcertSyncCore::RemapReplicationMap(
			OriginalReplicationMap,
			ClientSessionContent->ActorLabelRemappingData,
			*GWorld
			);
		if (TranslatedReplicationMap.IsEmpty())
		{
			AddRequestToClearClient();
			return EPresetState::FailedToMapObjects;
		}
		else
		{
			ProcessNonEmptyRequest(ClientSessionInfo, Request, ClientSessionContent, TranslatedReplicationMap);
			return EPresetState::Valid;
		}
	}

	static TOptional<EPresetState> FillStreamAndAuthorityRequest(
		FConcertReplication_PutState_Request& Request,
		const UMultiUserReplicationSessionPreset& Preset,
		const IConcertClientSession& Session,
		bool bClearUnreferencedClients
		)
	{
		// GWorld is required by AddClientToRequest
		if (ensure(GWorld))
		{
			EPresetState PresetState = AddClientToRequest(
				{ Session.GetSessionClientEndpointId(), Session.GetLocalClientInfo() }, Request, Preset, bClearUnreferencedClients
				);
			for (const FConcertSessionClientInfo& ClientSessionInfo : Session.GetSessionClients())
			{
				const EPresetState RemoteClientState = AddClientToRequest(ClientSessionInfo, Request, Preset, bClearUnreferencedClients);
				PresetState = Combine(PresetState, RemoteClientState);
			}
			return PresetState;
		}
		return {};
	}

	static void FillMuteStateRequest(
		FConcertReplication_PutState_Request& Request,
		const UMultiUserReplicationSessionPreset& Preset,
		const IConcertClientSession& Session
		)
	{
		FConcertReplication_ChangeMuteState_Request& MuteRequest = Request.MuteChange;
		MuteRequest.Flags = EConcertReplicationMuteRequestFlags::ClearMuteState;
		
		const auto IsReferencedByConnectedClient = [&Request, &Session](const FSoftObjectPath& ObjectPath)
		{
			return Algo::AnyOf(Request.NewStreams, [&Session, &ObjectPath](const TPair<FGuid, FConcertReplicationStreamArray>& ClientContent)
			{
				const FGuid& EndpointId = ClientContent.Key;
				FConcertSessionClientInfo Dummy;
				
				const bool bIsConnected = Session.GetSessionClientEndpointId() == EndpointId || Session.FindSessionClient(EndpointId, Dummy);
				// Case: User muted Floor but only Floor.StaticMeshComponent0 is replicated. Hence, also look for child objects being referenced.
				const bool bIsReferenced = ConcertSyncCore::IsObjectOrChildReferenced(ClientContent.Value.Streams, ObjectPath);
				
				return bIsConnected && bIsReferenced;
			});
		};

		const FMultiUserMuteSessionContent& MuteContent = Preset.GetMuteContent();
		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& MutedObject : MuteContent.MutedObjects)
		{
			if (IsReferencedByConnectedClient(MutedObject.Key))
			{
				MuteRequest.ObjectsToMute.Add(MutedObject.Key, MutedObject.Value);
			}
		}
		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& UnmutedObject : MuteContent.UnmutedObjects)
		{
			if (IsReferencedByConnectedClient(UnmutedObject.Key))
			{
				MuteRequest.ObjectsToUnmute.Add(UnmutedObject.Key, UnmutedObject.Value);
			}
		}
	}
	
	static TPair<FConcertReplication_PutState_Request, TOptional<EPresetState>> BuildRequest(
		const UMultiUserReplicationSessionPreset& Preset,
		const IConcertClientSession& Session,
		EApplyPresetFlags Flags
		)
	{
		FConcertReplication_PutState_Request Request;

		const bool bClearUnreferencedClients = EnumHasAnyFlags(Flags, EApplyPresetFlags::ClearUnreferencedClients);
		const TOptional<EPresetState> PresetState = FillStreamAndAuthorityRequest(Request, Preset, Session, bClearUnreferencedClients);

		// TODO UE-219829: Once the server allows sending the mute state disconnected clients should have when they rejoin,
		// simply send over all mute state instead of doing filtering here.
		FillMuteStateRequest(Request, Preset, Session);
		
		return { MoveTemp(Request), PresetState };
	}

	static EReplaceSessionContentErrorCode ExtractErrorCode(const FConcertReplication_PutState_Response& Response)
	{
		switch (Response.ResponseCode)
		{
		case EConcertReplicationPutStateResponseCode::Success: return EReplaceSessionContentErrorCode::Success;
		case EConcertReplicationPutStateResponseCode::Timeout: return EReplaceSessionContentErrorCode::Timeout;
		case EConcertReplicationPutStateResponseCode::FeatureDisabled: return EReplaceSessionContentErrorCode::FeatureDisabled;
						
		case EConcertReplicationPutStateResponseCode::ClientUnknown: [[fallthrough]];
		case EConcertReplicationPutStateResponseCode::StreamError: [[fallthrough]];
		case EConcertReplicationPutStateResponseCode::AuthorityConflict: [[fallthrough]];
		case EConcertReplicationPutStateResponseCode::MuteError: [[fallthrough]];
		default: return EReplaceSessionContentErrorCode::Rejected; 
		}
	}

	static void RemoveEmptyObjectsFromLocalClient(ConcertSharedSlate::IEditableReplicationStreamModel& EditModel)
	{
		TArray<FSoftObjectPath> EmptyObjects;
		EditModel.ForEachReplicatedObject([&EditModel, &EmptyObjects](const FSoftObjectPath& Object)
		{
			if (EditModel.GetNumProperties(Object) == 0)
			{
				EmptyObjects.Add(Object);
			}
			return EBreakBehavior::Continue;
		});
		EditModel.RemoveObjects(EmptyObjects);
	}

	static TArray<TPair<const FOnlineClient*, FConcertClientInfo>> DetermineSavedClients(
		const FOnlineClientManager& ClientManager,
		const IConcertClientSession& Session,
		const FSavePresetOptions& Options
		)
	{
		TArray<TPair<const FOnlineClient*, FConcertClientInfo>> IncludedClients;
		ClientManager.ForEachClient([&Session, &IncludedClients, &Options](const FOnlineClient& Client)
		{
			FConcertClientInfo ClientInfo;
			const bool bGotClientInfo = ClientUtils::GetClientDisplayInfo(Session, Client.GetEndpointId(), ClientInfo);
			if (!ensure(bGotClientInfo))
			{
				return EBreakBehavior::Continue;
			}

			const bool bIsFilteredOut = Options.ClientFilterDelegate.IsBound() && Options.ClientFilterDelegate.Execute(ClientInfo) == EFilterResult::Exclude; 
			if (bIsFilteredOut)
			{
				return EBreakBehavior::Continue;
			}
		
			IncludedClients.Emplace(&Client, ClientInfo);
			return EBreakBehavior::Continue;
		});
		return IncludedClients;
	}
}

namespace UE::MultiUserClient::Replication
{
	FPresetManager::FPresetManager(
		const IConcertSyncClient& SyncClient,
		const FOnlineClientManager& ClientManager,
		const FMuteStateSynchronizer& MuteStateSynchronizer
		)
		: SyncClient(SyncClient)
		, ClientManager(ClientManager)
		, MuteStateSynchronizer(MuteStateSynchronizer)
	{
		IConcertClientReplicationManager& ReplicationManager = *SyncClient.GetReplicationManager();
		ReplicationManager.OnPostRemoteEditApplied().AddRaw(this, &FPresetManager::OnPostRemoteEditApplied);
	}

	FPresetManager::~FPresetManager()
	{
		IConcertClientReplicationManager& ReplicationManager = *SyncClient.GetReplicationManager();
		ReplicationManager.OnPostRemoteEditApplied().RemoveAll(this);
	
		if (InProgressSessionReplacementOp)
		{
			InProgressSessionReplacementOp->EmplaceValue(FReplaceSessionContentResult{ EReplaceSessionContentErrorCode::Cancelled });
			InProgressSessionReplacementOp.Reset();
		}
	}

	TFuture<FReplaceSessionContentResult> FPresetManager::ReplaceSessionContentWithPreset(const UMultiUserReplicationSessionPreset& Preset, EApplyPresetFlags Flags)
	{
		if (!ensure(!IsPresetChangeInProgress()))
		{
			return MakeFulfilledPromise<FReplaceSessionContentResult>(EReplaceSessionContentErrorCode::InProgress).GetFuture();
		}
		
		const TSharedPtr<IConcertClientSession> Session = SyncClient.GetConcertClient()->GetCurrentSession();
		if (!ensure(Session))
		{
			return MakeFulfilledPromise<FReplaceSessionContentResult>(EReplaceSessionContentErrorCode::Timeout).GetFuture();
		}

		const auto[Request, PresetState] = Private::BuildRequest(Preset, *Session, Flags);
		if (!PresetState || *PresetState == Private::EPresetState::FailedToMapObjects)
		{
			const EReplaceSessionContentErrorCode ErrorCode = PresetState
				? EReplaceSessionContentErrorCode::NoObjectsFound : EReplaceSessionContentErrorCode::NoWorld;
			return MakeFulfilledPromise<FReplaceSessionContentResult>(ErrorCode).GetFuture();
		}
		
		InProgressSessionReplacementOp = MakeShared<TPromise<FReplaceSessionContentResult>>();
		SyncClient.GetReplicationManager()
			->PutClientState(Request)
			.Next(
				[this, WeakPromise = TWeakPtr<TPromise<FReplaceSessionContentResult>>(InProgressSessionReplacementOp)]
				(FConcertReplication_PutState_Response&& Response)
			{
				// If WeakPromise is stale, the request completes after our owning FPresetManager has been destroyed.
				// In that case, it is not safe to access this.
				if (const TSharedPtr<TPromise<FReplaceSessionContentResult>> PromisePin = WeakPromise.Pin())
				{
					// Destroy before emplacing promise because its future may trigger another ReplaceSessionContentWithPreset call (unlikely); PromisePin will keep it alive for now.
					InProgressSessionReplacementOp.Reset();
					PromisePin->EmplaceValue(Private::ExtractErrorCode(Response));

					// The client may have added an object via the Add button but not assigned any properties.
					// Those empty objects exist locally only and were never submitted to the server.
					// Remove those because it is a client expectation that the final list only contains the objects that were in the preset.
					ConcertSharedSlate::IEditableReplicationStreamModel& EditModel = *ClientManager.GetLocalClient().GetClientEditModel();
					Private::RemoveEmptyObjectsFromLocalClient(EditModel);
				}
			});
		
		return InProgressSessionReplacementOp->GetFuture();
	}

	ECanSaveResult FPresetManager::CanSavePreset(const FSavePresetOptions& Options) const
	{
		const TSharedPtr<IConcertClientSession> Session = SyncClient.GetConcertClient()->GetCurrentSession();
		checkf(Session, TEXT("FPresetManager is only supposed to exist while in a session"));
		
		const TArray<TPair<const FOnlineClient*, FConcertClientInfo>> IncludedClients = Private::DetermineSavedClients(ClientManager, *Session, Options);
		return IncludedClients.IsEmpty()
			? ECanSaveResult::NoClients
			: ECanSaveResult::Yes;
	}

	UMultiUserReplicationSessionPreset* FPresetManager::ExportToPresetAndSaveAs(const FSavePresetOptions& Options)
	{
		UMultiUserReplicationSessionPreset* Preset = ExportToPreset(Options);
		if (Preset)
		{
			TArray<UObject*> SavedAssets;
			FEditorFileUtils::SaveAssetsAs({ Preset }, SavedAssets);
		}
		return Preset;
	}

	UMultiUserReplicationSessionPreset* FPresetManager::ExportToPreset(const FSavePresetOptions& Options) const
	{
		const TSharedPtr<IConcertClientSession> Session = SyncClient.GetConcertClient()->GetCurrentSession();
		checkf(Session, TEXT("FPresetManager is only supposed to exist while in a session"));

		const TArray<TPair<const FOnlineClient*, FConcertClientInfo>> IncludedClients = Private::DetermineSavedClients(ClientManager, *Session, Options);
		if (IncludedClients.IsEmpty())
		{
			return nullptr;
		}
		
		UMultiUserReplicationSessionPreset* Preset = NewObject<UMultiUserReplicationSessionPreset>(
			GetTransientPackage(),
			MakeUniqueObjectName(GetTransientPackage(), UMultiUserReplicationSessionPreset::StaticClass(), TEXT("ReplicationPreset")),
			// Mark as transient so FEditorFileUtils::SaveAssetsAs creates a new package for the object.
			RF_Transient
			);
		for (const TPair<const FOnlineClient*, FConcertClientInfo>& ClientData : IncludedClients)
		{
			const auto[Client, ClientInfo] = ClientData;
			UMultiUserReplicationStream* CopiedClientStream = Client->GetClientStreamObject();
			FMultiUserReplicationClientPreset* TargetClientPreset = Preset->AddClientIfUnique(ClientInfo, MultiUserStreamID);
			if (!TargetClientPreset)
			{
				UE_LOGF(LogConcert, Warning,
					"There are multiple clients with display name %ls and device name %ls in the session. Only the 1st encountered will be saved into the preset. Did you perhaps launch 2 editors on the same machine (if so you can use -CONCERTDISPLAYNAME)?",
					*ClientInfo.DisplayName,
					*ClientInfo.DeviceName
					);
				continue;
			}

			TargetClientPreset->ReplicationMap = CopiedClientStream->ReplicationMap;
			TargetClientPreset->FrequencySettings = Client->GetStreamSynchronizer().GetFrequencySettings();
			TargetClientPreset->ActorLabelRemappingData = ConcertSyncCore::GenerateRemappingData(TargetClientPreset->ReplicationMap);
		}

		Preset->SetMuteContent(
			FMultiUserMuteSessionContent(MuteStateSynchronizer.GetExplicitlyMutedObjects(), MuteStateSynchronizer.GetExplicitlyUnmutedObjects())
			);
		
		return Preset;
	}

	void FPresetManager::OnPostRemoteEditApplied(const ConcertSyncClient::Replication::FRemoteEditEvent& Event) const
	{
		if (Event.Reason == EConcertReplicationChangeClientReason::PutRequest)
		{
			// The client may have added an object via the Add button but not assigned any properties.
			// Those empty objects exist locally only and were never submitted to the server.
			// Remove those because it is a client expectation that the final list only contains the objects that were in the preset.
			ConcertSharedSlate::IEditableReplicationStreamModel& EditModel = *ClientManager.GetLocalClient().GetClientEditModel();
			Private::RemoveEmptyObjectsFromLocalClient(EditModel);
		}
	}
}
