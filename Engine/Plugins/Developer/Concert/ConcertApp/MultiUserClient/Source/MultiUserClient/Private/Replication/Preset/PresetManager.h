// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReplaceSessionContentResult.h"
#include "SavePresetOptions.h"

#include "Async/Future.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Templates/UnrealTemplate.h"

struct FConcertClientInfo;
class IConcertSyncClient;
class UMultiUserReplicationSessionPreset;
enum class EMultiUserClientPresetLoadMode : uint8;

namespace UE::ConcertSyncClient::Replication { struct FRemoteEditEvent; }

namespace UE::MultiUserClient::Replication
{
	class FMuteStateSynchronizer;
	class FOnlineClientManager;
	
	/**
	 * Implements all logic for managing presets in the MU session: saving and loading presets.
	 * The UI directly interfaces with this class.
	 */
	class FPresetManager : public FNoncopyable
	{
	public:
		
		FPresetManager(
			const IConcertSyncClient& SyncClient UE_LIFETIMEBOUND,
			const FOnlineClientManager& ClientManager UE_LIFETIMEBOUND,
			const FMuteStateSynchronizer& MuteStateSynchronizer UE_LIFETIMEBOUND
			);
		~FPresetManager();

		/** @return Whether any preset is currently being applied. */
		bool IsPresetChangeInProgress() const { return InProgressSessionReplacementOp.IsValid(); }
		
		/** Applies Preset to all clients in the session. */
		TFuture<FReplaceSessionContentResult> ReplaceSessionContentWithPreset(const UMultiUserReplicationSessionPreset& Preset, EApplyPresetFlags Flags = EApplyPresetFlags::None);

		/** @return Whether a preset can be saved (i.e. at least one client is included). */
		ECanSaveResult CanSavePreset(const FSavePresetOptions& Options = {}) const;
		/**
		 * Exports the current session content to a preset, asks the user where to save it, then saves it.
		 * @return A future that finishes when saving has completed.
		 */
		UMultiUserReplicationSessionPreset* ExportToPresetAndSaveAs(const FSavePresetOptions& Options = {});

	private:

		/** Used to get display information of clients in the session. */
		const IConcertSyncClient& SyncClient;
		/** Used to get the clients' replication content. */
		const FOnlineClientManager& ClientManager;
		/** Used to get the mute state when saving. */
		const FMuteStateSynchronizer& MuteStateSynchronizer;

		/** Non-null for as long as the ReplaceSessionContentWithPreset network request takes. */
		TSharedPtr<TPromise<FReplaceSessionContentResult>> InProgressSessionReplacementOp;

		/** Exports the current session content to a preset. */
		UMultiUserReplicationSessionPreset* ExportToPreset(const FSavePresetOptions& Options) const;

		/** Called when the local client receives a remote edit. */
		void OnPostRemoteEditApplied(const ConcertSyncClient::Replication::FRemoteEditEvent&) const;
	};
}
