// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MultiUserReplicationClientPreset.h"
#include "Assets/MultiUserReplicationSessionPreset.h"
#include "Misc/EBreakBehavior.h"

#include <type_traits>

namespace UE::MultiUserClient::Replication
{
	/** Lists out all actors saved in the preset. */
	template<typename TLambda>
	requires std::is_invocable_r_v<EBreakBehavior, TLambda, const FSoftObjectPath& /*ActorPath*/, const FString& /*Label*/>
	void ForEachSavedActorLabel(const UMultiUserReplicationSessionPreset& Preset, TLambda&& Callback)
	{
		for (const FMultiUserReplicationClientPreset& ClientPreset : Preset.GetClientPresets())
		{
			for (const TPair<FSoftObjectPath, FConcertReplicationRemappingData_Actor>& Pair : ClientPreset.ActorLabelRemappingData.ActorData)
			{
				const FSoftObjectPath& ActorPath = Pair.Key;
				const FString& Label = Pair.Value.Label;
				if (Callback(ActorPath, Label) == EBreakBehavior::Break)
				{
					return;
				}
			}
		}
	}
}
