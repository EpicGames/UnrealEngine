// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::MultiUserClient::Replication
{
	enum class EReplaceSessionContentErrorCode : uint8
	{
		/** Request completed successfully. */
		Success,
		/** The preset did not make any changes because no objects could be found in the world. */
		NoObjectsFound,
		/** GWorld was not valid. */
		NoWorld,
		
		/** Request cancelled because FPresetManager was destroyed - probably because the use left the session during the request. */
		Cancelled,
		/** Another locally initiated operation is already in progress. */
		InProgress,
		
		/** Request timed out */
		Timeout,
		/** The feature is not enabled (i.e. EConcertSyncSessionFlags::ShouldEnableRemoteEditing or EConcertSyncSessionFlags::ShouldAllowGlobalMuting were not set on the server).  */
		FeatureDisabled,
		/** Server rejected the change because it was not valid */
		Rejected
	};
	
	/** Result of FPresetManager::ReplaceSessionContentWithPreset. */
	struct FReplaceSessionContentResult
	{
		EReplaceSessionContentErrorCode ErrorCode;

		FReplaceSessionContentResult(EReplaceSessionContentErrorCode ErrorCode = EReplaceSessionContentErrorCode::Success)
			: ErrorCode(ErrorCode)
		{}

		bool IsSuccess() const { return ErrorCode == EReplaceSessionContentErrorCode::Success; }
	};
}

