// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

struct FConcertClientInfo;

namespace UE::MultiUserClient::Replication
{
	enum class EApplyPresetFlags : uint8
	{
		None,
		/** If set clients that were not in the session when the preset was created will get their content reset, too. */
		ClearUnreferencedClients = 1 << 0
	};
	ENUM_CLASS_FLAGS(EApplyPresetFlags);

	enum class EFilterResult : uint8 { Include, Exclude };
	DECLARE_DELEGATE_RetVal_OneParam(EFilterResult, FFilterClientForPreset, const FConcertClientInfo&);
	
	/** Options for saving a preset */
	struct FSavePresetOptions
	{
		/** Filter that decides whether a client should be included in the preset. */
		FFilterClientForPreset ClientFilterDelegate; 
	};

	enum class ECanSaveResult : uint8
	{
		/** Yes, a preset can be saved. */
		Yes,
		/** There are no clients to save for */
		NoClients
	};
}