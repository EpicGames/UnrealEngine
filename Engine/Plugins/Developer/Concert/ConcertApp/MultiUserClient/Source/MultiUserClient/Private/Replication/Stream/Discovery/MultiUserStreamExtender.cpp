// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserStreamExtender.h"

#include "ReplicationDiscoveryContainer.h"
#include "Replication/IReplicationDiscoveryContext.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Editor/Model/Extension/IStreamExtensionContext.h"
#include "Settings/MultiUserReplicationSettings.h"

namespace UE::MultiUserClient::Replication
{
	FMultiUserStreamExtender::FMultiUserStreamExtender(const FGuid& InClientId, FReplicationDiscoveryContainer& InRegisteredExtenders)
		: ClientId(InClientId)
		, ExtendBySettings(
			// Use MU settings for auto adding properties & objects
			TAttribute<const FConcertStreamObjectAutoBindingRules*>::CreateLambda([]()
			{
				return &UMultiUserReplicationSettings::Get()->ReplicationEditorSettings;
			}))
		, RegisteredExtenders(InRegisteredExtenders)
	{}

	void FMultiUserStreamExtender::ExtendStream(UObject& ExtendedObject, ConcertSharedSlate::IStreamExtensionContext& Context)
	{
		if (bShouldExtend)
		{
			ExtendBySettings.ExtendStream(ExtendedObject, Context);
			ExtendStreamWithRegisteredDiscoverers(ExtendedObject, Context);
		}
	}
	
	void FMultiUserStreamExtender::ExtendStreamWithRegisteredDiscoverers(UObject& ExtendedObject, ConcertSharedSlate::IStreamExtensionContext& Context) const
	{
		class FDiscoveryContext : public IReplicationDiscoveryContext
		{
			ConcertSharedSlate::IStreamExtensionContext& Context;
		public:
			
			explicit FDiscoveryContext(ConcertSharedSlate::IStreamExtensionContext& InContext)
				: Context(InContext)
			{}

			virtual void AddPropertyTo(UObject& Object, FConcertPropertyChain PropertyChain) override
			{
				Context.AddPropertyTo(Object, MoveTemp(PropertyChain));
			}
			virtual void AddAdditionalObject(UObject& Object) override
			{
				Context.AddAdditionalObject(Object);
			}
		};
		
		FDiscoveryContext MultiUserContext(Context);
		RegisteredExtenders.DiscoverReplicationSettings({ ExtendedObject, ClientId, MultiUserContext });
	}
}
