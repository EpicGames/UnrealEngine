// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Extension/IStreamExtender.h"
#include "Replication/Editor/Model/Extension/StreamExtenderBySettings.h"

#include "Misc/Guid.h"

namespace UE::MultiUserClient::Replication
{
	class FReplicationDiscoveryContainer;
	
	/**
	 * When the user adds an object, this object handles auto selecting properties and adding additional objects from context.
	 * 
	 * The following sources exist:
	 * - Static Settings: user can specify properties & objects in the MU project settings
	 * - Dynamic API: by using the IMultiUserReplication, external modules can register dynamic rules for auto-discovery using IReplicationDiscoverer.
	 *
	 * Every FReplicationClient owns an instance of this and executes it when an object is added for the client by the local editor.
	 * Effectively, this is
	 */
	class FMultiUserStreamExtender : public ConcertSharedSlate::IStreamExtender
	{
	public:
		
		/**
		 * @param InClientId The ID of the owning FReplicationClient instance.
		 * @param InRegisteredExtenders Holds all IReplicationDiscoverer that were registered to Multi User. The caller ensures it outlives the constructed instance.
		 */
		FMultiUserStreamExtender(const FGuid& InClientId, FReplicationDiscoveryContainer& InRegisteredExtenders);

		/** Sets whether the stream should be extended. */
		void SetShouldExtend(bool bValue) { bShouldExtend = bValue; }

		//~ Begin IStreamExtender Interface
		virtual void ExtendStream(UObject& ExtendedObject, ConcertSharedSlate::IStreamExtensionContext& Context) override;
		//~ End IStreamExtender Interface

	private:

		/** The client ID to which this extender is adds objects. */
		const FGuid ClientId;

		/** Whether this extender is enabled. */
		bool bShouldExtend = false;
		
		/** Handles properties from the MU settings. */
		ConcertClientSharedSlate::FStreamExtenderBySettings ExtendBySettings;
		/** Allows generic extenders to extend added objects. These can be added via the IMultiUserReplication interface. */
		FReplicationDiscoveryContainer& RegisteredExtenders;
		
		void ExtendStreamWithRegisteredDiscoverers(UObject& ExtendedObject, ConcertSharedSlate::IStreamExtensionContext& Context) const;
	};
}

