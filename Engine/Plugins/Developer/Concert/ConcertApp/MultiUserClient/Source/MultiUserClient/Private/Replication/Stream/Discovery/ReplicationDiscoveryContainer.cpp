// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationDiscoveryContainer.h"

namespace UE::MultiUserClient::Replication
{
	void FReplicationDiscoveryContainer::AddDiscoverer(TSharedRef<IReplicationDiscoverer> Discoverer)
	{
		Discoverers.AddUnique(MoveTemp(Discoverer));
	}

	void FReplicationDiscoveryContainer::RemoveDiscoverer(const TSharedRef<IReplicationDiscoverer>& Discoverer)
	{
		Discoverers.RemoveSingle(Discoverer);
	}

	void FReplicationDiscoveryContainer::DiscoverReplicationSettings(const FReplicationDiscoveryParams& Params)
	{
		// Iterate in reverse order so we ignore any newly added ones
		for (int32 Index = Discoverers.Num() - 1; Discoverers.IsValidIndex(Index); --Index)
		{
			const TSharedRef<IReplicationDiscoverer>& Discoverer = Discoverers[Index];
			Discoverer->DiscoverReplicationSettings(Params);
			// if something was removed before i, we will visit them twice... which does not matter because adding properties is a no-op if they were already added.
		}
	}
}
