// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IReplicationDiscoverer.h"

#include "Containers/Array.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

class UObject;
struct FConcertPropertyChain;

namespace UE::MultiUserClient::Replication
{
	/**
	 * Just holds a bunch of IReplicationDiscoverer and calls forward the function calls to them.
	 * @see IReplicationDiscovery for the purpose of IReplicationDiscoverer.
	 */
	class FReplicationDiscoveryContainer : public IReplicationDiscoverer
	{
	public:

		void AddDiscoverer(TSharedRef<IReplicationDiscoverer> Discoverer);
		void RemoveDiscoverer(const TSharedRef<IReplicationDiscoverer>& Discoverer);
		
		//~ Begin IReplicationDiscovery Interface
		virtual void DiscoverReplicationSettings(const FReplicationDiscoveryParams& Params) override;
		//~ End IReplicationDiscovery Interface

	private:

		/** The discoverers to run through */
		TArray<TSharedRef<IReplicationDiscoverer>> Discoverers;
	};
}
