// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Delegates/Delegate.h"

struct FConcertStreamFrequencySettings;
struct FConcertObjectReplicationMap;

namespace UE::MultiUserClient::Replication
{
	/**
	 * Keeps track of a client's registered streams.
	 * Exposes ways for observing and mutating streams.
	 *
	 * The implementation varies depending on whether the client corresponds to the local or a remote editor.
	 */
	class IClientStreamSynchronizer
	{
	public:
		
		/** @return The managed client's stream ID. */
		virtual FGuid GetStreamId() const = 0;
		
		/** @return What local instance thinks the client's server state is. */
		virtual const FConcertObjectReplicationMap& GetServerState() const = 0;

		/** @return What the local instance thinks the client's replication frequencies are. */
		virtual const FConcertStreamFrequencySettings& GetFrequencySettings() const = 0;

		DECLARE_MULTICAST_DELEGATE(FOnServerStateChanged);
		/** @return Event executed when the result of GetServerState has been updated. */
		virtual FOnServerStateChanged& OnServerStreamChanged() = 0;
		
		virtual ~IClientStreamSynchronizer() = default;
	};

	/** Util base class for implementing the events */
	class FStreamSynchronizer_Base
		: public IClientStreamSynchronizer
	{
	public:

		//~ Begin IClientStreamSynchronizer Interface
		virtual FOnServerStateChanged& OnServerStreamChanged() override { return OnServerStateChangedDelegate; }
		//~ End IClientStreamSynchronizer Interface

	protected:

		/** Triggered by subclasses when the authority state changes. */
		FOnServerStateChanged OnServerStateChangedDelegate;
	};
}
