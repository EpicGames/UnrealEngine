// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

namespace UE::MultiUserClient::Replication
{
	/** The stream ID that is used by all Multi-User streams. */
	constexpr FGuid MultiUserStreamID { 0xAAAAAAAA, 0xBBBBBBBB, 0xCCCCCCCC, 0xDDDDDDDD };
}