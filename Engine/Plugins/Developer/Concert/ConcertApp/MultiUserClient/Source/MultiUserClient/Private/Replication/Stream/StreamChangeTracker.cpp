// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamChangeTracker.h"

#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/IConcertClientReplicationManager.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FLocalClientStreamDiffer"

namespace UE::MultiUserClient::Replication
{
	FStreamChangeTracker::FStreamChangeTracker(
		IClientStreamSynchronizer& InStreamSynchronizer,
		TAttribute<FConcertObjectReplicationMap*> InStreamWithInProgressChangesAttribute,
		FOnModifyReplicationMap InOnModifyReplicationMapDelegate
		)
		: StreamSynchronizer(InStreamSynchronizer)
		, StreamWithInProgressChangesAttribute(MoveTemp(InStreamWithInProgressChangesAttribute))
		, OnModifyReplicationMapDelegate(MoveTemp(InOnModifyReplicationMapDelegate))
	{
		check(InStreamWithInProgressChangesAttribute.IsBound() || InStreamWithInProgressChangesAttribute.IsSet());
		StreamSynchronizer.OnServerStreamChanged().AddRaw(this, &FStreamChangeTracker::RefreshChangesCache);
	}

	FStreamChangeTracker::~FStreamChangeTracker()
	{
		StreamSynchronizer.OnServerStreamChanged().RemoveAll(this);
	}

	void FStreamChangeTracker::RefreshChangesCache()
	{
		CachedDeltaChange = DiffChanges();
	}

	bool FStreamChangeTracker::HasChanges() const
	{
		return !CachedDeltaChange.ObjectsToPut.IsEmpty() || !CachedDeltaChange.ObjectsToRemove.IsEmpty();
	}

	bool FStreamChangeTracker::DoesObjectHavePropertiesAfterSubmit(const FSoftObjectPath& ObjectPath) const
	{
		const EObjectChangeType ChangeType = GetObjectChanges(ObjectPath);
		switch (ChangeType)
		{
		case EObjectChangeType::NoChange: return StreamSynchronizer.GetServerState().ReplicatedObjects.Contains(ObjectPath);
			
		case EObjectChangeType::PropertiesModified: return true;
		case EObjectChangeType::Added: return true;
			
		case EObjectChangeType::Removed: return false;
		default:
			checkNoEntry();
			return false;
		}
	}

	const FConcertPropertySelection* FStreamChangeTracker::GetPropertiesAfterSubmit(const FSoftObjectPath& ObjectPath) const
	{
		const FGuid StreamId = StreamSynchronizer.GetStreamId();
		const FConcertObjectInStreamID ObjectId { StreamId, ObjectPath };
		const FConcertReplication_ChangeStream_PutObject* PutObject = CachedDeltaChange.ObjectsToPut.Find(ObjectId);
		if (PutObject && !PutObject->Properties.ReplicatedProperties.IsEmpty())
		{
			return &PutObject->Properties;
		}

		const FConcertReplicatedObjectInfo* ObjectInfo = StreamSynchronizer.GetServerState().ReplicatedObjects.Find(ObjectPath);
		return ObjectInfo ? &ObjectInfo->PropertySelection : nullptr;
	}

	FStreamChangeTracker::EObjectChangeType FStreamChangeTracker::GetObjectChanges(const FSoftObjectPath& Object) const
	{
		const FGuid StreamId = StreamSynchronizer.GetStreamId();
		const FConcertObjectInStreamID ObjectId { StreamId, Object };
		if (CachedDeltaChange.ObjectsToRemove.Contains(ObjectId))
		{
			return EObjectChangeType::Removed;
		}

		const FConcertReplication_ChangeStream_PutObject* PutObject = CachedDeltaChange.ObjectsToPut.Find(ObjectId);
		if (!PutObject)
		{
			return EObjectChangeType::NoChange;
		}

		const bool bObjectIsOnServer = StreamSynchronizer.GetServerState().ReplicatedObjects.Contains(Object);
		return bObjectIsOnServer ? EObjectChangeType::PropertiesModified : EObjectChangeType::Added;
	}
	
	FStreamChangelist FStreamChangeTracker::DiffChanges(
		const FGuid& StreamId,
		const FConcertObjectReplicationMap& Base,
		const FConcertObjectReplicationMap& Changed
		)
	{
		// BuildRequestFromDiff does not tolerate any invalid entries (like empty properties, which the UI generates right after you add an object to the list)
		FConcertObjectReplicationMap Cleansed = Changed;
		ConcertSyncCore::Replication::ChangeStreamUtils::IterateInvalidEntries(Changed, [&Cleansed](const FSoftObjectPath& InvalidObject, const FConcertReplicatedObjectInfo&)
		{
			Cleansed.ReplicatedObjects.Remove(InvalidObject);
			return EBreakBehavior::Continue;
		});

		FConcertReplication_ChangeStream_Request Request = ConcertSyncCore::Replication::ChangeStreamUtils::BuildRequestFromDiff(StreamId, Base, Cleansed);
		return { MoveTemp(Request.ObjectsToRemove), MoveTemp(Request.ObjectsToPut) };
	}
}

#undef LOCTEXT_NAMESPACE