// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientStreamSynchronizer.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Messages/ChangeStream.h"
#include "Replication/Misc/Util/StreamRequestUtils.h"

#include "Async/Future.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Misc/Attribute.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;
struct FConcertReplication_ChangeStream_Response;
struct FConcertReplication_ChangeStream_Request;

namespace UE::MultiUserClient::Replication
{
	/**
	 * Knows of the local client's registered replication streams and builds a changelist. The changelist tracks the
	 * unconfirmed changes to the client's streams and is updated when the server confirms the change.
	 */
	class FStreamChangeTracker : public FNoncopyable
	{
	public:

		DECLARE_DELEGATE(FOnModifyReplicationMap);
		FStreamChangeTracker(
			IClientStreamSynchronizer& InStreamSynchronizer,
			TAttribute<FConcertObjectReplicationMap*> InStreamWithInProgressChangesAttribute,
			FOnModifyReplicationMap InOnModifyReplicationMapDelegate
			);
		~FStreamChangeTracker();

		/**
		 * Clears and rebuilds CachedDeltaChange in response to the underlying StreamWithInProgressChangesAttribute having been changed.
		 * 
		 * This iterates through all elements, which is suboptimal. However, this is needed e.g. after an undo & redo command.
		 * In the future, we could implement a more efficient version that just responds to the relevant events (like adding an object).
		 * While this function does redundant work, it should really not be a big performance bog after all. Also it is called
		 * infrequently e.g. in response to a user pressing a button so we should be fine.
		 */
		void RefreshChangesCache();
		const FStreamChangelist& GetCachedDeltaChange() const { return CachedDeltaChange; }

		/** @return Whether there are any changes that that can be submitted to the server (excludes those with warnings). */
		bool HasChanges() const;

		/** @return If changes are submitted, is ObjectPath in the stream? */
		bool DoesObjectHavePropertiesAfterSubmit(const FSoftObjectPath& ObjectPath) const;
		/** @return Gets the final state the object will have after submission. */
		const FConcertPropertySelection* GetPropertiesAfterSubmit(const FSoftObjectPath& ObjectPath) const;

		enum class EObjectChangeType
		{
			NoChange,
			Added,
			Removed,
			PropertiesModified
		};
		/** @return Whether this object's local configuration differs from the server version. */
		EObjectChangeType GetObjectChanges(const FSoftObjectPath& Object) const;
		
	private:

		/** Keeps track of the edited stream's server state. Outlives the FStreamChangeTracker instance. */
		IClientStreamSynchronizer& StreamSynchronizer;

		/**
		 * Represents ConfirmedServerState with changes made to it. These changes have not been sent to the server, yet.
		 * This is cleared when a change is sent to the server. 
		 */
		const TAttribute<FConcertObjectReplicationMap*> StreamWithInProgressChangesAttribute;
		
		/**
		 * The changes that if applied to ConfirmedServerState would result in StreamWithInProgressChangesAttribute.
		 * This is updated every time the local client makes changes.
		 */
		FStreamChangelist CachedDeltaChange;
		
		/** Called when StreamWithInProgressChangesAttribute is about to be modified. */
		FOnModifyReplicationMap OnModifyReplicationMapDelegate;

		FStreamChangelist DiffChanges() const
		{
			const FConcertObjectReplicationMap* Map = StreamWithInProgressChangesAttribute.Get();
			return Map ? DiffChanges(StreamSynchronizer.GetStreamId(), StreamSynchronizer.GetServerState(), *Map) : FStreamChangelist{};
		}
		/** Builds the changelist to get from Base to Changed. */
		static FStreamChangelist DiffChanges(const FGuid& StreamId, const FConcertObjectReplicationMap& Base, const FConcertObjectReplicationMap& Changed);
	};
}

