// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamSynchronizer_LocalClient.h"

#include "IConcertSyncClient.h"
#include "Replication/IConcertClientReplicationManager.h"

#define LOCTEXT_NAMESPACE "FLocalClientStreamDiffer"

namespace UE::MultiUserClient::Replication
{
	FStreamSynchronizer_LocalClient::FStreamSynchronizer_LocalClient(
		TSharedRef<IConcertSyncClient> InLocalClient,
		const FGuid& InLocalClientStreamId
		)
		: LocalClient(MoveTemp(InLocalClient))
		, LocalClientStreamId(InLocalClientStreamId)
	{
		IConcertClientReplicationManager* ReplicationManager = LocalClient->GetReplicationManager();
		if (ensure(ReplicationManager))
		{
			ReplicationManager->OnPostStreamsChanged().AddRaw(this, &FStreamSynchronizer_LocalClient::OnPostStreamsChanged);
		}
	}

	FStreamSynchronizer_LocalClient::~FStreamSynchronizer_LocalClient()
	{
		if (IConcertClientReplicationManager* ReplicationManager = LocalClient->GetReplicationManager())
		{
			ReplicationManager->OnPostStreamsChanged().RemoveAll(this);
		}
	}

	const FConcertObjectReplicationMap& FStreamSynchronizer_LocalClient::GetServerState() const
	{
		const FConcertBaseStreamInfo* Description = GetLocalMultiUserStream();
		return Description ? Description->ReplicationMap : EmptyState.ReplicationMap;
	}

	const FConcertStreamFrequencySettings& FStreamSynchronizer_LocalClient::GetFrequencySettings() const
	{
		const FConcertBaseStreamInfo* Description = GetLocalMultiUserStream();
		return Description ? Description->FrequencySettings : EmptyState.FrequencySettings;
	}

	void FStreamSynchronizer_LocalClient::OnPostStreamsChanged()
	{
		// Note that this fires even for non MU-related changes, e.g. due to some other system's Concert replication API usage.
		OnServerStateChangedDelegate.Broadcast();
	}

	const FConcertBaseStreamInfo* FStreamSynchronizer_LocalClient::GetLocalMultiUserStream() const
	{
		IConcertClientReplicationManager* ReplicationManager = LocalClient->GetReplicationManager();
		if (!ensure(ReplicationManager))
		{
			return nullptr;
		}
		
		const FConcertBaseStreamInfo* Result = nullptr;
		ReplicationManager->ForEachRegisteredStream([this, &Result](const FConcertReplicationStream& Stream)
		{
			if (Stream.BaseDescription.Identifier == LocalClientStreamId)
			{
				Result = &Stream.BaseDescription;
				return EBreakBehavior::Break;
			}
			return EBreakBehavior::Continue;
		});
		return Result;
	}
}

#undef LOCTEXT_NAMESPACE