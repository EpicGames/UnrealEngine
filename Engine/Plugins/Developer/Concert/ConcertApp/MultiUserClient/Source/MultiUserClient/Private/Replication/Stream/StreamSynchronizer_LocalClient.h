// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientStreamSynchronizer.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Data/ReplicationStream.h"
#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;

namespace UE::MultiUserClient::Replication
{
	/** Asks the local client's IConcertClientReplicationManager about the current state of the streams. */
	class FStreamSynchronizer_LocalClient : public IClientStreamSynchronizer, public FNoncopyable
	{
	public:

		FStreamSynchronizer_LocalClient(TSharedRef<IConcertSyncClient> InLocalClient, const FGuid& InLocalClientStreamId);
		virtual ~FStreamSynchronizer_LocalClient() override;

		//~ Begin IClientStreamSynchronizer Interface
		virtual FGuid GetStreamId() const override { return LocalClientStreamId; }
		virtual const FConcertObjectReplicationMap& GetServerState() const override;
		virtual const FConcertStreamFrequencySettings& GetFrequencySettings() const override;
		virtual FOnServerStateChanged& OnServerStreamChanged() override { return OnServerStateChangedDelegate; }
		//~ End IClientStreamSynchronizer Interface

	private:
		
		/** Returned by GetServerState when there is no registered stream. */
		const FConcertBaseStreamInfo EmptyState; 
		
		/** Owning client. Used to send change requests to the server. */
		const TSharedRef<IConcertSyncClient> LocalClient;
		/** The ID of the local client's stream this FLocalClientStreamDiffer is managing. */
		const FGuid LocalClientStreamId;
		
		/** Event executed when the result of GetServerState has been synched. */
		FOnServerStateChanged OnServerStateChangedDelegate;
		
		void OnPostStreamsChanged();
		const FConcertBaseStreamInfo* GetLocalMultiUserStream() const;
	};
}

