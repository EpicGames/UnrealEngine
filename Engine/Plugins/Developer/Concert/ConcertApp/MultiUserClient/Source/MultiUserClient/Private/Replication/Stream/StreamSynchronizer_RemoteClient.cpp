// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamSynchronizer_RemoteClient.h"

#include "Assets/MultiUserReplicationClientPreset.h"
#include "MultiUserStreamId.h"
#include "Replication/Misc/Query/StreamAndAuthorityQueryService.h"

namespace UE::MultiUserClient::Replication
{
	constexpr float QueryTimeInterval = 1.f;
	
	FStreamSynchronizer_RemoteClient::FStreamSynchronizer_RemoteClient(const FGuid& RemoteEndpointId, FStreamAndAuthorityQueryService& InQueryService)
		: QueryService(InQueryService)
		, QueryStreamHandle(
			QueryService.RegisterStreamQuery(
				RemoteEndpointId,
				FStreamQueryDelegate::CreateRaw(this, &FStreamSynchronizer_RemoteClient::HandleStreamQuery)
				)
			)
	{}

	FStreamSynchronizer_RemoteClient::~FStreamSynchronizer_RemoteClient()
	{
		QueryService.UnregisterStreamQuery(QueryStreamHandle);
	}

	FGuid FStreamSynchronizer_RemoteClient::GetStreamId() const
	{
		return MultiUserStreamID;
	}

	void FStreamSynchronizer_RemoteClient::HandleStreamQuery(const TArray<FConcertBaseStreamInfo>& Streams)
	{
		// MU clients use UMultiUserReplicationClientPreset::MultiUserStreamID for streams.
		// That handles the (unlikely) case in which some external logic had added streams to the same client which we must differentiate
		const FConcertBaseStreamInfo* MultiUserDescription = Streams.FindByPredicate([](const FConcertBaseStreamInfo& Description)
		{
			return Description.Identifier == MultiUserStreamID;
		});
		if (MultiUserDescription
			// Avoid unnecessary Broadcast()s
			&& LastKnownServerState != *MultiUserDescription)
		{
			LastKnownServerState = *MultiUserDescription;
			OnServerStateChangedDelegate.Broadcast();
		}
		// If the remote client removed the last object, the stream is implicitly deleted.
		else if (!MultiUserDescription && !LastKnownServerState.ReplicationMap.ReplicatedObjects.IsEmpty())
		{
			LastKnownServerState = {};
			OnServerStateChangedDelegate.Broadcast();
		}
	}
}
