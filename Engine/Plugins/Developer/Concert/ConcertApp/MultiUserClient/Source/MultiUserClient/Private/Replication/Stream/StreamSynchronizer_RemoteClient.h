// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientStreamSynchronizer.h"
#include "Replication/Data/ReplicationStream.h"

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

namespace UE::MultiUserClient::Replication
{
	class FStreamAndAuthorityQueryService;
	
	/** Tracks the state of a remote client by querying the client's state in regular intervals. */
	class FStreamSynchronizer_RemoteClient : public IClientStreamSynchronizer, public FNoncopyable
	{
	public:
		
		FStreamSynchronizer_RemoteClient(const FGuid& RemoteEndpointId, FStreamAndAuthorityQueryService& InQueryService UE_LIFETIMEBOUND);
		virtual ~FStreamSynchronizer_RemoteClient() override;

		//~ Begin IClientStreamSynchronizer Interface
		virtual FGuid GetStreamId() const override;
		virtual const FConcertObjectReplicationMap& GetServerState() const override { return LastKnownServerState.ReplicationMap; }
		virtual const FConcertStreamFrequencySettings& GetFrequencySettings() const override { return LastKnownServerState.FrequencySettings; }
		virtual FOnServerStateChanged& OnServerStreamChanged() override { return OnServerStateChangedDelegate; }
		//~ End IClientStreamSynchronizer Interface

	private:

		/** Queries the server in regular intervals. This services outlives our object. */
		FStreamAndAuthorityQueryService& QueryService;
		/** Used to unregister HandleStreamQuery upon destruction. */
		const FDelegateHandle QueryStreamHandle; 

		/** Represents what the local client thinks the replication map on the server currently looks like. */
		FConcertBaseStreamInfo LastKnownServerState;
		
		/** Event executed when the result of GetServerState has been synched. */
		FOnServerStateChanged OnServerStateChangedDelegate;

		/** Called in regular intervals with the contents of the remote client. */
		void HandleStreamQuery(const TArray<FConcertBaseStreamInfo>& Streams);
	};
}

