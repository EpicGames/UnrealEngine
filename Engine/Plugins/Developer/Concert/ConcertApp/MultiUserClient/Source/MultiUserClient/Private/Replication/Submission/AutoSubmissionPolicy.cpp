// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoSubmissionPolicy.h"

#include "ChangeRequestBuilder.h"
#include "ISubmissionWorkflow.h"
#include "Replication/Authority/AuthorityChangeTracker.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Replication/Misc/Frequency/FrequencyChangeTracker.h"
#include "Replication/Submission/Queue/SubmissionQueue.h"

namespace UE::MultiUserClient::Replication
{
	FAutoSubmissionPolicy::FAutoSubmissionPolicy(
		FSubmissionQueue& InSubmissionQueue,
		const FChangeRequestBuilder& InRequestBuilder,
		ConcertSharedSlate::IEditableReplicationStreamModel& InStreamEditorModel,
		FAuthorityChangeTracker& InAuthorityChangeTracker,
		FFrequencyChangeTracker& InFrequencyChangeTracker
		)
		: FSelfUnregisteringDeferredSubmitter(InSubmissionQueue)
		, SubmissionQueue(InSubmissionQueue)
		, RequestBuilder(InRequestBuilder)
		, StreamEditorModel(InStreamEditorModel)
		, AuthorityChangeTracker(InAuthorityChangeTracker)
		, FrequencyChangeTracker(InFrequencyChangeTracker)
	{
		StreamEditorModel.OnObjectsChanged().AddRaw(this, &FAutoSubmissionPolicy::OnObjectsChanged);
		StreamEditorModel.OnPropertiesChanged().AddRaw(this, &FAutoSubmissionPolicy::OnChangesDetected);
		AuthorityChangeTracker.OnChangedOwnedObjects().AddRaw(this, &FAutoSubmissionPolicy::OnChangesDetected);
		FrequencyChangeTracker.OnFrequencySettingsChanged().AddRaw(this, &FAutoSubmissionPolicy::OnChangesDetected);
	}

	FAutoSubmissionPolicy::~FAutoSubmissionPolicy()
	{
		StreamEditorModel.OnObjectsChanged().RemoveAll(this);
		StreamEditorModel.OnPropertiesChanged().RemoveAll(this);
		AuthorityChangeTracker.OnChangedOwnedObjects().RemoveAll(this);
		FrequencyChangeTracker.OnFrequencySettingsChanged().RemoveAll(this);
	}

	void FAutoSubmissionPolicy::ProcessAccumulatedChangesAndSubmit()
	{
		if (bIsDirty)
		{
			SubmissionQueue.SubmitNowOrEnqueue_GameThread(*this);
		}
	}

	void FAutoSubmissionPolicy::PerformSubmission_GameThread(ISubmissionWorkflow& Workflow)
	{
		using namespace UE::ConcertSyncClient::Replication;
		bIsDirty = false;
			
		// Even though authority request is sent after server confirms stream change, the authority request is pre-built to avoid sending changes
		// the local client makes while we're waiting for the latent server responses.
		TOptional<FConcertReplication_ChangeAuthority_Request> AuthorityChangeRequest = RequestBuilder.BuildAuthorityChange();
		TOptional<FConcertReplication_ChangeStream_Request> StreamRequest = RequestBuilder.BuildStreamChange();
			
		if (AuthorityChangeRequest || StreamRequest)
		{
			Workflow.SubmitChanges({ StreamRequest, AuthorityChangeRequest });
		}
	}

	void FAutoSubmissionPolicy::OnChangesDetected()
	{
		bIsDirty = true;
	}
}
