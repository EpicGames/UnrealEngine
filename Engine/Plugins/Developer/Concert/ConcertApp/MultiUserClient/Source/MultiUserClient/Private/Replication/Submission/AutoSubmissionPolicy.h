// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Replication/Submission/Queue/DeferredSubmitter.h"
#include "Templates/UnrealTemplate.h"

namespace UE::MultiUserClient::Replication
{
	class FFrequencyChangeTracker;
}

namespace UE::ConcertSharedSlate
{
	class IEditableReplicationStreamModel;
}

namespace UE::MultiUserClient::Replication
{
	class FChangeRequestBuilder;
	class IClientStreamSynchronizer;
	class ISubmissionWorkflow;
	class FAuthorityChangeTracker;
	class FStreamChangeTracker;
	
	/**
	 * Automatically submits stream and authority changes as the user makes them.
	 * If a change is in progress when the change is made, the changes are queued and sent once the pending submission is done.
	 *
	 * This policy accumulates all changes until the end of frame and then sends them. This means multiple changes are made in the same frame,
	 * only one request will be sent containing all changes.
	 */
	class FAutoSubmissionPolicy
		: public FNoncopyable
		, FSelfUnregisteringDeferredSubmitter
	{
	public:
		
		FAutoSubmissionPolicy(
			FSubmissionQueue& InSubmissionQueue UE_LIFETIMEBOUND,
			const FChangeRequestBuilder& InRequestBuilder UE_LIFETIMEBOUND,
			ConcertSharedSlate::IEditableReplicationStreamModel& InStreamEditorModel UE_LIFETIMEBOUND,
			FAuthorityChangeTracker& InAuthorityChangeTracker UE_LIFETIMEBOUND,
			FFrequencyChangeTracker& InFrequencyChangeTracker UE_LIFETIMEBOUND
			);
		~FAutoSubmissionPolicy();

		/** Checks whether any changes have been made since the last call and submits a change request if so. */
		void ProcessAccumulatedChangesAndSubmit();
		
	private:

		/** Handles performing the submission */
		FSubmissionQueue& SubmissionQueue;
		/** Used to building the requests that are passed to SubmissionWorkflow. */
		const FChangeRequestBuilder& RequestBuilder;

		/** Informs us when the stream is structurally changed by the user. */
		ConcertSharedSlate::IEditableReplicationStreamModel& StreamEditorModel;
		/** Informs us when authority is changed by the user. */
		FAuthorityChangeTracker& AuthorityChangeTracker;
		/** Informs us when frequency settings are changed by the user. */
		FFrequencyChangeTracker& FrequencyChangeTracker;

		/** Whether any changes were made. */
		bool bIsDirty = false;

		//~ Begin FSelfUnregisteringDeferredSubmitter Interface
		virtual void PerformSubmission_GameThread(ISubmissionWorkflow& Workflow) override;
		//~ End FSelfUnregisteringDeferredSubmitter Interface

		void OnObjectsChanged(TArrayView<UObject* const>, TArrayView<const FSoftObjectPath>, ConcertSharedSlate::EReplicatedObjectChangeReason) { OnChangesDetected(); }
		void OnChangesDetected();
	};
}

