// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChangeRequestBuilder.h"

#include "Replication/Authority/AuthorityChangeTracker.h"
#include "Replication/Stream/StreamChangeTracker.h"
#include "Replication/Misc/GlobalAuthorityCache.h"
#include "Replication/Misc/Util/StreamRequestUtils.h"

namespace UE::MultiUserClient::Replication
{
	FChangeRequestBuilder::FChangeRequestBuilder(
		const FGuid& InLocalClientId,
		const FGlobalAuthorityCache& InAuthorityCache,
		IClientStreamSynchronizer& InStreamSynchronizer,
		FStreamChangeTracker& InStreamChangeTracker,
		FAuthorityChangeTracker& InAuthorityChangeTracker,
		FFrequencyChangeTracker& InFrequencyChangeTracker
		)
		: LocalClientId(InLocalClientId)
		, AuthorityCache(InAuthorityCache)
		, StreamSynchronizer(InStreamSynchronizer)
		, StreamChangeTracker(InStreamChangeTracker)
		, AuthorityChangeTracker(InAuthorityChangeTracker)
		, FrequencyChangeTracker(InFrequencyChangeTracker)
	{}

	TOptional<FConcertReplication_ChangeStream_Request> FChangeRequestBuilder::BuildStreamChange() const
	{
		using namespace UE::ConcertSyncClient::Replication;
		TOptional<FConcertReplication_ChangeStream_Request> StreamRequest;
		
		const FStreamChangelist& Changelist = StreamChangeTracker.GetCachedDeltaChange();
		const bool bIsChangelistEmpty = Changelist.ObjectsToPut.IsEmpty() && Changelist.ObjectsToRemove.IsEmpty();
		if (!bIsChangelistEmpty)
		{
			const FFrequencyChangelist& FrequencyChanges = FrequencyChangeTracker.BuildForSubmission(Changelist);
			
			StreamRequest = StreamSynchronizer.GetServerState().ReplicatedObjects.IsEmpty()
				? StreamRequestUtils::BuildChangeRequest_CreateNewStream(GetLocalClientStreamId(), Changelist, FrequencyChanges)
				: StreamRequestUtils::BuildChangeRequest_UpdateExistingStream(GetLocalClientStreamId(), Changelist, FrequencyChanges);
				
			// Predict conflicts in case the remote client's streams have changed. Also: while the UI highlights "bad" requests, it does not correct it.
			AuthorityCache.CleanseConflictsFromStreamRequest(*StreamRequest, LocalClientId);
		}

		return StreamRequest;
	}

	TOptional<FConcertReplication_ChangeAuthority_Request> FChangeRequestBuilder::BuildAuthorityChange() const
	{
		return AuthorityChangeTracker.BuildChangeRequest(GetLocalClientStreamId());
	}

	FGuid FChangeRequestBuilder::GetLocalClientStreamId() const
	{
		return StreamSynchronizer.GetStreamId();
	}
}
