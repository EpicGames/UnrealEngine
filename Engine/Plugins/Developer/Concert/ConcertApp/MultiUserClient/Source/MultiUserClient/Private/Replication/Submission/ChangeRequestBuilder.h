// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Misc/Optional.h"

struct FConcertReplication_ChangeAuthority_Request;
struct FConcertReplication_ChangeStream_Request;

namespace UE::MultiUserClient::Replication
{
	class IClientStreamSynchronizer;
	class FAuthorityChangeTracker;
	class FFrequencyChangeTracker;
	class FGlobalAuthorityCache;
	class FStreamChangeTracker;
	
	/**
	 * Util that knows how to build FConcertReplication_ChangeStream_Request and FConcertReplication_ChangeAuthority_Request based on the local client's changes
	 * and knowledge of other remote clients.
	 */
	class FChangeRequestBuilder
	{
	public:
		
		/**
		 * @param InLocalClientId Endpoint ID of the client for which the request is being generated
		 * @param InAuthorityCache Used to predict conflicts. The caller ensures it outlives the constructed instance.
		 * @param InStreamSynchronizer Used to determine whether a new stream must be created. The caller ensures it outlives the constructed instance.
		 * @param InStreamChangeTracker Builds object part of the stream request. The caller ensures it outlives the constructed instance.
		 * @param InAuthorityChangeTracker Builds the authority changing request. The caller ensures it outlives the constructed instance.
		 * @param InFrequencyChangeTracker Builds the object replication frequency part of the request. The caller ensures it outlives the constructed instance.
		 */
		FChangeRequestBuilder(
			const FGuid& InLocalClientId,
			const FGlobalAuthorityCache& InAuthorityCache UE_LIFETIMEBOUND,
			IClientStreamSynchronizer& InStreamSynchronizer UE_LIFETIMEBOUND,
			FStreamChangeTracker& InStreamChangeTracker UE_LIFETIMEBOUND,
			FAuthorityChangeTracker& InAuthorityChangeTracker UE_LIFETIMEBOUND,
			FFrequencyChangeTracker& InFrequencyChangeTracker UE_LIFETIMEBOUND
			);

		/** @return Valid request that can be sent to the server, if there are any local changes. */
		TOptional<FConcertReplication_ChangeStream_Request> BuildStreamChange() const;

		/** @return Valid request that can be sent to the server, if there are any local changes. */
		TOptional<FConcertReplication_ChangeAuthority_Request> BuildAuthorityChange() const;

	private:

		/** Id of the local client in the session. */
		const FGuid LocalClientId;

		/** Used to filter out changes that would cause a conflict when submitted. */
		const FGlobalAuthorityCache& AuthorityCache;
		
		/** Used to determine whether a new stream needs to be created when submitting (happens if nothing was previously registered). */
		IClientStreamSynchronizer& StreamSynchronizer;
		
		/** Used to get changes made to the stream */
		FStreamChangeTracker& StreamChangeTracker;
		/** Informs us when authority is changed by the user. */
		FAuthorityChangeTracker& AuthorityChangeTracker;
		/** Used to build the object replication frequency requests. */
		FFrequencyChangeTracker& FrequencyChangeTracker;

		/** @return The ID MU uses for its single stream. */
		FGuid GetLocalClientStreamId() const;
	};
}

