// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Submission/Data/AuthoritySubmission.h"

#define LOCTEXT_NAMESPACE "EAuthoritySubmissionErrorCode"

namespace UE::MultiUserClient::Replication
{
	FText LexToText(EAuthoritySubmissionResponseErrorCode ErrorCode)
	{
		switch (ErrorCode)
		{
		case EAuthoritySubmissionResponseErrorCode::Success: return LOCTEXT("Success", "Success");
		case EAuthoritySubmissionResponseErrorCode::NoChange: return LOCTEXT("NoChange", "No Change");
		case EAuthoritySubmissionResponseErrorCode::Timeout: return LOCTEXT("Timeout", "Timeout");
		case EAuthoritySubmissionResponseErrorCode::CancelledDueToStreamUpdate: return LOCTEXT("CancelledDueToStreamUpdate", "Failed because stream dependency could not be updated");
		case EAuthoritySubmissionResponseErrorCode::Cancelled: return LOCTEXT("Cancelled", "Cancelled");
		default:
			checkNoEntry();
			return LOCTEXT("Unknown", "Unknown");;
		}
	}
}

#undef LOCTEXT_NAMESPACE