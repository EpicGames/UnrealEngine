// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Replication/IConcertClientReplicationManager.h"

namespace UE::MultiUserClient::Replication
{
	class ISubmissionOperation;
	
	enum class EAuthoritySubmissionRequestErrorCode : uint8
	{
		/** The operation completed as expected */
		Success,

		/** No change was sent to the server because there were no local changes. */
		NoChange, 

		/** Cancelled because the required stream update could not be made. */
		CancelledDueToStreamUpdate,
		
		/** The request was cancelled (e.g. because client disconnected while operation was in progress) */
		Cancelled
	};

	enum class EAuthoritySubmissionResponseErrorCode : uint8
    {
    	/** The operation completed as expected */
    	Success,

    	/** No change was sent to the server because there were no local changes. */
    	NoChange, 

    	/** The request timed out */
    	Timeout,

    	/** Cancelled because the required stream update could not be made. */
    	CancelledDueToStreamUpdate,
    	
    	/** The request was cancelled (e.g. because client disconnected while operation was in progress) */
    	Cancelled
    };

	FText LexToText(EAuthoritySubmissionResponseErrorCode ErrorCode);

	struct FSubmitAuthorityChangesRequest
	{
		EAuthoritySubmissionRequestErrorCode ErrorCode;
		
		/** Only valid if ErrorCode == EAuthoritySubmissionErrorCode::Success */
		TOptional<FConcertReplication_ChangeAuthority_Request> Request;
	};
	
	struct FSubmitAuthorityChangesResponse
	{
		EAuthoritySubmissionResponseErrorCode ErrorCode = EAuthoritySubmissionResponseErrorCode::Cancelled;
		
		/** Only valid if ErrorCode == EAuthoritySubmissionErrorCode::Success */
		TOptional<FConcertReplication_ChangeAuthority_Response> Response;
	};
}
