// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::MultiUserClient::Replication
{
	enum class EChangeUploadability
	{
		/** Can click the upload button */
		Ready,

		/** A previous upload operation is in progress */
		InProgress,

		/** Changing is generally not available, e.g. remote client does not allow changes. */
		NotAvailable
	};

	inline bool CanEverSubmit(EChangeUploadability Uploadability)
	{
		return Uploadability != EChangeUploadability::NotAvailable;
	}
}