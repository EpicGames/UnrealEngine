// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Replication/IConcertClientReplicationManager.h"

namespace UE::MultiUserClient::Replication
{
	class ISubmissionOperation;

	enum class EStreamSubmissionErrorCode : uint8
	{
		/** Changes were submitted to and processed by the server */
		Success,

		/** No change was sent to the server because there were no local changes. */
		NoChange,

		/** The request timed out */
		Timeout,

		/** The request was cancelled (e.g. because client disconnected while operation was in progress) */
		Cancelled
	};

	/** Contains the original request and response of a completed stream change. */
	struct FCompletedChangeSubmission
	{
		FConcertReplication_ChangeStream_Request Request;
		FConcertReplication_ChangeStream_Response Response;
	};

	/** Resulting of submitting stream changes */
	struct FSubmitStreamChangesResponse
	{
		/** Error code of the submission. Determines whether Response is valid. */
		EStreamSubmissionErrorCode ErrorCode = EStreamSubmissionErrorCode::Cancelled;

		/** Valid if ErrorCode == ESubmitChangesErrorCode::Success. */
		TOptional<FCompletedChangeSubmission> SubmissionInfo;
	};
}
