// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientChangeConversionUtils.h"

#include "Replication/ChangeOperationTypes.h"
#include "Replication/Submission/Data/AuthoritySubmission.h"
#include "Replication/Submission/Data/StreamSubmission.h"
#include "Replication/Misc/Util/StreamRequestUtils.h"

#include "Algo/RemoveIf.h"
#include "Misc/Guid.h"

namespace UE::MultiUserClient::Replication::ClientChangeConversionUtils
{
	namespace Private
	{
		static TOptional<FConcertReplication_ChangeStream_PutObject> GeneratePutRequest(
			const UObject& Object,
			FPropertyChange&& PropertyChange,
			const FConcertObjectReplicationMap& ClientStreamContent
			)
		{
			// TODO UE-201166: This step could be simplified if we had an append operation in FConcertReplication_ChangeStream_PutObject
			const FConcertReplicatedObjectInfo* ExistingObjectInfo = ClientStreamContent.ReplicatedObjects.Find(&Object);
			if (ExistingObjectInfo)
			{
				FConcertReplicatedObjectInfo NewObjectInfo = FConcertReplicatedObjectInfo::Make(Object);
				TSet<FConcertPropertyChain>& ReplicatedProperties = NewObjectInfo.PropertySelection.ReplicatedProperties;
				switch (PropertyChange.ChangeType)
				{
				case EPropertyChangeType::Put:
					ReplicatedProperties = MoveTemp(PropertyChange.Properties);
					break;
				case EPropertyChangeType::Add:
					ReplicatedProperties.Append(PropertyChange.Properties);
					break;
				case EPropertyChangeType::Remove:
					for (const FConcertPropertyChain& Property : PropertyChange.Properties)
					{
						ReplicatedProperties.Remove(Property);
					}
					break;
				default: checkNoEntry(); break;
				}
				return FConcertReplication_ChangeStream_PutObject::MakeFromChange(*ExistingObjectInfo, NewObjectInfo);
			}
			else
			{
				FConcertReplicatedObjectInfo NewObjectInfo = FConcertReplicatedObjectInfo::Make(Object);
				TSet<FConcertPropertyChain>& ReplicatedProperties = NewObjectInfo.PropertySelection.ReplicatedProperties;
				switch (PropertyChange.ChangeType)
				{
				case EPropertyChangeType::Put:
				case EPropertyChangeType::Add:
					ReplicatedProperties = MoveTemp(PropertyChange.Properties);
					return FConcertReplication_ChangeStream_PutObject::MakeFromInfo(NewObjectInfo);
				case EPropertyChangeType::Remove:
					return {};
				default: checkNoEntry(); break;
				}
			}
				
			return {};
		}

		static void BuildStreamChanges(
			TMap<UObject*, FPropertyChange>&& PropertyChanges,
			const FGuid& ClientStreamId,
			const FConcertObjectReplicationMap& ClientStreamContent,
			FStreamChangelist& StreamChangelist
			)
		{
			for (TPair<UObject*, FPropertyChange>& PropertyChange : PropertyChanges)
			{
				UObject* Object = PropertyChange.Key; 
				if (!Object)
				{
					continue;
				}
				
				TOptional<FConcertReplication_ChangeStream_PutObject> PutRequest = GeneratePutRequest(*Object, MoveTemp(PropertyChange.Value), ClientStreamContent);
				if (PutRequest)
				{
					// Gracefully handle APi caller (e.g. Blueprinter) forgetting to specify parent properties,
					// e.g. if they only specify ["Vector", "X"], we'll add ["Vector"] as well because it is needed for proper replication. 
					PutRequest->Properties.DiscoverAndAddImplicitParentProperties();
					
					StreamChangelist.ObjectsToPut.Add(
						FConcertObjectInStreamID { ClientStreamId, Object },
						MoveTemp(*PutRequest)
					);
				}
			}
		}

		static void BuildAuthorityChanges(TSet<FSoftObjectPath>&& ObjectsToRemove, const FGuid& ClientStreamId, FStreamChangelist& StreamChangelist)
		{
			for (FSoftObjectPath& RemovedObject : ObjectsToRemove)
			{
				StreamChangelist.ObjectsToRemove.Emplace(
					FConcertObjectInStreamID{ ClientStreamId, MoveTemp(RemovedObject) }
				);
			}
		}

		static FFrequencyChangelist BuildFrequencyChanges(FConcertReplication_ChangeStream_Frequency FrequencyChanges)
		{
			FFrequencyChangelist Changelist;
			Changelist.OverridesToAdd = MoveTemp(FrequencyChanges.OverridesToAdd);
			Changelist.OverridesToRemove = MoveTemp(FrequencyChanges.OverridesToRemove);
			Changelist.NewDefaults = EnumHasAnyFlags(FrequencyChanges.Flags, EConcertReplicationChangeFrequencyFlags::SetDefaults)
				? MoveTemp(FrequencyChanges.NewDefaults) : TOptional<FConcertObjectReplicationSettings>{};
			return Changelist;
		}
	}

	TOptional<FConcertReplication_ChangeStream_Request> Transform(
		FChangeStreamRequest Request,
		const FGuid& ClientStreamId,
		const FConcertObjectReplicationMap& ClientStreamContent
		)
	{
		FStreamChangelist StreamChangelist;
		Private::BuildStreamChanges(MoveTemp(Request.PropertyChanges), ClientStreamId, ClientStreamContent, StreamChangelist);
		Private::BuildAuthorityChanges(MoveTemp(Request.ObjectsToRemove), ClientStreamId, StreamChangelist);
		FFrequencyChangelist FrequencyChangelist = Private::BuildFrequencyChanges(MoveTemp(Request.FrequencyChanges));
		if (StreamChangelist.ObjectsToPut.IsEmpty() && StreamChangelist.ObjectsToRemove.IsEmpty() && FrequencyChangelist.IsEmpty())
		{
			return {};
		}
		
		const bool bNeedsToRegisterStream = ClientStreamContent.IsEmpty();
		return bNeedsToRegisterStream
			? StreamRequestUtils::BuildChangeRequest_CreateNewStream(ClientStreamId, StreamChangelist, MoveTemp(FrequencyChangelist))
			: StreamRequestUtils::BuildChangeRequest_UpdateExistingStream(ClientStreamId, MoveTemp(StreamChangelist), MoveTemp(FrequencyChangelist));
	}

	TOptional<FConcertReplication_ChangeAuthority_Request> Transform(
		FChangeAuthorityRequest Request,
		const FGuid& ClientStreamId
		)
	{
		FConcertReplication_ChangeAuthority_Request Result;
		for (FSoftObjectPath& TakeAuthority : Request.ObjectsToStartReplicating)
		{
			Result.TakeAuthority.Emplace(MoveTemp(TakeAuthority))
				.StreamIds = { { ClientStreamId } };
		}
		for (FSoftObjectPath& ReleaseAuthority : Request.ObjectToStopReplicating)
		{
			Result.ReleaseAuthority.Emplace(MoveTemp(ReleaseAuthority))
				.StreamIds = { { ClientStreamId } };
		}
		return Result;
	}
	
	EChangeStreamOperationResult ExtractErrorCode(const FSubmitStreamChangesResponse& Response)
	{
		switch (Response.ErrorCode)
		{
		case EStreamSubmissionErrorCode::Success:
			// EStreamSubmissionErrorCode::Success the op was processed but the EChangeStreamOperationResult distinguishes between successful or rejected ops.
				return !ensureMsgf(Response.SubmissionInfo.IsSet(), TEXT("Success implies set response. Investigate.")) || Response.SubmissionInfo->Response.IsFailure()
					? EChangeStreamOperationResult::Rejected
					: EChangeStreamOperationResult::Success;
		case EStreamSubmissionErrorCode::NoChange: return EChangeStreamOperationResult::NoChanges; 
		case EStreamSubmissionErrorCode::Timeout: return EChangeStreamOperationResult::Timeout; 
		case EStreamSubmissionErrorCode::Cancelled: return EChangeStreamOperationResult::Cancelled;
		default:
			checkNoEntry();
			return EChangeStreamOperationResult::Cancelled;
		}
	}
	
	EChangeAuthorityOperationResult ExtractErrorCode(const FSubmitAuthorityChangesResponse& Response)
	{
		switch (Response.ErrorCode)
		{
		case EAuthoritySubmissionResponseErrorCode::Success:
			// EAuthoritySubmissionResponseErrorCode::Success the op was processed but the EChangeStreamOperationResult distinguishes between successful or rejected ops.
				return !ensureMsgf(Response.Response.IsSet(), TEXT("Success implies set response. Investigate.")) || !Response.Response->RejectedObjects.IsEmpty()
					? EChangeAuthorityOperationResult::RejectedFullyOrPartially
					: EChangeAuthorityOperationResult::Success;
		case EAuthoritySubmissionResponseErrorCode::NoChange: return EChangeAuthorityOperationResult::NoChanges; 
		case EAuthoritySubmissionResponseErrorCode::Timeout: return EChangeAuthorityOperationResult::Timeout; 
		case EAuthoritySubmissionResponseErrorCode::CancelledDueToStreamUpdate: return EChangeAuthorityOperationResult::CancelledDueToStreamUpdate; 
		case EAuthoritySubmissionResponseErrorCode::Cancelled: return EChangeAuthorityOperationResult::Cancelled;
		default:
			checkNoEntry();
			return EChangeAuthorityOperationResult::Cancelled;
		}
	}

	FChangeClientStreamResponse Transform(const FSubmitStreamChangesResponse& Response)
	{
		FChangeClientStreamResponse Result { ExtractErrorCode(Response) };
		
		if (Response.SubmissionInfo)
		{
			FConcertReplication_ChangeStream_Response ContainedResponse = Response.SubmissionInfo->Response;
			
			for (const TPair<FConcertObjectInStreamID, FConcertReplicatedObjectId>& Pair : ContainedResponse.AuthorityConflicts)
			{
				Result.AuthorityConflicts.Add(Pair.Key.Object, Pair.Value.SenderEndpointId);
			}

			for (const TPair<FConcertObjectInStreamID, EConcertPutObjectErrorCode>& Pair : ContainedResponse.ObjectsToPutSemanticErrors)
			{
				Result.SemanticErrors.Add(Pair.Key.Object, Transform(Pair.Value));
			}

			for (const TPair<FConcertObjectInStreamID, EConcertChangeObjectFrequencyErrorCode>& Pair : ContainedResponse.FrequencyErrors.OverrideFailures)
			{
				Result.FrequencyErrors.ObjectErrors.Add(Pair.Key.Object, Transform(Pair.Value));
			}
			if (!ContainedResponse.FrequencyErrors.DefaultFailures.IsEmpty())
			{
				ensure(ContainedResponse.FrequencyErrors.DefaultFailures.Num() == 1);
				Result.FrequencyErrors.DefaultChangeErrorCode = Transform(ContainedResponse.FrequencyErrors.DefaultFailures.CreateIterator().Value());
			}

			Result.bFailedStreamCreation = !ContainedResponse.FailedStreamCreation.IsEmpty();
		}
		
		return Result;
	}
	
	FChangeClientAuthorityResponse Transform(const FSubmitAuthorityChangesResponse& Response)
	{
		FChangeClientAuthorityResponse Result{ ExtractErrorCode(Response) };

		if (Response.Response)
		{
			for (const TPair<FSoftObjectPath, FConcertStreamArray>& Pair : Response.Response->RejectedObjects)
			{
				Result.RejectedObjects.Add(Pair.Key);
			}
		}
		
		return Result;
	}

	EPutObjectErrorCode Transform(EConcertPutObjectErrorCode ErrorCode)
	{
		static_assert(static_cast<int32>(EPutObjectErrorCode::MissingData) == static_cast<int32>(EConcertPutObjectErrorCode::MissingData), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EPutObjectErrorCode::UnresolvedStream) == static_cast<int32>(EConcertPutObjectErrorCode::UnresolvedStream), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EPutObjectErrorCode::Count) == static_cast<int32>(EConcertPutObjectErrorCode::Count), "Update this code when making modification to the enums");
		return static_cast<EPutObjectErrorCode>(ErrorCode);
	}
	
	EChangeObjectFrequencyErrorCode Transform(EConcertChangeObjectFrequencyErrorCode ErrorCode)
	{
		static_assert(static_cast<int32>(EChangeObjectFrequencyErrorCode::UnregisteredStream) == static_cast<int32>(EConcertChangeObjectFrequencyErrorCode::NotRegistered), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EChangeObjectFrequencyErrorCode::InvalidReplicationRate) == static_cast<int32>(EConcertChangeObjectFrequencyErrorCode::InvalidReplicationRate), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EChangeObjectFrequencyErrorCode::Count) == static_cast<int32>(EConcertChangeObjectFrequencyErrorCode::Count), "Update this code when making modification to the enums");
		return static_cast<EChangeObjectFrequencyErrorCode>(ErrorCode);
	}

	EChangeObjectFrequencyErrorCode Transform(EConcertChangeStreamFrequencyErrorCode ErrorCode)
	{
		static_assert(static_cast<int32>(EChangeObjectFrequencyErrorCode::UnregisteredStream) == static_cast<int32>(EConcertChangeStreamFrequencyErrorCode::UnknownStream), "Update this code when making modification to the enums");
        static_assert(static_cast<int32>(EChangeObjectFrequencyErrorCode::InvalidReplicationRate) == static_cast<int32>(EConcertChangeStreamFrequencyErrorCode::InvalidReplicationRate), "Update this code when making modification to the enums");
        static_assert(static_cast<int32>(EChangeObjectFrequencyErrorCode::Count) == static_cast<int32>(EConcertChangeStreamFrequencyErrorCode::Count), "Update this code when making modification to the enums");
        return static_cast<EChangeObjectFrequencyErrorCode>(ErrorCode);
	}
}
