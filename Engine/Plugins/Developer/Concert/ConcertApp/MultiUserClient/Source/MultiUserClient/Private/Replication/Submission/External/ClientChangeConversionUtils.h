// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/ChangeOperationTypes.h"
#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Messages/ChangeStream.h"

#include "HAL/Platform.h"
#include "Misc/Optional.h"

struct FConcertObjectReplicationMap;
struct FGuid;

namespace UE::MultiUserClient
{
	struct FChangeClientAuthorityResponse;
	struct FChangeClientStreamResponse;
}

namespace UE::MultiUserClient::Replication
{
	enum class EAuthoritySubmissionResponseErrorCode : uint8;
	
	struct FSubmitAuthorityChangesResponse;
	struct FSubmitStreamChangesResponse;
}

/**
 * Utils for transforming the simplified params in ChangeOperationTypes.h to the ones used by ConcertSyncCore.
 */
namespace UE::MultiUserClient::Replication::ClientChangeConversionUtils
{
	/** Transforms FChangeStreamRequest to FConcertReplication_ChangeStream_Request, if the request is correctly formatted. */
	TOptional<FConcertReplication_ChangeStream_Request> Transform(
		FChangeStreamRequest Request,
		const FGuid& ClientStreamId,
		const FConcertObjectReplicationMap& ClientStreamContent
		);
	
	/** Transforms FChangeAuthorityRequest to FConcertReplication_ChangeStream_Request, if the request is correctly formatted. */
	TOptional<FConcertReplication_ChangeAuthority_Request> Transform(
		FChangeAuthorityRequest Request,
		const FGuid& ClientStreamId
		);

	EChangeStreamOperationResult ExtractErrorCode(const FSubmitStreamChangesResponse& Response);
	EChangeAuthorityOperationResult ExtractErrorCode(const FSubmitAuthorityChangesResponse& Response);
	
	FChangeClientStreamResponse Transform(const FSubmitStreamChangesResponse& Response);
	FChangeClientAuthorityResponse Transform(const FSubmitAuthorityChangesResponse& Response);

	EChangeObjectFrequencyErrorCode Transform(EConcertChangeObjectFrequencyErrorCode ErrorCode);
	EChangeObjectFrequencyErrorCode Transform(EConcertChangeStreamFrequencyErrorCode ErrorCode);
	EPutObjectErrorCode Transform(EConcertPutObjectErrorCode ErrorCode);

	inline TOptional<FConcertReplication_ChangeStream_Request> Transform(
		TOptional<FChangeStreamRequest> Request,
		const FGuid& ClientStreamId,
		const FConcertObjectReplicationMap& ClientStreamContent
		)
	{
		return Request
			? Transform(MoveTemp(*Request), ClientStreamId, ClientStreamContent)
			: TOptional<FConcertReplication_ChangeStream_Request>{};
	}
	
	inline TOptional<FConcertReplication_ChangeAuthority_Request> Transform(
		TOptional<FChangeAuthorityRequest> Request,
		const FGuid& ClientStreamId
		)
	{
		return Request
			? Transform(MoveTemp(*Request), ClientStreamId)
			: TOptional<FConcertReplication_ChangeAuthority_Request>{};
	}
}

