// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientChangeOperation.h"

#include "ClientChangeConversionUtils.h"
#include "ConcertLogGlobal.h"
#include "Replication/Submission/ISubmissionWorkflow.h"
#include "Replication/Submission/Queue/SubmissionQueue.h"
#include "Replication/Submission/ISubmissionOperation.h"

namespace UE::MultiUserClient::Replication
{
	TSharedRef<FClientChangeOperation> FClientChangeOperation::StartOperation(
		const FGuid& InStreamId,
		FGetStreamContent InGetStreamContentDelegate,
		FSubmissionQueue& InSubmissionQueue,
		FOnOperationCompleted InOnOperationCompletedFunc,
		TAttribute<FChangeClientReplicationRequest> InMakeSubmissionParamsAttribute
		)
	{
		TSharedRef<FClientChangeOperation> Operation = MakeShared<FClientChangeOperation>(
			InStreamId,
			MoveTemp(InGetStreamContentDelegate),
			MoveTemp(InOnOperationCompletedFunc),
			MoveTemp(InMakeSubmissionParamsAttribute)
			);
		
		// Must fully construct the TSharedRef first because FClientChangeOperation::SetUpOperation requires it for keeping a weak ptr
		InSubmissionQueue.SubmitNowOrEnqueue_GameThread(*Operation);
		
		return Operation;
	}
	
	FClientChangeOperation::FClientChangeOperation(
		const FGuid& InStreamId,
		FGetStreamContent InGetStreamContentDelegate,
		FOnOperationCompleted OnOperationCompletedFunc,
		TAttribute<FChangeClientReplicationRequest> InMakeSubmissionParamsAttribute
		)
		: StreamId(InStreamId)
		, GetStreamContentDelegate(MoveTemp(InGetStreamContentDelegate))
		, OnOperationCompletedFunc(MoveTemp(OnOperationCompletedFunc))
		, MakeSubmissionParamsAttribute(MoveTemp(InMakeSubmissionParamsAttribute))
	{}

	FClientChangeOperation::~FClientChangeOperation()
	{
		const bool bIsAlreadyCompleted = StreamResult && AuthorityResult;
		if (!StreamResult)
		{
			EmplaceStreamResult(FChangeClientStreamResponse{ EChangeStreamOperationResult::Cancelled });
		}
		if (!AuthorityResult)
		{
			// Also executes OnOperationCompleted
			EmplaceAuthorityResult(FChangeClientAuthorityResponse{ EChangeAuthorityOperationResult::Cancelled });
		}
		if (!bIsAlreadyCompleted)
		{
			OperationCompletedPromise.EmplaceValue(FChangeClientReplicationResult{ *StreamResult, *AuthorityResult });
		}
	}

	void FClientChangeOperation::PerformSubmission_GameThread(ISubmissionWorkflow& Workflow)
	{
		check(!StreamResult && !AuthorityResult);

		FChangeClientReplicationRequest Request = MakeSubmissionParamsAttribute.Get();
		if (!Request.StreamChangeRequest && !Request.AuthorityChangeRequest)
		{
			EmplaceStreamResult(FChangeClientStreamResponse{ EChangeStreamOperationResult::NoChanges });
			EmplaceAuthorityResult(FChangeClientAuthorityResponse{ EChangeAuthorityOperationResult::NoChanges });
			CompleteOperation();
		}
		else
		{
			const TSharedPtr<ISubmissionOperation> SubmissionOperation = Workflow.SubmitChanges({
				ClientChangeConversionUtils::Transform(MoveTemp(Request.StreamChangeRequest), StreamId, *GetStreamContentDelegate.Execute()),
				ClientChangeConversionUtils::Transform(MoveTemp(Request.AuthorityChangeRequest), StreamId)
			});
			SetUpOperation(SubmissionOperation);
		}
	}

	void FClientChangeOperation::SetUpOperation(const TSharedPtr<ISubmissionOperation>& SubmissionOperation)
	{
		if (!SubmissionOperation)
		{
			UE_LOGF(LogConcert, Error, "Failed to submit request to server. Check log for additional errors.");
			EmplaceStreamResult(FChangeClientStreamResponse{ EChangeStreamOperationResult::FailedToSendRequest });
			EmplaceAuthorityResult(FChangeClientAuthorityResponse{ EChangeAuthorityOperationResult::FailedToSendRequest });
			CompleteOperation();
			return;
		}

		// When submission workflow is destroyed, it will finish the below futures.
		// Keep weak reference to us in case the owning FExternalClientChangeRequestHandler is destroyed before the submission workflow.
		// This way our code does not rely on whether FExternalClientChangeRequestHandler or ISubmissionWorkflow is destroyed first.
		SubmissionOperation->OnCompleteStreamChangesFuture_AnyThread()
			.Next([WeakThis = SharedThis(this).ToWeakPtr()](FSubmitStreamChangesResponse&& Response)
			{
				if (const TSharedPtr<FClientChangeOperation> ThisPin = WeakThis.Pin())
				{
					ThisPin->EmplaceStreamResult(FChangeClientStreamResponse{ ClientChangeConversionUtils::Transform(Response) });
				}
			});
		SubmissionOperation->OnCompleteAuthorityChangeFuture_AnyThread()
			.Next([WeakThis = SharedThis(this).ToWeakPtr()](FSubmitAuthorityChangesResponse&& Response)
			{
				if (const TSharedPtr<FClientChangeOperation> ThisPin = WeakThis.Pin())
				{
					ThisPin->EmplaceAuthorityResult(FChangeClientAuthorityResponse{ ClientChangeConversionUtils::Transform(Response) });
				}
			});
		SubmissionOperation->OnCompletedOperation_AnyThread()
			.Next([WeakThis = SharedThis(this).ToWeakPtr()](ESubmissionOperationCompletedCode Response)
			{
				if (const TSharedPtr<FClientChangeOperation> ThisPin = WeakThis.Pin())
				{
					check(ThisPin->StreamResult && ThisPin->AuthorityResult);
					ThisPin->CompleteOperation();
				}
			});
	}
}
