// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExternalClientChangeDelegates.h"
#include "Replication/ChangeOperationTypes.h"
#include "Replication/IClientChangeOperation.h"
#include "Replication/Submission/Queue/DeferredSubmitter.h"

#include "Misc/Attribute.h"
#include "Misc/Optional.h"

namespace UE::MultiUserClient::Replication
{
	class ISubmissionOperation;
	
	/**
	 * Abstracts an operation invoked externally, i.e. IMultiUserReplication::EnqueueChanges part of our public API.
	 *
	 * It stores all the operation data that was passed to EnqueueChanges and handles enqueuing itself to the replication client's FSubmissionQueue.
	 * FClientChangeOperation is created by FExternalClientChangeRequestHandler, which ensures it is kept alive for at least the lifetime of the
	 * client the request is made for.
	 * Since this operation is exposed to the public API caller, it can outlive the client it was created for.
	 */
	class FClientChangeOperation
		: public IClientChangeOperation
		, public IDeferredSubmitter
		, public TSharedFromThis<FClientChangeOperation>
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
	public:

		using FOnOperationCompleted = TUniqueFunction<void()>;

		/**
		 * @param InStreamId The stream ID the client uses.
		 * @param InGetStreamContentDelegate Delegate for getting the content of the client's stream
		 * @param InSubmissionQueue The client's queue, which is used to enqueue this operation. The caller, which is FExternalClientChangeRequestHandler,
		 * keeps the constructed operation alive until either it is executed by the queue or the queue is destroyed. The caller handles dequeuing this
		 * operation if the client is destroyed.
		 * @param InOnOperationCompletedFunc Callback function which let's FExternalClientChangeRequestHandler know when the operation is done. Executing
		 * this function causes FExternalClientChangeRequestHandler to stop reference this operation so it may be destroyed immediately when invoked!
		 * @param InMakeSubmissionParamsAttribute Callback to build submission params when task is ready to be executed.
		 */
		static TSharedRef<FClientChangeOperation> StartOperation(
			const FGuid& InStreamId,
			FGetStreamContent InGetStreamContentDelegate,
			FSubmissionQueue& InSubmissionQueue,
			FOnOperationCompleted InOnOperationCompletedFunc,
			TAttribute<FChangeClientReplicationRequest> InMakeSubmissionParamsAttribute
			);
		virtual ~FClientChangeOperation() override;

		//~ Begin IClientChangeOperation Interface
		virtual TFuture<FChangeClientStreamResponse> OnChangeStream() override { return ChangeStreamPromise.GetFuture(); }
		virtual TFuture<FChangeClientAuthorityResponse> OnChangeAuthority() override { return ChangeAuthorityPromise.GetFuture(); }
		virtual TFuture<FChangeClientReplicationResult> OnOperationCompleted() override { return OperationCompletedPromise.GetFuture(); }
		//~ End IClientChangeOperation Interface

		//~ Begin IDeferredSubmitter Interface
		virtual void PerformSubmission_GameThread(ISubmissionWorkflow& Workflow) override;
		//~ End IDeferredSubmitter Interface

	private:

		/** The stream ID the client uses. */
		const FGuid StreamId;
		/** Delegate for getting the content of the client's stream */
		const FGetStreamContent GetStreamContentDelegate;
		
		/**
		 * Invokes when this operation completes.
		 * Causes this instance to no longer be referenced by FExternalClientChangeRequestHandler, which means calling this may invoke the destructor.
		 * @important Invoking this function may cause this instance to be destroyed immediately.
		 */
		const FOnOperationCompleted OnOperationCompletedFunc;
		/** Callback to build submission params when task is ready to be executed. */
		const TAttribute<FChangeClientReplicationRequest> MakeSubmissionParamsAttribute;

		TPromise<FChangeClientStreamResponse> ChangeStreamPromise;
		TPromise<FChangeClientAuthorityResponse> ChangeAuthorityPromise;
		TPromise<FChangeClientReplicationResult> OperationCompletedPromise;

		/** Set when ChangeStreamPromise is fulfilled. */
		TOptional<FChangeClientStreamResponse> StreamResult;
		/** Set when ChangeAuthorityPromise is fulfilled. */
		TOptional<FChangeClientAuthorityResponse> AuthorityResult;

		FClientChangeOperation(
			const FGuid& InStreamId,
			FGetStreamContent InGetStreamContentDelegate,
			FOnOperationCompleted InOnOperationCompletedFunc,
			TAttribute<FChangeClientReplicationRequest> InMakeSubmissionParamsAttribute
			);

		void EmplaceStreamResult(FChangeClientStreamResponse Result)
		{
			if (ensure(!StreamResult))
			{
				StreamResult = MoveTemp(Result);
				ChangeStreamPromise.EmplaceValue(*StreamResult);
			}
		}
		void EmplaceAuthorityResult(FChangeClientAuthorityResponse Result)
		{
			if (ensure(!AuthorityResult))
			{
				AuthorityResult = MoveTemp(Result);
				ChangeAuthorityPromise.EmplaceValue(*AuthorityResult);
			}
		}
		void CompleteOperation()
		{
			OperationCompletedPromise.EmplaceValue(FChangeClientReplicationResult{ *StreamResult, *AuthorityResult });
			OnOperationCompletedFunc();
		}

		/** Handles an operation received from ISubmissionWorkflow. */
		void SetUpOperation(const TSharedPtr<ISubmissionOperation>& SubmissionOperation);
	};
}

