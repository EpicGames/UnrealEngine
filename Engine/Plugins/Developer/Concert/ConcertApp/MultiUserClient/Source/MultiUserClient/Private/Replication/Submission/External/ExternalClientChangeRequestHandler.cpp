// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalClientChangeRequestHandler.h"

#include "ClientChangeOperation.h"
#include "Replication/Submission/Queue/SubmissionQueue.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/Future.h"

namespace UE::MultiUserClient::Replication
{
	TSharedRef<IClientChangeOperation> FExternalClientChangeRequestHandler::MakeFailedOperation(
		EChangeStreamOperationResult StreamResult,
		EChangeAuthorityOperationResult AuthorityResult
		)
	{
		class FNotInSessionOperation : public IClientChangeOperation
		{
			TPromise<FChangeClientStreamResponse> ChangeStreamPromise;
			TPromise<FChangeClientAuthorityResponse> ChangeAuthorityPromise;
			TPromise<FChangeClientReplicationResult> OperationCompletedPromise;
			
		public:

			FNotInSessionOperation(EChangeStreamOperationResult StreamErrorCode, EChangeAuthorityOperationResult AuthorityErrorCode)
				: ChangeStreamPromise(MakeFulfilledPromise<FChangeClientStreamResponse>(FChangeClientStreamResponse{ StreamErrorCode }))
				, ChangeAuthorityPromise(MakeFulfilledPromise<FChangeClientAuthorityResponse>(FChangeClientAuthorityResponse{ AuthorityErrorCode }))
				, OperationCompletedPromise(MakeFulfilledPromise<FChangeClientReplicationResult>(FChangeClientReplicationResult{ { StreamErrorCode }, { AuthorityErrorCode } }))
			{}
			
			//~ Begin IClientChangeOperation Interface
			virtual TFuture<FChangeClientStreamResponse> OnChangeStream() override { return ChangeStreamPromise.GetFuture(); }
			virtual TFuture<FChangeClientAuthorityResponse> OnChangeAuthority() override { return ChangeAuthorityPromise.GetFuture(); }
			virtual TFuture<FChangeClientReplicationResult> OnOperationCompleted() override { return OperationCompletedPromise.GetFuture(); }
			//~ End IClientChangeOperation Interface
		};

		return MakeShared<FNotInSessionOperation>(StreamResult, AuthorityResult);
	}

	FExternalClientChangeRequestHandler::FExternalClientChangeRequestHandler(
		const FGuid& InStreamId, 
		FGetStreamContent InGetStreamContentDelegate,
		FSubmissionQueue& InSubmissionQueue
		)
		: StreamId(InStreamId)
		, GetStreamContentDelegate(MoveTemp(InGetStreamContentDelegate))
		, SubmissionQueue(InSubmissionQueue)
	{}

	FExternalClientChangeRequestHandler::~FExternalClientChangeRequestHandler()
	{
		// We keeps the all operations alive until either they are executed by the queue or the queue is destroyed.
		// There is no technical reason to dequeue since the queue will not reference us when it is destroyed.
		// However 1. we'll follow RAII and 2. FSubmissionQueue::SubmitNowOrEnqueue_GameThread's contract dictates we call Dequeue_GameThread when destroyed.
		// So we'll dequeue it anyway.
		for (const TPair<FOperationId, TSharedRef<FClientChangeOperation>>& Op : PendingOperations)
		{
			// Dequeue_GameThread will ensure already if we're not in game thread so no need to double-check again.
			SubmissionQueue.Dequeue_GameThread(*Op.Value);
		}
	}

	TSharedRef<IClientChangeOperation> FExternalClientChangeRequestHandler::HandleRequest(TAttribute<FChangeClientReplicationRequest> SubmissionParams)
	{
		check(IsInGameThread());
		TSharedRef<FClientChangeOperation> Result = FClientChangeOperation::StartOperation(
			StreamId,
			GetStreamContentDelegate,
			SubmissionQueue,
			[this, OperationId = NextOperationId, WeakToken = LifetimeToken.ToWeakPtr()]()
			{
				OnOperationCompleted(OperationId, WeakToken, this);
			},
			MoveTemp(SubmissionParams)
		);
		PendingOperations.Add(NextOperationId, Result);
		++NextOperationId;
		return Result;
	}

	void FExternalClientChangeRequestHandler::OnOperationCompleted(
		FOperationId OperationId,
		TWeakPtr<FToken> WeakToken,
		FExternalClientChangeRequestHandler* GuardedThis)
	{
		const TSharedPtr<FToken> TokenPin = WeakToken.Pin();
		if (!TokenPin)
		{
			// FExternalClientChangeRequestHandler was destroyed.
			// Happens when external module keeps  operation alive through strong reference and the operation is cancelled as part of ISubmissionWorkflow being destroyed.
			return;
		}

		auto Unregister = [OperationId, GuardedThis]()
		{
			const int32 NumRemoved = GuardedThis->PendingOperations.Remove(OperationId);
			const bool bFailedInstantly = OperationId == GuardedThis->NextOperationId;
			check(NumRemoved > 0 || bFailedInstantly);
		};
		
		if (IsInGameThread())
		{
			Unregister();
		}
		else
		{
			// PendingOperations may only be modified on the game thread to avoid race conditions.
			AsyncTask(ENamedThreads::GameThread, [WeakToken, Unregister]()
			{
				// Latent callback. Must check for destruction of FExternalClientChangeRequestHandler again.
				if (const TSharedPtr<FToken> TokenPin = WeakToken.Pin())
				{
					Unregister();
				}
			});
		}
	}
}
