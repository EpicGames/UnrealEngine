// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExternalClientChangeDelegates.h"
#include "Replication/IToken.h"

#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Templates/UnrealTemplate.h"

namespace UE::MultiUserClient
{
	class IClientChangeOperation;
	struct FChangeClientReplicationRequest;
	
	enum class EChangeAuthorityOperationResult : uint8;
	enum class EChangeStreamOperationResult : uint8;
}

namespace UE::MultiUserClient::Replication
{
	class FClientChangeOperation;
	class FOnlineClientManager;
	class FSubmissionQueue;
	
	/**
	 * Handles change requests made by external modules via IMultiUserReplication::EnqueueChanges.
	 * This is instantiated per client. As such, each instance manages change requests made for a specific client.
	 *
	 * This class implements the Adapter Pattern: it transforms the public IMultiUserReplication::EnqueueChanges API to our internal API using
	 * ISubmissionOperation & ISubmissionWorkflow. This way, the public and private APIs can change independently.
	 */
	class FExternalClientChangeRequestHandler : public FNoncopyable
	{
	public:

		/** @return An operation to return when not in any session. */
		static TSharedRef<IClientChangeOperation> MakeFailedOperation(EChangeStreamOperationResult StreamResult, EChangeAuthorityOperationResult AuthorityResult);

		/**
		 * @param InStreamId The stream ID the client uses.
		 * @param InGetStreamContentDelegate Delegate for getting the content of the client's stream
		 * @param InSubmissionQueue Used to enqueue the external requests. The owning FMultiUserReplicationManager ensures it outlives the constructed instance.
		 */
		FExternalClientChangeRequestHandler(
			const FGuid& InStreamId,
			FGetStreamContent InGetStreamContentDelegate,
			FSubmissionQueue& InSubmissionQueue
			);
		~FExternalClientChangeRequestHandler();
		
		/** IMultiUserReplication::EnqueueChanges forwards requests to this function. */
		TSharedRef<IClientChangeOperation> HandleRequest(TAttribute<FChangeClientReplicationRequest> SubmissionParams);

	private:

		/**
		 * Detects whether the FExternalClientChangeRequestHandler has been / is being destroyed when processing an operation's completion.
		 *
		 * Here's what could happen, and is likely:
		 * - ~FReplicationClient invokes ~FExternalClientChangeRequestHandler 
		 * - ~FReplicationClient invokes ~ISubmissionWorkflow, which cancels an operation originally started by FExternalClientChangeRequestHandler.
		 * This problem could be solved in a synchronized way (like unsubscribing our OnOperationCompleted), but here's another case albeit unlikely:
		 * - ~FReplicationClient invokes ~FExternalClientChangeRequestHandler
		 * - Just before ~ISubmissionWorkflow is invoked, the UDP unfortunately decides to timeout one of our operations.
		 * 
		 * TL;DR: OnOperationCompleted checks whether this is valid before doing anything.
		 */
		const TSharedRef<FToken> LifetimeToken = FToken::Make();

		/** The stream ID the client uses. */
		const FGuid StreamId;
		/** Delegate for getting the content of the client's stream */
		const FGetStreamContent GetStreamContentDelegate;
		
		/**
		 * Used to enqueue the external requests.
		 * The owning FMultiUserReplicationManager ensures it outlives the constructed instance.
		 */
		FSubmissionQueue& SubmissionQueue;

		using FOperationId = uint32;
		
		FOperationId NextOperationId = 0;
		/**
		 * Each time IMultiUserReplication::EnqueueChanges invokes HandleRequest, an operation is added here and returned by HandleRequest.
		 * The operations invoke OnOperationCompleted when the complete, which removes them from this array.
		 *
		 * This property can only be read & written from the game thread. Relevant to OnOperationCompleted being executed as part of a timeout.
		 */
		TMap<FOperationId, TSharedRef<FClientChangeOperation>> PendingOperations;

		/** Removes the operation from PendingOperations. */
		static void OnOperationCompleted(FOperationId OperationId, TWeakPtr<FToken> WeakToken, FExternalClientChangeRequestHandler* GuardedThis);
	};
}

