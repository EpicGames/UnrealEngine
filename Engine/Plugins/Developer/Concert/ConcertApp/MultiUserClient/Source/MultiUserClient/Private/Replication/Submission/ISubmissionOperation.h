// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Data/AuthoritySubmission.h"
#include "Data/StreamSubmission.h"

namespace UE::MultiUserClient::Replication
{
	enum class ESubmissionOperationCompletedCode
	{
		/** The operation was processed normally */
		Processed,
		/** Cancelled, e.g. because the workflow was destroyed */
		Cancelled
	};
	
	/**
	 * Exposes several events that happen as part of submitting to the server.
	 * Every time ISubmissionWorkflow::SubmitChanges is called a new instance is created.
	 *
	 * The events are exposed as futures, which can complete on any thread.
	 */
	class ISubmissionOperation
	{
	public:

		/** @return Whether a request for changing streams has been or will be sent. */
		virtual bool IsModifyingStreams() const = 0;
		
		/**
		 * Completes when the operation of changing streams has completed.
		 * 
		 * @note This can be called at most once; subsequent calls result in an unset future.
		 * @note This can complete on any thread. Usually this finishes on the game thread but timeouts usually occur on the messaging (UDP) thread.
		 */
		virtual TFuture<FSubmitStreamChangesResponse> OnCompleteStreamChangesFuture_AnyThread() = 0;

		/**
		 * Completes when the authority change request has been sent to the server.
		 * 
		 * @note This can be called at most once; subsequent calls result in an unset future.
		 * @note This can complete on any thread. Usually this finishes on the game thread but timeouts usually occur on the messaging (UDP) thread.
		 */
		virtual TFuture<FSubmitAuthorityChangesRequest> OnRequestAuthorityChangeFuture_AnyThread() = 0;
		
		/**
		 * Completes when the operation of changing authority has completed.
		 * 
		 * @note This can be called at most once; subsequent calls result in an unset future.
		 * @note This can complete on any thread. Usually this finishes on the game thread but timeouts usually occur on the messaging (UDP) thread.
		 */
		virtual TFuture<FSubmitAuthorityChangesResponse> OnCompleteAuthorityChangeFuture_AnyThread() = 0;

		/**
		 * Completes when the operation is done. No further work will be performed.
		 * 
		 * @note This can be called at most once; subsequent calls result in an unset future.
		 * @note This can complete on any thread. Usually this finishes on the game thread but timeouts usually occur on the messaging (UDP) thread.
		 */
		virtual TFuture<ESubmissionOperationCompletedCode> OnCompletedOperation_AnyThread() = 0;
		
		virtual ~ISubmissionOperation() = default;
	};
}
