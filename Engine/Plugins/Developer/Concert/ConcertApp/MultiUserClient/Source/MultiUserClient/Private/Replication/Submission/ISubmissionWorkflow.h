// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/EChangeUploadability.h"
#include "Replication/IConcertClientReplicationManager.h"

#include "Delegates/Delegate.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

struct FConcertReplication_ChangeAuthority_Request;
struct FConcertReplication_ChangeStream_Request;

namespace UE::MultiUserClient::Replication
{
	class ISubmissionOperation;
	struct FSubmitStreamChangesResponse;
	struct FSubmitAuthorityChangesRequest;
	struct FSubmitAuthorityChangesResponse;

	/** At least StreamRequest or AuthorityRequest need to be valid to form a valid request */
	struct FSubmissionParams
	{
		TOptional<FConcertReplication_ChangeStream_Request> StreamRequest;
		TOptional<FConcertReplication_ChangeAuthority_Request> AuthorityRequest;
		
		bool IsStreamChangeEmpty() const { return !StreamRequest.IsSet() || StreamRequest->IsEmpty(); }
		bool IsAuthorityChangeEmpty() const { return !AuthorityRequest.IsSet() || AuthorityRequest->IsEmpty(); }
		bool IsEmpty() const { return IsStreamChangeEmpty() && IsAuthorityChangeEmpty(); }
	};
	
	/**
	 * Manages the flow of changing stream and authority on the server.
	 * Handles both submitting and reverting.
	 *
	 * The general flow is as follows:
	 *	1. Submit stream changes
	 *	2. Submit authority changes
	 */
	class ISubmissionWorkflow
	{
	public:

		/**
		 * Synchronizes the server with the locally made changes to streams and authority.
		 *
		 * This function creates an operation object which emits special events.
		 * If there is already an operation in progress, this function fails.
		 * @see CanSubmit
		 *
		 * @note The operation might start and instantly stop before SubmitChanges finishes (e.g. a network request fails to be created instantly).
		 * @return The operation object if the operation was started
		 */
		virtual TSharedPtr<ISubmissionOperation> SubmitChanges(FSubmissionParams Params) = 0;

		/** @return Detailed information about whether Submit can be called */
		virtual EChangeUploadability GetUploadability() const = 0;
		bool CanSubmit() const { return GetUploadability() == EChangeUploadability::Ready; }
		
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnStreamRequestCompleted, const FSubmitStreamChangesResponse&);
		/**
		 * Broadcasts whenever a submit operation completes the stream change request stage. Useful for accumulating and counting errors.
		 * @note No stream changes may have been requested. Check the error code.
		 */
		virtual FOnStreamRequestCompleted& OnStreamRequestCompleted_AnyThread() = 0;

		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAuthorityRequestCompleted, const FSubmitAuthorityChangesRequest&, const FSubmitAuthorityChangesResponse&);
		/**
		 * Broadcasts whenever a submit operation completes the authority change request stage. Useful for accumulating and counting errors.
		 * @note No authority changes may have been requested. Check the error code.
		 */
		virtual FOnAuthorityRequestCompleted& OnAuthorityRequestCompleted_AnyThread() = 0;

		DECLARE_MULTICAST_DELEGATE(FOnSubmitOperationCompleted);
		/** Broadcasts whenever a submit operation completes. Useful for enqueuing submit operations. */
		virtual FOnSubmitOperationCompleted& OnSubmitOperationCompleted_AnyThread() = 0;

		virtual ~ISubmissionWorkflow() = default;
	};

	/** Shared implementation for ISubmissionWorkflow. */
	class FSubmissionWorkflowBase : public ISubmissionWorkflow
	{
	public:

		//~ Begin ISubmissionWorkflow Interface
		virtual FOnStreamRequestCompleted& OnStreamRequestCompleted_AnyThread() override { return StreamRequestCompletedDelegate; }
		virtual FOnAuthorityRequestCompleted& OnAuthorityRequestCompleted_AnyThread() override { return AuthorityRequestCompletedDelegate; }
		virtual FOnSubmitOperationCompleted& OnSubmitOperationCompleted_AnyThread() override { return OnSubmitOperationCompletedDelegate; }
		//~ End ISubmissionWorkflow Interface

	protected:
		
		FOnStreamRequestCompleted StreamRequestCompletedDelegate;
		FOnAuthorityRequestCompleted AuthorityRequestCompletedDelegate;
		FOnSubmitOperationCompleted OnSubmitOperationCompletedDelegate;
	};
}
