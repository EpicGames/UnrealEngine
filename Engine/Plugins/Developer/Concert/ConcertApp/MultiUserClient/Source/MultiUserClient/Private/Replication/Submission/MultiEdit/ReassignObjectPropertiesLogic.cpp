// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReassignObjectPropertiesLogic.h"

#include "ConcertLogGlobal.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Submission/ISubmissionOperation.h"
#include "Replication/Misc/Util/SynchronizedRequestUtils.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Async/Async.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "FReassignObjectPropertiesLogic"

namespace UE::MultiUserClient::Replication::ReassignObjectProperties
{
	using FClientId = FGuid;
		
	struct FObjectReassignment
	{
		/** Used to build the final changelist to apply to the target client. Maps ClientIds to the object data they had registered. */
		TMap<FClientId, FConcertObjectReplicationMap> OldRegisteredObjects;
		/** The frequency settings clients when reassignment operation was started. */
		TMap<FClientId, FConcertStreamFrequencySettings> OldFrequencies;
		/** Changes to make to all the other clients */
		TMap<FClientId, FConcertReplication_ChangeStream_Request> ReassignedClientRequests;
		/** Authority that the clients had before. This is used to determine whether to give the assigned to client authority once the stream changes have been made. */
		TMap<FClientId, TArray<FSoftObjectPath>> ReassignedAuthority;
	};
	
	/** Builds all the changes that must be made to the clients we are transferring from.  */
	static FObjectReassignment BuildChangesForTransferal(const FOnlineClientManager& ClientManager, TConstArrayView<FSoftObjectPath> ObjectsToReassign, const FGuid& StreamId)
	{
		FObjectReassignment Result;
		TMap<FGuid, FConcertObjectReplicationMap>& OldRegisteredObjects = Result.OldRegisteredObjects;
		TMap<FGuid, FConcertStreamFrequencySettings>& OldFrequencies = Result.OldFrequencies;
		TMap<FGuid, FConcertReplication_ChangeStream_Request>& ReassignedClientRequests = Result.ReassignedClientRequests;
		TMap<FGuid, TArray<FSoftObjectPath>>& ReassignedAuthority = Result.ReassignedAuthority;
	
		for (const FSoftObjectPath& ObjectPath : ObjectsToReassign)
		{
			const FConcertObjectInStreamID ObjectId{ StreamId, ObjectPath };

			const FGlobalAuthorityCache& AuthorityCache = ClientManager.GetAuthorityCache();
			AuthorityCache.ForEachClientWithObjectInStream(ObjectPath,
				[&ClientManager, &OldRegisteredObjects, &OldFrequencies, &ReassignedClientRequests, &ReassignedAuthority, &ObjectPath, &ObjectId, &AuthorityCache](const FGuid& ClientId)
				{
					// Get the property assignments of the client being reassigned from
					const FOnlineClient* ClientToReassignFrom = ClientManager.FindClient(ClientId);
					if (!ensureMsgf(ClientToReassignFrom, TEXT("Authority cache out of sync")))
					{
						return EBreakBehavior::Continue;
					}
					
					const IClientStreamSynchronizer& ReassignedStreamSynchronizer = ClientToReassignFrom->GetStreamSynchronizer();
					const FConcertReplicatedObjectInfo* ObjectInfo = ReassignedStreamSynchronizer.GetServerState().ReplicatedObjects.Find(ObjectId.Object);
					if (!ensureMsgf(ObjectInfo, TEXT("ForEachClientWithObjectInStream lied")))
					{
						return EBreakBehavior::Continue;
					}

					// Transfer from client being reassigned from to the client being assigned to
					OldRegisteredObjects.FindOrAdd(ClientId).ReplicatedObjects.Add(ObjectPath, *ObjectInfo);
					// Remove the object completely from the reassigned client.
					ReassignedClientRequests.FindOrAdd(ClientId).ObjectsToRemove.Add(ObjectId);
					// Later we can determine whether to give the assigned to client authority
					if (AuthorityCache.HasAuthorityOverObject(ObjectPath, ClientId))
					{
						ReassignedAuthority.FindOrAdd(ClientId).Add(ObjectPath);
					}

					// Keep track of this object's frequency settings so it can be transferred
					const FConcertObjectReplicationSettings* OverrideFrequencySettings = ReassignedStreamSynchronizer.GetFrequencySettings().ObjectOverrides.Find(ObjectId.Object);
					if (OverrideFrequencySettings)
					{
						OldFrequencies.FindOrAdd(ClientId).ObjectOverrides.Add(ObjectId.Object, *OverrideFrequencySettings);
					}
				
					return EBreakBehavior::Continue;
				});
		}

		// Need to set Defaults of the OldFrequencies because that's not been done, yet.
		for (TPair<FGuid, FConcertStreamFrequencySettings>& OldFrequency : OldFrequencies)
		{
			const FOnlineClient* ClientToReassignFrom = ClientManager.FindClient(OldFrequency.Key);
			check(ClientToReassignFrom);
			OldFrequency.Value.Defaults = ClientToReassignFrom->GetStreamSynchronizer().GetFrequencySettings().Defaults;
		}

		return Result;
	}

	static FConcertObjectReplicationSettings GetHighestFrequency(const FSoftObjectPath& Object, const TMap<FClientId, FConcertStreamFrequencySettings>& OldFrequencies)
	{
		TOptional<FConcertObjectReplicationSettings> Settings;
		for (const TPair<FClientId, FConcertStreamFrequencySettings>& Pair : OldFrequencies)
		{
			const FConcertObjectReplicationSettings& ClientSettings = Pair.Value.GetSettingsFor(Object);
			if (!Settings)
			{
				Settings = ClientSettings;
			}
			else if (*Settings <= ClientSettings)
			{
				Settings = ClientSettings;
			}
		}
		return Settings.Get(FConcertObjectReplicationSettings{});
	}

	/** Builds a changelist based on which remote clients we managed to change successfully. */
	static TPair<FStreamChangelist, FFrequencyChangelist> MakeChangelistFromAppliedChanges(
		const TMap<FGuid, FConcertObjectReplicationMap>& OldRegisteredObjects,
		const TMap<FClientId, FConcertStreamFrequencySettings>& OldFrequencies,
		const FParallelExecutionResult& ParallelExecutionResult,
		const IClientStreamSynchronizer& AssignedToStreamSynchronizer
		)
	{
		FStreamChangelist ObjectChanges;
		FFrequencyChangelist FrequencyChanges;
		
		const FConcertObjectReplicationMap& CurrentTargetClientState = AssignedToStreamSynchronizer.GetServerState();
		const FConcertStreamFrequencySettings& CurrentFrequencySettings = AssignedToStreamSynchronizer.GetFrequencySettings();
		const FGuid TargetStreamId = AssignedToStreamSynchronizer.GetStreamId();
		for (const TPair<FGuid, FSubmitStreamChangesResponse>& ChangesRequestedOnRemote : ParallelExecutionResult.StreamResponses)
		{
			const TOptional<FCompletedChangeSubmission>& SubmissionInfo = ChangesRequestedOnRemote.Value.SubmissionInfo;
			if (!SubmissionInfo || SubmissionInfo->Response.IsFailure())
			{
				continue;
			}
			
			const FConcertObjectReplicationMap& ObjectReplicationMap = OldRegisteredObjects[ChangesRequestedOnRemote.Key];
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ChangesToApply : ObjectReplicationMap.ReplicatedObjects)
			{
				const FSoftObjectPath& ObjectPath = ChangesToApply.Key;
				const FConcertObjectInStreamID ObjectId { TargetStreamId, ObjectPath };
				TOptional<FConcertReplication_ChangeStream_PutObject> PutRequest = FConcertReplication_ChangeStream_PutObject::MakeFromInfo(ChangesToApply.Value);
				if (!ensure(PutRequest))
				{
					continue;
				}

				// We want to append the other client's properties to the ones the target client already has.
				// TODO UE-201166: This step would not be necessary if we had an append operation in FConcertReplication_ChangeStream_PutObject
				const FConcertReplicatedObjectInfo* CurrentInfo = CurrentTargetClientState.ReplicatedObjects.Find(ObjectId.Object);
				if (CurrentInfo)
				{
					for (const FConcertPropertyChain& PropertyChain : CurrentInfo->PropertySelection.ReplicatedProperties)
					{
						PutRequest->Properties.ReplicatedProperties.Add(PropertyChain);
					}
				}
				
				ObjectChanges.ObjectsToPut.Add(ObjectId, MoveTemp(*PutRequest));

				// The reassigned to client will replicate at the highest rate of either its current rate or the highest from the source clients.
				const FConcertObjectReplicationSettings& CurrentObjectFrequencySettings = CurrentFrequencySettings.GetSettingsFor(ObjectPath);
				const FConcertObjectReplicationSettings HighestSourceFrequencySettings = GetHighestFrequency(ObjectPath, OldFrequencies);
				if (CurrentObjectFrequencySettings < HighestSourceFrequencySettings)
				{
					FrequencyChanges.OverridesToAdd.Add(ObjectPath, HighestSourceFrequencySettings);
				}
			}
		}
		return { ObjectChanges, FrequencyChanges };
	}

	/** Builds list of authority changes based on which clients we managed to change successfully. */
	static FConcertReplication_ChangeAuthority_Request MakeAuthorityRequestFrom(
		const TMap<FGuid, TArray<FSoftObjectPath>>& ReassignedAuthority, 
		const FParallelExecutionResult& ParallelExecutionResult,
		const FGuid& TargetStreamId
		)
	{
		FConcertReplication_ChangeAuthority_Request ResultRequest;
		for (const TPair<FGuid, FSubmitStreamChangesResponse>& ChangesRequestedOnRemote : ParallelExecutionResult.StreamResponses)
		{
			const TOptional<FCompletedChangeSubmission>& SubmissionInfo = ChangesRequestedOnRemote.Value.SubmissionInfo;
			if (!SubmissionInfo || SubmissionInfo->Response.IsFailure())
			{
				continue;
			}

			const TArray<FSoftObjectPath>* PreviousAuthority = ReassignedAuthority.Find(ChangesRequestedOnRemote.Key);
			if (!PreviousAuthority)
			{
				continue;
			}

			for (const FSoftObjectPath& ObjectPath : *PreviousAuthority)
			{
				ResultRequest.TakeAuthority.Add(ObjectPath, FConcertStreamArray{{ TargetStreamId }});
			}
		}
		return ResultRequest;
	}

	/** @return Whether TargetClientObjects includes all properties ClientsToConsider have registered for ObjectToCheck. */
	static bool DoesTargetIncludeOtherClientsContent_SingleObject(
		const FConcertObjectReplicationMap& TargetClientObjects,
		const FOnlineClientManager& ClientManager,
		const FSoftObjectPath& ObjectToCheck
		)
	{
		const FConcertReplicatedObjectInfo* ObjectInfo = TargetClientObjects.ReplicatedObjects.Find(ObjectToCheck);

		bool bIncludesAllOtherClients = true;
		ClientManager.ForEachClient([&ObjectToCheck, ObjectInfo, &bIncludesAllOtherClients](const FOnlineClient& OtherClient)
		{
			const FConcertObjectReplicationMap& OtherClientObjects = OtherClient.GetStreamSynchronizer().GetServerState();
			const FConcertReplicatedObjectInfo* OtherObjectInfo = OtherClientObjects.ReplicatedObjects.Find(ObjectToCheck);
				
			// TargetClient has at least as much if OtherClient has nothing
			if (!OtherObjectInfo || OtherObjectInfo->PropertySelection.ReplicatedProperties.IsEmpty())
			{
				return EBreakBehavior::Continue;
			}

			const bool bTargetContainsOther = ObjectInfo && ObjectInfo->PropertySelection.Includes(OtherObjectInfo->PropertySelection);
			bIncludesAllOtherClients &= bTargetContainsOther;
			return bIncludesAllOtherClients ? EBreakBehavior::Continue : EBreakBehavior::Break;
		});

		return bIncludesAllOtherClients;
	}
	
	/** @return Whether ClientId includes all properties assigned to ObjectsToCheck as all other clients do. */
	static bool DoesTargetIncludeOtherClientsContent(
		const FGuid& TargetClientId,
		TConstArrayView<FSoftObjectPath> ObjectsToCheck,
		const FOnlineClientManager& ClientManager
		)
	{
		const FOnlineClient* TargetClient = ClientManager.FindClient(TargetClientId);
		if (!ensure(TargetClient))
		{
			return false;
		}

		const FConcertObjectReplicationMap& TargetClientObjects = TargetClient->GetStreamSynchronizer().GetServerState();
		return Algo::AllOf(ObjectsToCheck, [&ClientManager, &TargetClientObjects](const FSoftObjectPath& ObjectToCheck)
		{
			return DoesTargetIncludeOtherClientsContent_SingleObject(TargetClientObjects, ClientManager, ObjectToCheck);
		});
	}

	/** @return Whether ClientId has any properties assigned to any of the objects in ObjectsToCheck. */
	static bool HasAnyProperties(const FGuid& ClientId, TConstArrayView<FSoftObjectPath> ObjectsToCheck, const FOnlineClientManager& ClientManager)
	{
		const FOnlineClient* TargetClient = ClientManager.FindClient(ClientId);
		if (!ensure(TargetClient))
		{
			return false;
		}

		const FConcertObjectReplicationMap& TargetClientObjects = TargetClient->GetStreamSynchronizer().GetServerState();
		return Algo::AnyOf(ObjectsToCheck, [&TargetClientObjects](const FSoftObjectPath& ObjectPath)
		{
			const FConcertReplicatedObjectInfo* ObjectInfo = TargetClientObjects.ReplicatedObjects.Find(ObjectPath);
			return ObjectInfo && !ObjectInfo->PropertySelection.ReplicatedProperties.IsEmpty();
		});
	}

	template<typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, const FGuid&>
	static void EnumerateOwningOnlineClients(
		const FOnlineClientManager& ClientManager,
		const FSoftObjectPath& ObjectPath,
		TCallback&& Callback
		) 
	{
		ClientManager.ForEachClient([&ObjectPath, &Callback](const FOnlineClient& ReplicationClient)
		{
			const bool bHasProperties = ReplicationClient.GetStreamSynchronizer().GetServerState().HasProperties(ObjectPath);
			return bHasProperties ? Callback(ReplicationClient.GetEndpointId()) : EBreakBehavior::Continue;
		});
	}
}

namespace UE::MultiUserClient::Replication
{
	FReassignObjectPropertiesLogic::FReassignObjectPropertiesLogic(FOnlineClientManager& InClientManager)
		: ClientManager(InClientManager)
	{
		ClientManager.OnRemoteClientsChanged().AddRaw(this, &FReassignObjectPropertiesLogic::OnRemoteClientsChanged);
	}

	FReassignObjectPropertiesLogic::~FReassignObjectPropertiesLogic()
	{
		// Note that all of this clean up is not REALLY needed because the FReassignObjectPropertiesLogic is a member of ClientManager.
		// However, we'll follow good RAII here in case that should ever change.
		ClientManager.OnRemoteClientsChanged().RemoveAll(this);

		if (FSubmissionQueue* SubmissionQueue = FindSubmissionQueueForTargetClient())
		{
			SubmissionQueue->Dequeue_GameThread(*this);
		}
	}

	bool FReassignObjectPropertiesLogic::OwnsAnyOf(TConstArrayView<FSoftObjectPath> Objects, const FGuid& TargetClientId) const
	{
		bool bOwnsAny = false;
		for (int32 i = 0; i < Objects.Num() && !bOwnsAny; ++i)
		{
			ReassignObjectProperties::EnumerateOwningOnlineClients(ClientManager, Objects[i],[&bOwnsAny, &TargetClientId](const FGuid& ClientId)
			{
				bOwnsAny |= ClientId == TargetClientId;
				return bOwnsAny ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});
		}
		return bOwnsAny;
	}

	bool FReassignObjectPropertiesLogic::IsAnyObjectOwned(TConstArrayView<FSoftObjectPath> Objects) const
	{
		bool bOwnsAny = false;
		for (int32 i = 0; i < Objects.Num() && !bOwnsAny; ++i)
		{
			ReassignObjectProperties::EnumerateOwningOnlineClients(ClientManager,Objects[i], [&bOwnsAny](const FGuid& ClientId)
			{
				bOwnsAny = true;
				return bOwnsAny ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});
		}
		return bOwnsAny;
	}

#define SET_REASON(Text) if (Reason) { *Reason = Text; }
	bool FReassignObjectPropertiesLogic::CanReassignAnyTo(TConstArrayView<FSoftObjectPath> ObjectsToReassign, const FGuid& ClientId, FText* Reason) const
	{
		// TODO UE-201309: Implement queue for reassignment operations for better UX
		if (InProgressOperation)
		{
			SET_REASON(LOCTEXT("OperationInProgress", "Wait for the previous reassignment operation to finish."));
			return false;
		}
		
		const FOnlineClient* Client = ClientManager.FindClient(ClientId);
		if (!Client)
		{
			SET_REASON(LOCTEXT("Reason.ClientDisconnected", "Client disconnected"));
			return false;
		}

		// Relevant after user has re-assigned all properties to a given client
		if (ReassignObjectProperties::DoesTargetIncludeOtherClientsContent(ClientId, ObjectsToReassign, ClientManager))
		{
			const bool bHasAnyProperties = ReassignObjectProperties::HasAnyProperties(ClientId, ObjectsToReassign, ClientManager);
			SET_REASON(
				FText::Format(
					LOCTEXT("Reason.NothingToAssignFmt", "Nothing to assign: {0}"),
					bHasAnyProperties ? LOCTEXT("OwnsAll", "client already owns all properties.") : LOCTEXT("HasNone", "assign some properties first.")
					)
				);
			return false;
		}

		return true;
	}
#undef SET_REASON
	
	void FReassignObjectPropertiesLogic::ReassignAllTo(TArray<FSoftObjectPath> ObjectsToReassign, const FGuid& ClientId)
	{
		using namespace ConcertSyncClient::Replication;
		
		const FOnlineClient* ClientToAssignTo = ClientManager.FindClient(ClientId);
		if (!ensure(ClientToAssignTo))
		{
			UE_LOGF(LogConcert, Error, "Property Reassignment: The target client is not editable.");
			return;
		}

		const IClientStreamSynchronizer& StreamSynchronizer = ClientToAssignTo->GetStreamSynchronizer();
		const FGuid& TargetStreamId = StreamSynchronizer.GetStreamId();
		auto[OldRegisteredObjects, OldFrequencies, ReassignedClientRequests, ReassignedAuthority]
			= ReassignObjectProperties::BuildChangesForTransferal(
				ClientManager,
				ObjectsToReassign,
				TargetStreamId
			);

		// ParallelSubmissionOperation's RAII cleans up on destruction, e.g. dequeues submission tasks, so InProgressOperation keeps a reference to it.
		const TSharedPtr<IParallelSubmissionOperation> ParallelSubmissionOperation = ExecuteParallelStreamChanges(ClientManager, ReassignedClientRequests);
		if (!ensureMsgf(ParallelSubmissionOperation, TEXT("Investigate why starting task failed")))
		{
			return;
		}
		
		InProgressOperation = FOperationData{ MoveTemp(ObjectsToReassign), ClientId,
			MoveTemp(OldRegisteredObjects),MoveTemp(OldFrequencies), MoveTemp(ReassignedAuthority),
			ParallelSubmissionOperation.ToSharedRef()
		};
		ParallelSubmissionOperation->OnCompletedFuture_AnyThread()
			.Next([this](FParallelExecutionResult&& Result)
			{
				// When "this" is being destroyed, there is no sense in further processing.
				// IParallelSubmissionOperation's RAII sets bWasCancelled = true when it is destroyed.
				// The IParallelSubmissionOperation is destroyed just before "this" due to InProgressOperation keeping the only strong reference.
				if (!Result.bWasCancelled)
				{
					OnAssignedFromClientsCompleted(Result);
				}
			});
	}

	bool FReassignObjectPropertiesLogic::IsReassigning(const FSoftObjectPath& ObjectPath) const
	{
		if (!InProgressOperation)
		{
			return false;
		}

		return Algo::AnyOf(InProgressOperation->OldRegisteredObjects, [&ObjectPath](const TPair<FGuid, FConcertObjectReplicationMap>& Change)
		{
			const FConcertReplicatedObjectInfo* ObjectInfo = Change.Value.ReplicatedObjects.Find(ObjectPath);
			return ObjectInfo && !ObjectInfo->PropertySelection.ReplicatedProperties.IsEmpty();
		});
	}

	TOptional<FDateTime> FReassignObjectPropertiesLogic::GetTimeReassignmentWasStarted() const
	{
		return InProgressOperation ? InProgressOperation->StartTime : TOptional<FDateTime>{};
	}

	void FReassignObjectPropertiesLogic::OnRemoteClientsChanged()
	{
		if (InProgressOperation && !ClientManager.FindClient(InProgressOperation->AssignedToClient))
		{
			UE_LOGF(LogConcert, Warning,
				"Client %ls disconnected while reassigning objects to it. Cancelling the operation however object may already have been removed on other clients.",
				*InProgressOperation->AssignedToClient.ToString()
				);
			InProgressOperation.Reset();
		}
	}

	void FReassignObjectPropertiesLogic::OnAssignedFromClientsCompleted(FParallelExecutionResult ParallelResult)
	{
		InProgressOperation->ParallelIntermediateResult = MoveTemp(ParallelResult);
		GuardedExecuteOnGameThread([this]()
		{
			FSubmissionQueue* SubmissionQueue = FindSubmissionQueueForTargetClient();
			if (ensureMsgf(SubmissionQueue, TEXT("Target client (likely) disconnected during reassignment but FReassignObjectPropertiesLogic should have handled canelling the reassignment.")))
			{
				// It would be better to check ParallelResult for failed sub-actions and possibly take them out of the request we're about to send to the reassigned to client.
				// For now, we'll just send and worst case the change is rejected by the server.
				SubmissionQueue->SubmitNowOrEnqueue_GameThread(*this);
			}
		});
	}

	FSubmissionQueue* FReassignObjectPropertiesLogic::FindSubmissionQueueForTargetClient() const
	{
		FOnlineClient* Client = InProgressOperation.IsSet()
			? ClientManager.FindClient(InProgressOperation->AssignedToClient)
			: nullptr;
		return Client ? &Client->GetSubmissionQueue() : nullptr;
	}

	void FReassignObjectPropertiesLogic::PerformSubmission_GameThread(ISubmissionWorkflow& Workflow)
	{
		// Was the pending change cancelled, e.g. due to the target client disconnecting?
		if (!InProgressOperation)
		{
			return;
		}
		
		const FOnlineClient* ClientToAssignTo = ClientManager.FindClient(InProgressOperation->AssignedToClient);
		if (!ensureMsgf(ClientToAssignTo, TEXT("FReassignObjectPropertiesLogic should have cancelled the change upon disconnect")))
		{
			UE_LOGF(LogConcert, Error, "Property Reassignment: The target client is no longer editable.");
			return;
		}
		
		// TODO UE-201340: If there were errors, add a dedicated notification
		const FParallelExecutionResult& ParallelResult = *InProgressOperation->ParallelIntermediateResult;
		const bool bAtLeastOneSucceeded = Algo::AnyOf(ParallelResult.StreamResponses, [](const TPair<FGuid, FSubmitStreamChangesResponse>& Response)
		{
			return Response.Value.SubmissionInfo && Response.Value.SubmissionInfo->Response.IsSuccess();
		});
		if (!bAtLeastOneSucceeded)
		{
			UE_LOGF(LogConcert, Warning, "Property Reassignment: No reassignment will take place because all remote clients failed.");
			InProgressOperation.Reset();
			return;
		}
		
		const IClientStreamSynchronizer& StreamSynchronizer = ClientToAssignTo->GetStreamSynchronizer();
		const FGuid& StreamId = StreamSynchronizer.GetStreamId();
		const bool bIsStreamRegistered = !StreamSynchronizer.GetServerState().IsEmpty();
		auto[ObjectChanges, FrequencyChanges] = ReassignObjectProperties::MakeChangelistFromAppliedChanges(
			InProgressOperation->OldRegisteredObjects,
			InProgressOperation->OldFrequencies,
			ParallelResult,
			StreamSynchronizer
			);
		FConcertReplication_ChangeStream_Request AssignedToClientRequest = bIsStreamRegistered
			? StreamRequestUtils::BuildChangeRequest_UpdateExistingStream(StreamId, ObjectChanges, MoveTemp(FrequencyChanges))
			: StreamRequestUtils::BuildChangeRequest_CreateNewStream(StreamId, ObjectChanges, MoveTemp(FrequencyChanges));
		
		const TSharedPtr<ISubmissionOperation> SubmitOperation = Workflow.SubmitChanges({
			MoveTemp(AssignedToClientRequest),
			ReassignObjectProperties::MakeAuthorityRequestFrom(InProgressOperation->OldAuthority, ParallelResult, StreamId)
		});
		if (!ensure(SubmitOperation))
		{
			UE_LOGF(LogConcert, Error, "Property Reassignment: Failed to give target client properties.");
			return;
		}
		
		SubmitOperation->OnCompletedOperation_AnyThread().Next([this](ESubmissionOperationCompletedCode)
		{
			// We're not on game thread if OnCompletedOperation completes due to a timeout.
			// By contract, IParallelSubmissionOperation must be destroyed on the game thread!
			GuardedExecuteOnGameThread([this]() { InProgressOperation.Reset(); }); 
		});
	}
	
	void FReassignObjectPropertiesLogic::GuardedExecuteOnGameThread(TUniqueFunction<void()> Callback) const
	{
		if (IsInGameThread())
		{
			Callback();
		}
		else
		{
			AsyncTask(ENamedThreads::GameThread, [Callback = MoveTemp(Callback), WeakToken = DestructionToken.ToWeakPtr()]()
			{
				if (WeakToken.IsValid())
				{
					Callback();
				}
			});
		}
	}
}

#undef LOCTEXT_NAMESPACE
