// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IToken.h"
#include "Replication/Misc/Util/SynchronizedRequestUtils.h"
#include "Replication/Submission/Queue/DeferredSubmitter.h"

#include "Misc/OptionalFwd.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

enum class EBreakBehavior : uint8;
struct FConcertPropertyChain;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
	class IParallelSubmissionOperation;
	
	/**
	 * Knows how to transfer object to properties mappings from clients to other clients.
	 * Acts as a model for the UI.
	 *
	 * Currently transfering is implemented using a cooperative peer-to-peer environment. This makes the operation difficult to sync and impossible to make atomic:
	 * that is if one client fails, we cannot rollback the changes already done on other clients.
	 * TODO UE-201136: In the future UE-201136 is supposed to address this problem by introducing a dedicated server operation.
	 * 
	 * @see SMultiClientView, MultiStreamColumns::ReassignOwnership
	 */
	class FReassignObjectPropertiesLogic
		: public FNoncopyable
		, IDeferredSubmitter
	{
	public:

		enum class EOwnershipState
		{
			HasObjectRegistered,
			NoOwnership
		};
		using FProcessClientOwnership = TFunctionRef<EBreakBehavior(const FGuid& ClientId, EOwnershipState Ownership)>;
		
		FReassignObjectPropertiesLogic(FOnlineClientManager& InClientManager);
		virtual ~FReassignObjectPropertiesLogic() override;

		/** @return Whether ClientId has ownership over at least one */
		bool OwnsAnyOf(TConstArrayView<FSoftObjectPath> Objects, const FGuid& TargetClientId) const;
		/** @return Whether any of the following objects has at least one owner */
		bool IsAnyObjectOwned(TConstArrayView<FSoftObjectPath> Objects) const;
		
		/** @return Whether content can be assigned to this client (not possible if client does not allow remote changing of streams). */
		bool CanReassignAnyTo(TConstArrayView<FSoftObjectPath> ObjectsToReassign, const FGuid& ClientId, FText* Reason = nullptr) const;
		/** Reassigns all properties mapped to the objects from other clients to ClientId.*/
		void ReassignAllTo(TArray<FSoftObjectPath> ObjectsToReassign, const FGuid& ClientId);

		/** @return Whether ObjectPath is currently being reassigned by this logic. */
		bool IsReassigning(const FSoftObjectPath& ObjectPath) const;
		/** @return The local time at which reassignment operation was started. Useful for UI showing throbber but only if operation takes long.*/
		TOptional<FDateTime> GetTimeReassignmentWasStarted() const;

	private:

		/** Detects when "this" is destroyed to avoid dereferencing it in latent actions. */
		const TSharedRef<FToken> DestructionToken = FToken::Make();

		/** Used to get info about clients and register to client list changing. */
		FOnlineClientManager& ClientManager;

		struct FOperationData
		{
			using FClientId = FGuid;
			
			const TArray<FSoftObjectPath> ObjectsBeingReassigned;
			const FGuid AssignedToClient;
			/** The objects we tried to remove from all the other clients. If the removal request worked, the (previously) assigned properties will be transferred to the target client. */
			const TMap<FClientId, FConcertObjectReplicationMap> OldRegisteredObjects;
			/** The frequency settings that clients had at the beginning of the reassign operation. */
			const TMap<FClientId, FConcertStreamFrequencySettings> OldFrequencies; 
			/** Clients to the objects they had authority over. Used to later to determine which objects to give the target client authority over (do not transfer authority from failed clients). */
			const TMap<FClientId, TArray<FSoftObjectPath>> OldAuthority;
			
			/**
			 * The first step of reassignment is to remove all object bindings from all other clients.
			 * 
			 * The operation does RAII for cleaning up latent resources, such as dequeuing submission tasks, and handling underlying Concert messages
			 * futures completing at some later time.
			 *
			 * @note This must be destroyed on the game thread (see ExecuteParallelStreamChanges)
			 */
			TSharedRef<IParallelSubmissionOperation> RemoveObjectsFromClientsStep;
			/** Result from completing RemoveObjectsFromClientsStep once it is completed. Unset until RemoveObjectsFromClientsStep completes. */
			TOptional<FParallelExecutionResult> ParallelIntermediateResult;

			/** Only exists for GetTimeReassignmentWasStarted. */
			const FDateTime StartTime = FDateTime::Now(); 
		};
		/** Set whilst an operation is in progress. */
		TOptional<FOperationData> InProgressOperation;
		
		/** Broadcast when the result of EnumerateClientOwnershipState may have changed. */
		FSimpleMulticastDelegate OnOwnershipChangedDelegate;

		void OnRemoteClientsChanged();

		/** Called once all clients being reassigned have remove their object bindings. The target client will now receive all the previous bindings. */
		void OnAssignedFromClientsCompleted(FParallelExecutionResult ParallelResult);
		/** @return Finds the submission queue for the client being reassigned to. */
		FSubmissionQueue* FindSubmissionQueueForTargetClient() const;

		//~ Begin IDeferredSubmitter Interface
		virtual void PerformSubmission_GameThread(ISubmissionWorkflow& Workflow) override;
		//~ End IDeferredSubmitter Interface

		/** Executes an action straight away if on game thread or schedules for it to be executed. Will ensure that "this" is not destroyed if scheduled latently. */
		void GuardedExecuteOnGameThread(TUniqueFunction<void()> Callback) const;
	};
}
