// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "UObject/SoftObjectPath.h"

namespace UE::MultiUserClient::Replication
{
	struct FAccumulatedStreamErrors
	{
		int32 NumTimeouts = 0;

		/**
		 * Changing the these objects would cause authority conflicts.
		 * Maps to the number of times it was rejected (across multiple submits).
		 */
		TMap<FSoftObjectPath, int32> AuthorityConflicts;
		/**
		 * This should be empty. If not, that means our client code made a bad request.
		 * Maps to the number of times it was rejected (across multiple submits).
		 */
		TMap<FSoftObjectPath, int32> SemanticErrors;
		/** Should be false. If not, that means our client code made a bad request. */
		bool bFailedStreamCreation = false;
	};
	
	struct FAccumulatedAuthorityErrors
	{
		int32 NumTimeouts = 0;

		/**
		 * These objects had conflicts.
		 * Maps to the number of times it was rejected (across multiple submits).
		 */
		TMap<FSoftObjectPath, int32> Rejected;
	};
}