// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAuthorityRejectedNotification.h"

#include "AccumulatedSubmissionErrors.h"

#include "Algo/Accumulate.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAuthorityRejectedNotification"

namespace UE::MultiUserClient::Replication
{
	void SAuthorityRejectedNotification::Construct(const FArguments& InArgs)
	{
		ErrorsAttribute = InArgs._Errors;
		
		Super::Construct(
			Super::FArguments()
			.Message(LOCTEXT("MainMessage", "Errors taking ownership"))
			.OnCloseClicked(InArgs._OnCloseClicked)
			);
	}

	void SAuthorityRejectedNotification::Refresh()
	{
		TSharedRef<SVerticalBox> Result = SNew(SVerticalBox);
		const FAccumulatedAuthorityErrors& Errors = *ErrorsAttribute.Get();
		
		if (Errors.NumTimeouts > 0)
		{
			AddTimeoutErrorWidget(Result, Errors);
		}
		if (!Errors.Rejected.IsEmpty())
		{
			AddRejectionErrorWidget(Result, Errors);
		}
		
		SetErrorContent(Result);
	}

	void SAuthorityRejectedNotification::AddTimeoutErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedAuthorityErrors& Errors)
	{
		Result->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("TimeoutsFmt", "{0} {0}|plural(one=Timeout,other=Timeouts)"), Errors.NumTimeouts))
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			];
	}

	void SAuthorityRejectedNotification::AddRejectionErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedAuthorityErrors& Errors)
	{
		const int32 NumConflicts = Algo::TransformAccumulate(Errors.Rejected, [](const TPair<FSoftObjectPath, int32>& Pair){ return Pair.Value; }, 0);
		const FText Text = FText::Format(
			LOCTEXT("RejectionsFmt", "{0} {0}|plural(one=Rejection,other=Rejections) for {1} {1}|plural(one=Object,other=Objects)"),
			NumConflicts,
			Errors.Rejected.Num()
			);
		
		Result->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(Text)
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			];
	}
}

#undef LOCTEXT_NAMESPACE