// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBaseRejectionNotification.h"

#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SBaseRejectionNotification"

namespace UE::MultiUserClient::Replication
{
	void SBaseRejectionNotification::Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Left)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
				.DesiredSizeOverride(FVector2D(32,32))
			]
			
			+SHorizontalBox::Slot()
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				BuildErrorContent(InArgs)
			]
		];

		Refresh();
	}

	void SBaseRejectionNotification::SetErrorContent(const TSharedRef<SWidget>& Widget)
	{
		ErrorContent->SetContent(Widget);
	}

	TSharedRef<SWidget> SBaseRejectionNotification::BuildErrorContent(const FArguments& InArgs)
	{
		return SNew(SVerticalBox)
			
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(InArgs._Message)
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontBold")))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			]

			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 10.0f, 0.0f, 2.0f))
			.AutoHeight()
			[
				SAssignNew(ErrorContent, SBox)
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("Close", "Close"))
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
					.OnClicked(InArgs._OnCloseClicked) 
				]
			];
	}
}

#undef LOCTEXT_NAMESPACE