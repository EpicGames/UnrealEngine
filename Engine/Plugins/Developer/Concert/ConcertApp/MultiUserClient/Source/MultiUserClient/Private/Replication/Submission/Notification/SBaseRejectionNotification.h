// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/Notifications/SNotificationList.h"

class SBox;
	
namespace UE::MultiUserClient::Replication
{
	/** Shared base class for rejection notifications. Builds the general UI structure. */
	class SBaseRejectionNotification : public SCompoundWidget, public INotificationWidget
	{
	public:

		SLATE_BEGIN_ARGS(SBaseRejectionNotification)
		{}
			/** The main message displayed at the top */
			SLATE_ARGUMENT(FText, Message)
			/** Called when the close button is pressed. Is supposed to close the notification. */
			SLATE_EVENT(FOnClicked, OnCloseClicked)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/** Rebuilds the UI from the errors. */
		virtual void Refresh() = 0;
		
		//~ Begin INotificationWidget interface
		virtual void OnSetCompletionState(SNotificationItem::ECompletionState State) override {}
		virtual TSharedRef<SWidget> AsWidget() override { return AsShared(); }
		//~ End INotificationWidget interface

	protected:

		void SetErrorContent(const TSharedRef<SWidget>& Widget);
		
	private:

		/** Subclasses place their content in this */
		TSharedPtr<SBox> ErrorContent;

		TSharedRef<SWidget> BuildErrorContent(const FArguments& InArgs);
	};
}


