// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStreamRejectedNotification.h"

#include "AccumulatedSubmissionErrors.h"

#include "Algo/Accumulate.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SStreamRejectedNotification"

namespace UE::MultiUserClient::Replication
{
	void SStreamRejectedNotification::Construct(const FArguments& InArgs)
	{
		ErrorsAttribute = InArgs._Errors;
		
		Super::Construct(
			Super::FArguments()
			.Message(LOCTEXT("MainMessage", "Errors updating stream"))
			.OnCloseClicked(InArgs._OnCloseClicked)
			);
	}

	void SStreamRejectedNotification::Refresh()
	{
		TSharedRef<SVerticalBox> Result = SNew(SVerticalBox);
		const FAccumulatedStreamErrors& Errors = *ErrorsAttribute.Get();
		
		if (Errors.NumTimeouts > 0)
		{
			AddTimeoutErrorWidget(Result, Errors);
		}
		if (!Errors.AuthorityConflicts.IsEmpty())
		{
			AddAuthorityConflictErrorWidget(Result, Errors);
		}
		if (!Errors.SemanticErrors.IsEmpty())
		{
			AddSemanticErrorWidget(Result, Errors);
		}
		if (Errors.bFailedStreamCreation)
		{
			AddStreamCreationErrorWidget(Result, Errors);
		}
		
		SetErrorContent(Result);
	}

	void SStreamRejectedNotification::AddTimeoutErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors)
	{
		Result->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("TimeoutsFmt", "{0} {0}|plural(one=Timeout,other=Timeouts)"), Errors.NumTimeouts))
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			];
	}

	void SStreamRejectedNotification::AddAuthorityConflictErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors)
	{
		const int32 NumConflicts = Algo::TransformAccumulate(Errors.AuthorityConflicts, [](const TPair<FSoftObjectPath, int32>& Pair){ return Pair.Value; }, 0);
		const FText Text = FText::Format(
			LOCTEXT("ConflictsFmt", "{0} {0}|plural(one=Conflict,other=Conflicts) for {1} {1}|plural(one=Object,other=Objects)"),
			NumConflicts,
			Errors.AuthorityConflicts.Num()
			);
		
		Result->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(Text)
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			];
	}
	
	void SStreamRejectedNotification::AddSemanticErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors)
	{
		const int32 NumConflicts = Algo::TransformAccumulate(Errors.SemanticErrors, [](const TPair<FSoftObjectPath, int32>& Pair){ return Pair.Value; }, 0);
		const FText Text = FText::Format(
			LOCTEXT("SemanticErrorsFmt", "{0} {0}|plural(one=Semantic Error,other=Semantic Errors) for {1} {1}|plural(one=Object,other=Objects)"),
			NumConflicts,
			Errors.AuthorityConflicts.Num()
			);
		
		Result->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(Text)
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			];
	}
	
	void SStreamRejectedNotification::AddStreamCreationErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors)
	{
		Result->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RejectionsFmt", "Failed to create stream"))
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			];
	}
}

#undef LOCTEXT_NAMESPACE