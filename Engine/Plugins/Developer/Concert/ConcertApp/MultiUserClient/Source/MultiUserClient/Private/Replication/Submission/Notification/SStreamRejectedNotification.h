// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SBaseRejectionNotification.h"

namespace UE::MultiUserClient::Replication
{
	struct FAccumulatedStreamErrors;
	
	/** Displays FAccumulatedStreamErrors. */
	class SStreamRejectedNotification : public SBaseRejectionNotification
	{
		using Super = SBaseRejectionNotification;
	public:

		SLATE_BEGIN_ARGS(SStreamRejectedNotification)
		{}
			/** The errors to display */
			SLATE_ATTRIBUTE(const FAccumulatedStreamErrors*, Errors)
			/** Called when the close button is pressed. Is supposed to close the notification. */
			SLATE_EVENT(FOnClicked, OnCloseClicked)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/** Rebuilds the UI from the errors. */
		void Refresh();
		
		//~ Begin INotificationWidget interface
		virtual void OnSetCompletionState(SNotificationItem::ECompletionState State) override {}
		virtual TSharedRef<SWidget> AsWidget() override { return AsShared(); }
		//~ End INotificationWidget interface

	private:

		TAttribute<const FAccumulatedStreamErrors*> ErrorsAttribute;
		
		void AddTimeoutErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors);
		void AddAuthorityConflictErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors);
		void AddSemanticErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors);
		void AddStreamCreationErrorWidget(TSharedRef<SVerticalBox> Result, const FAccumulatedStreamErrors& Errors);
	};
}

