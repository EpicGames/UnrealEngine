// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmissionNotifier.h"

#include "SAuthorityRejectedNotification.h"
#include "SStreamRejectedNotification.h"
#include "Replication/Client/Online/RemoteClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Submission/Data/AuthoritySubmission.h"
#include "Replication/Submission/Data/StreamSubmission.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Input/Reply.h"
#include "Stats/Stats.h"
#include "Widgets/Notifications/SNotificationList.h"

namespace UE::MultiUserClient::Replication
{
	FSubmissionNotifier::FSubmissionNotifier(FOnlineClientManager& InReplicationClientManager)
		: ReplicationClientManager(InReplicationClientManager)
	{
		ReplicationClientManager.OnPostRemoteClientAdded().AddRaw(this, &FSubmissionNotifier::OnPostRemoteClientAdded);
		RegisterClient(ReplicationClientManager.GetLocalClient());
	}

	FSubmissionNotifier::~FSubmissionNotifier()
	{
		CloseStreamNotification();
		CloseAuthorityNotification();
		
		// FSubmissionNotifier is usually owned by ReplicationClientManager so this clean-up is not really needed.
		// But we'll follow RAII here in case the ownership changes the future. 
		ReplicationClientManager.OnPostRemoteClientAdded().RemoveAll(this);
		ReplicationClientManager.OnPreRemoteClientRemoved().RemoveAll(this);
		
		UnregisterClient(ReplicationClientManager.GetLocalClient());
		for (const TNonNullPtr<FRemoteClient>& RemoteClient : ReplicationClientManager.GetRemoteClients())
		{
			UnregisterClient(*RemoteClient);
		}
	}

	void FSubmissionNotifier::Tick(float DeltaTime)
	{
		ProcessCompletedStreamChanges();
		ProcessCompletedAuthorityChanges();
	}

	TStatId FSubmissionNotifier::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FSubmissionNotifier, STATGROUP_Tickables);
	}

	void FSubmissionNotifier::OnPostRemoteClientAdded(FRemoteClient& RemoteReplicationClient)
	{
		RegisterClient(RemoteReplicationClient);
	}

	void FSubmissionNotifier::RegisterClient(FOnlineClient& Client)
	{
		Client.GetSubmissionWorkflow().OnStreamRequestCompleted_AnyThread().AddRaw(this, &FSubmissionNotifier::OnStreamRequestCompleted_AnyThread);
		Client.GetSubmissionWorkflow().OnAuthorityRequestCompleted_AnyThread().AddRaw(this, &FSubmissionNotifier::OnAuthorityRequestCompleted_AnyThread);
	}

	void FSubmissionNotifier::UnregisterClient(FOnlineClient& Client)
	{
		Client.GetSubmissionWorkflow().OnStreamRequestCompleted_AnyThread().RemoveAll(this);
		Client.GetSubmissionWorkflow().OnAuthorityRequestCompleted_AnyThread().RemoveAll(this);
	}

	void FSubmissionNotifier::OnStreamRequestCompleted_AnyThread(const FSubmitStreamChangesResponse& Response)
	{
		// Must be processed on the game thread because it might create a SNotification
		StreamRequestQueue.Enqueue(Response);
	}

	void FSubmissionNotifier::OnAuthorityRequestCompleted_AnyThread(const FSubmitAuthorityChangesRequest& Request, const FSubmitAuthorityChangesResponse& Response)
	{
		// Must be processed on the game thread because it might create a SNotification
		AuthorityRequestQueue.Enqueue({ Request, Response });
	}

	void FSubmissionNotifier::ProcessCompletedStreamChanges()
	{
		FSubmitStreamChangesResponse Response;
		while (StreamRequestQueue.Dequeue(Response))
		{
			AccumulateStreamRejections(Response);
		}
		
		CreateOrUpdateStreamNotification();
	}

	void FSubmissionNotifier::ProcessCompletedAuthorityChanges()
	{
		TPair<FSubmitAuthorityChangesRequest, FSubmitAuthorityChangesResponse> PendingPair;
		while (AuthorityRequestQueue.Dequeue(PendingPair))
		{
			AccumulateAuthorityRejections(PendingPair.Key, PendingPair.Value);
		}
		
		CreateOrUpdateAuthorityNotification();
	}

	void FSubmissionNotifier::AccumulateStreamRejections(const FSubmitStreamChangesResponse& CompletedOp)
	{
		switch(CompletedOp.ErrorCode)
		{
		case EStreamSubmissionErrorCode::Success:
			for (const TPair<FConcertObjectInStreamID, FConcertReplicatedObjectId>& Pair : CompletedOp.SubmissionInfo->Response.AuthorityConflicts)
			{
				StreamErrors.AuthorityConflicts.FindOrAdd(Pair.Key.Object, 0) += 1;
			}
			for (const TPair<FConcertObjectInStreamID, EConcertPutObjectErrorCode>& Pair : CompletedOp.SubmissionInfo->Response.ObjectsToPutSemanticErrors)
			{
				StreamErrors.SemanticErrors.FindOrAdd(Pair.Key.Object, 0) += 1;
			}
			StreamErrors.bFailedStreamCreation |= !CompletedOp.SubmissionInfo->Response.FailedStreamCreation.IsEmpty();
			break;
			
		case EStreamSubmissionErrorCode::Timeout:
			++StreamErrors.NumTimeouts;
			break;
			
		case EStreamSubmissionErrorCode::NoChange: break;
		case EStreamSubmissionErrorCode::Cancelled: break;
		default: ;
		}
	}

	void FSubmissionNotifier::AccumulateAuthorityRejections(const FSubmitAuthorityChangesRequest& RequestOp, const FSubmitAuthorityChangesResponse& ResponseOp)
	{
		switch (ResponseOp.ErrorCode)
		{
		case EAuthoritySubmissionResponseErrorCode::Success:
			for (const TPair<FSoftObjectPath, FConcertStreamArray>& Pair : ResponseOp.Response->RejectedObjects)
			{
				AuthorityErrors.Rejected.FindOrAdd(Pair.Key, 0) += 1;
			}
			break;
			
		case EAuthoritySubmissionResponseErrorCode::Timeout:
			++AuthorityErrors.NumTimeouts;
			break;
			
		case EAuthoritySubmissionResponseErrorCode::NoChange: break;
		case EAuthoritySubmissionResponseErrorCode::CancelledDueToStreamUpdate: break;
		case EAuthoritySubmissionResponseErrorCode::Cancelled: break;
		default: ;
		}
	}

	void FSubmissionNotifier::CreateOrUpdateStreamNotification()
	{
		if (!HasStreamRejections())
		{
			return;
		}

		if (StreamRejectedNotification)
		{
			StreamRejectedNotification->Refresh();
		}
		else
		{
			StreamRejectedNotification = SNew(SStreamRejectedNotification)
				.Errors_Lambda([this](){ return &StreamErrors; })
				.OnCloseClicked_Raw(this, &FSubmissionNotifier::CloseStreamNotification);
			FNotificationInfo Info(StreamRejectedNotification.ToSharedRef());
			Info.bFireAndForget = false;
			Info.ExpireDuration = 1.f;
			StreamNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
		}
	}

	void FSubmissionNotifier::CreateOrUpdateAuthorityNotification()
	{
		if (!HasAuthorityRejections())
		{
			return;
		}

		if (AuthorityRejectedNotification)
		{
			AuthorityRejectedNotification->Refresh();
		}
		else
		{
			AuthorityRejectedNotification = SNew(SAuthorityRejectedNotification)
				.Errors_Lambda([this](){ return &AuthorityErrors; })
				.OnCloseClicked_Raw(this, &FSubmissionNotifier::CloseAuthorityNotification);
			FNotificationInfo Info(AuthorityRejectedNotification.ToSharedRef());
			Info.bFireAndForget = false;
			Info.ExpireDuration = 1.f;
			AuthorityNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
		}
	}

	FReply FSubmissionNotifier::CloseStreamNotification()
	{
		if (StreamNotificationItem)
		{
			StreamRejectedNotification.Reset();
			StreamNotificationItem->ExpireAndFadeout();
			StreamNotificationItem.Reset();

			StreamErrors = {};
		}
		
		return FReply::Handled();
	}

	FReply FSubmissionNotifier::CloseAuthorityNotification()
	{
		if (AuthorityNotificationItem)
		{
			AuthorityRejectedNotification.Reset();
			AuthorityNotificationItem->ExpireAndFadeout();
			AuthorityNotificationItem.Reset();

			AuthorityErrors = {};
		}

		return FReply::Handled();
	}
}
