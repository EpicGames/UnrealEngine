// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AccumulatedSubmissionErrors.h"

#include "Containers/Queue.h"
#include "Templates/UnrealTemplate.h"
#include "TickableEditorObject.h"

class SNotificationItem;
class FReply;

namespace UE::MultiUserClient::Replication
{
	class FRemoteClient;
	class FOnlineClient;
	class FOnlineClientManager;
	
	class SAuthorityRejectedNotification;
	class SStreamRejectedNotification;
	
	struct FSubmitAuthorityChangesResponse;
	struct FSubmitAuthorityChangesRequest;
	struct FSubmitStreamChangesResponse;
	
	/**
	 * Adds a SNotificationItem when ISubmissionWorkflow operations fail.
	 * Failing operations are bundled into a single notification item until it is manually dismissed.
	 */
	class FSubmissionNotifier : public FNoncopyable, public FTickableEditorObject
	{
	public:
		
		FSubmissionNotifier(FOnlineClientManager& InReplicationClientManager);
		~FSubmissionNotifier();

		//~ Begin FTickableEditorObject Interface
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		//~ End FTickableEditorObject Interface

	private:

		/** Emits events when remote clients are discovered. */
		FOnlineClientManager& ReplicationClientManager;

		/** Only valid if changing streams has failed and the user has not yet dismissed the notification. */
		TSharedPtr<SNotificationItem> StreamNotificationItem;
		/** Only valid if changing authority has failed and the user has not yet dismissed the notification. */
		TSharedPtr<SNotificationItem> AuthorityNotificationItem;
		
		/** Wrapped by StreamNotification */
		TSharedPtr<SStreamRejectedNotification> StreamRejectedNotification;
		/** Wrapped by AuthorityNotification */
		TSharedPtr<SAuthorityRejectedNotification> AuthorityRejectedNotification;

		/** Displayed by StreamRejectedNotification */
		FAccumulatedStreamErrors StreamErrors;
		/** Displayed by AuthorityRejectedNotification */
		FAccumulatedAuthorityErrors AuthorityErrors;

		/** Queues stream completed requests since they may be completed from any thread (usually on game thread except if it times out). */
		TQueue<FSubmitStreamChangesResponse, EQueueMode::Mpsc> StreamRequestQueue;
		/** Queues authority completed requests since they may be completed from any thread (usually on game thread except if it times out). */
		TQueue<TPair<FSubmitAuthorityChangesRequest, FSubmitAuthorityChangesResponse>, EQueueMode::Mpsc> AuthorityRequestQueue;
		
		
		// Client events
		void OnPostRemoteClientAdded(FRemoteClient& RemoteReplicationClient);
		void RegisterClient(FOnlineClient& Client);
		void UnregisterClient(FOnlineClient& Client);

		// Per-client submission events
		void OnStreamRequestCompleted_AnyThread(const FSubmitStreamChangesResponse& Response);
		void OnAuthorityRequestCompleted_AnyThread(const FSubmitAuthorityChangesRequest& Request, const FSubmitAuthorityChangesResponse& Response);

		// Process asynchronously received completions
		void ProcessCompletedStreamChanges();
		void ProcessCompletedAuthorityChanges();

		// Handle rejections
		void AccumulateStreamRejections(const FSubmitStreamChangesResponse& CompletedOp);
		void AccumulateAuthorityRejections(const FSubmitAuthorityChangesRequest& RequestOp, const FSubmitAuthorityChangesResponse& ResponseOp);
		bool HasStreamRejections() const { return StreamErrors.NumTimeouts > 0 || !StreamErrors.AuthorityConflicts.IsEmpty() || !StreamErrors.SemanticErrors.IsEmpty() || StreamErrors.bFailedStreamCreation; }
		bool HasAuthorityRejections() const { return AuthorityErrors.NumTimeouts > 0 || !AuthorityErrors.Rejected.IsEmpty(); }

		// Updating the notifications
		void CreateOrUpdateStreamNotification();
		void CreateOrUpdateAuthorityNotification();

		// Button events
		FReply CloseStreamNotification();
		FReply CloseAuthorityNotification();
	};
}

