// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SubmissionQueue.h"
#include "Templates/UnrealTemplate.h"

namespace UE::MultiUserClient::Replication
{
	class ISubmissionWorkflow;
	
	/**
	 * Implemented by systems that want to generate a submission once the ISubmissionWorkflow is ready.
	 * This is an interface instead of a simple callback so the FSubmissionQueue can use the address of this object to avoid double enqueuing.
	 */
	class IDeferredSubmitter
	{
	public:

		/**
		 * Called when the ISubmissionWorkflow is ready.
		 *
		 * You are expected to call SubmitChanges but the queue handles the case if you don't (by dequeuing)
		 * but you should prefer to dequeue once you know no more changes are needed.
		 */
		virtual void PerformSubmission_GameThread(ISubmissionWorkflow& Workflow) = 0;

		virtual ~IDeferredSubmitter() = default;
	};

	/** Util for systems to auto-unregister when they are destroyed. */
	class FSelfUnregisteringDeferredSubmitter : public IDeferredSubmitter
	{
	protected:
		FSubmissionQueue& SubmissionQueue;
	public:

		FSelfUnregisteringDeferredSubmitter(FSubmissionQueue& InSubmissionQueue)
			: SubmissionQueue(InSubmissionQueue)
		{}
		
		virtual ~FSelfUnregisteringDeferredSubmitter()
		{
			SubmissionQueue.Dequeue_GameThread(*this);
		}
	};
}

