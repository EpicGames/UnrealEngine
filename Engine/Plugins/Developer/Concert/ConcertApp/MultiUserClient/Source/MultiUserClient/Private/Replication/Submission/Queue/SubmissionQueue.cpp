// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmissionQueue.h"

#include "DeferredSubmitter.h"
#include "Replication/Submission/ISubmissionWorkflow.h"

#include "Stats/Stats.h"

namespace UE::MultiUserClient::Replication
{
	FSubmissionQueue::FSubmissionQueue(ISubmissionWorkflow& InWorkflow)
		: Workflow(InWorkflow)
	{
		Workflow.OnSubmitOperationCompleted_AnyThread().AddRaw(this, &FSubmissionQueue::OnSubmissionCompleted);
	}

	FSubmissionQueue::~FSubmissionQueue()
	{
		Workflow.OnSubmitOperationCompleted_AnyThread().RemoveAll(this);
	}

	void FSubmissionQueue::SubmitNowOrEnqueue_GameThread(IDeferredSubmitter& DeferredSubmission)
	{
		if (!ensure(IsInGameThread()))
		{
			return;
		}

		if (Workflow.CanSubmit())
		{
			DeferredSubmission.PerformSubmission_GameThread(Workflow);
		}
		else if (!Queue.Contains(&DeferredSubmission))
		{
			Queue.AddTail(&DeferredSubmission);
		}
	}

	void FSubmissionQueue::Dequeue_GameThread(IDeferredSubmitter& DeferredSubmission)
	{
		if (!IsInGameThread())
		{
			// It is imaginable that some async system (like a request) pushes a task to the game thread which is supposed to call Dequeue_GameThread.
			// Since the engine is shutting down, never the task would never actually be executed.
			// However, the underlying system is still destroyed, even if living on another thread, since the engine is shutting down.
			ensureMsgf(IsEngineExitRequested(), TEXT("Called outside of game thread but not a problem if engine is shutting down. Check your destruction logic."));
			return;
		}
		
		Queue.RemoveNode(&DeferredSubmission);
	}

	void FSubmissionQueue::Tick(float DeltaTime)
	{
		if (bTaskFinishedOutsideGameThread)
		{
			ProcessNextInQueue();
		}
	}

	TStatId FSubmissionQueue::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FSubmissionNotifier, STATGROUP_Tickables);
	}

	void FSubmissionQueue::OnSubmissionCompleted()
	{
		if (IsInGameThread())
		{
			ProcessNextInQueue();
		}
		else
		{
			bTaskFinishedOutsideGameThread = true;
		}
	}

	void FSubmissionQueue::ProcessNextInQueue()
	{
		check(IsInGameThread());
		bTaskFinishedOutsideGameThread = false;
		
		if (!Queue.GetHead())
		{
			return;
		}

		IDeferredSubmitter* Submitter = Queue.GetHead()->GetValue();
		Queue.RemoveNode(Queue.GetHead());
		Submitter->PerformSubmission_GameThread(Workflow);

		// PerformSubmission_GameThread did nothing? Progress the queue to avoid infinite waiting.
		if (Workflow.CanSubmit())
		{
			ProcessNextInQueue();
		}
	}
}
