// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "Templates/UnrealTemplate.h"
#include "TickableEditorObject.h"

namespace UE::MultiUserClient::Replication
{
	class IDeferredSubmitter;
	class ISubmissionWorkflow;
	
	/**
	 * The submission queue is used to enforce a deterministic queue for calling ISubmissionWorkflow::SubmitChanges.
	 * 
	 * ISubmissionWorkflow only allows at most one pending change in progress at a time.
	 * Hence, if one is in progress you must wait. This queue implements shared waiting logic.
	 *
	 * Since submissions are not real-time sensitive and to simplify concurrency design, the queue will always call
	 * IDeferredSubmission::PerformSubmission_GameThread on the game thread.
	 * This is not a big restriction either: usually requests complete on the game thread.
	 * Only timeouts take place on the UDP thread, in which case we synchronize.
	 */
	class FSubmissionQueue : public FNoncopyable, public FTickableEditorObject
	{
	public:
		
		/**
		 * @param InWorkflow Used to dispatch requests. The caller ensures it outlives the constructed instance.
		 */
		FSubmissionQueue(ISubmissionWorkflow& InWorkflow);
		virtual ~FSubmissionQueue();

		/**
		 * Enqueues a submission.
		 *
		 * For convenience, enqueueing the same object multiple times is ignored.
		 * Suppose you call Enqueue(A), Enqueue(B), Enqueue(A), then the queue will look like this A > B.
		 *
		 * @note You are responsible for calling Dequeue_GameThread when your IDeferredSubmission is destroyed.
		 * @see FSelfUnregisteringDeferredSubmission
		 */
		void SubmitNowOrEnqueue_GameThread(IDeferredSubmitter& DeferredSubmission);

		/** Dequeues a latent submission. It does not need to executed anymore. */
		void Dequeue_GameThread(IDeferredSubmitter& DeferredSubmission);

		//~ Begin FTickableEditorObject Interface
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		//~ End FTickableEditorObject Interface

	private:

		/** Implements the queue. The head is the next to execute. Used instead of TQueue because TQueue does not support removing. */
		TDoubleLinkedList<IDeferredSubmitter*> Queue;

		/**
		 * The workflow that is being enqueued for.
		 * Outlives this object.
		 */
		ISubmissionWorkflow& Workflow;

		/** Set to true when a task finishes outside of the game thread. Written exactly once by other threads other than game thread. Read by game thread. */
		bool bTaskFinishedOutsideGameThread = false;

		void OnSubmissionCompleted();
		void ProcessNextInQueue();
	};
}

