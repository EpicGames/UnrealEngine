// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSubmissionListener.h"

#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "RemoteSubmissionMessages.h"
#include "Replication/MultiUserReplicationCVars.h"
#include "Replication/Client/ClientUtils.h"
#include "Replication/Stream/IClientStreamSynchronizer.h"
#include "Replication/Submission/ISubmissionOperation.h"
#include "Replication/Submission/ISubmissionWorkflow.h"
#include "Replication/Submission/Queue/DeferredSubmitter.h"

namespace UE::MultiUserClient::Replication
{
	FRemoteSubmissionListener::FRemoteSubmissionListener(
		TSharedRef<IConcertClientSession> InConcertSession,
		IClientStreamSynchronizer& InStreamSynchronizer,
		FSubmissionQueue& InSubmissionQueue
		)
		: FSelfUnregisteringDeferredSubmitter(InSubmissionQueue)
		, ConcertSession(MoveTemp(InConcertSession))
		, StreamSynchronizer(InStreamSynchronizer)
		, SubmissionQueue(InSubmissionQueue)
	{
		ConcertSession->RegisterCustomRequestHandler<FMultiUser_ChangeRemote_Request, FMultiUser_ChangeRemote_Response>(this, &FRemoteSubmissionListener::HandleChangeRemoteRequest);
	}

	FRemoteSubmissionListener::~FRemoteSubmissionListener()
	{
		ConcertSession->UnregisterCustomRequestHandler<FMultiUser_ChangeRemote_Request>();
	}

	EConcertSessionResponseCode FRemoteSubmissionListener::HandleChangeRemoteRequest(
		const FConcertSessionContext& Context,
		const FMultiUser_ChangeRemote_Request& Request,
		FMultiUser_ChangeRemote_Response& Response
		)
	{
		UE_LOGF(LogConcert, Log, "Received remote change request from %ls", *ClientUtils::GetClientDisplayName(*ConcertSession, Context.SourceEndpointId));
		LogNetworkMessage(CVarLogRemoteChangeRequestsAndResponses, Request);
		ON_SCOPE_EXIT{ LogNetworkMessage(CVarLogRemoteChangeRequestsAndResponses, Response); };
		Response.bTimeout = false;

		if (InProgressOperation.IsSet())
		{
			Response.Error = EMultiUserChangeRemoteRequestError::OtherInProgress;
			return EConcertSessionResponseCode::Failed;
		}
		
		if (!CanProcessRequest(Request))
		{
			Response.Error = EMultiUserChangeRemoteRequestError::RejectedChange;
			return EConcertSessionResponseCode::Failed;
		}
		
		InProgressOperation = { Request, Context.SourceEndpointId };
		SubmissionQueue.SubmitNowOrEnqueue_GameThread(*this);
		return EConcertSessionResponseCode::Success;
	}

	void FRemoteSubmissionListener::PerformSubmission_GameThread(ISubmissionWorkflow& Workflow)
	{
		using namespace ConcertSyncClient::Replication;

		const FMultiUser_ChangeRemote_Request& RequestData = InProgressOperation->Request;
		if (!CanProcessRequest(RequestData))
		{
			CancelOperation(EMultiUserCancelRemoteChangeReason::PredictedConflict);
			return;
		}

		const bool bHasStreamOnServer = !StreamSynchronizer.GetServerState().ReplicatedObjects.IsEmpty();
		const bool bIsAddingStream = !RequestData.StreamChangeRequest.StreamsToAdd.IsEmpty();
		const bool bHasStreamCreationConflict = (bHasStreamOnServer && bIsAddingStream) || (!bHasStreamOnServer && !bIsAddingStream);
		if (bHasStreamCreationConflict)
		{
			// Problem: Requesting client and local client disagree about whether the local client already has a registered stream. In theory we can resolve this but for this WIP we won't.
			CancelOperation(EMultiUserCancelRemoteChangeReason::PredictedConflict);
			return;
		}
		
		const TSharedPtr<ISubmissionOperation> Operation = Workflow.SubmitChanges({
			FConcertReplication_ChangeStream_Request { RequestData.StreamChangeRequest }, FConcertReplication_ChangeAuthority_Request{ RequestData.AuthorityRequest } }
			);
		if (!ensure(Operation))
		{
			CancelOperation(EMultiUserCancelRemoteChangeReason::FailedToCreate);
			return;
		}

		if (!RequestData.StreamChangeRequest.IsEmpty())
		{
			Operation->OnCompleteStreamChangesFuture_AnyThread().Next([this, Token = LifetimeToken.ToWeakPtr()](FSubmitStreamChangesResponse&& Response)
			{
				if (const TSharedPtr<FToken> TokenPin = Token.Pin())
				{
					HandleStreamChangeDone(Response);
				}
			});
		}
		if (!RequestData.AuthorityRequest.IsEmpty())
		{
			Operation->OnCompleteAuthorityChangeFuture_AnyThread().Next([this, Token = LifetimeToken.ToWeakPtr()](FSubmitAuthorityChangesResponse&& Response)
			{
				if (const TSharedPtr<FToken> TokenPin = Token.Pin())
				{
					HandleAuthorityChangeDone(Response);
				}
			});
		}
	}

	void FRemoteSubmissionListener::HandleStreamChangeDone(const FSubmitStreamChangesResponse& Response)
	{
		const bool bProcessed = Response.ErrorCode == EStreamSubmissionErrorCode::Success && ensure(Response.SubmissionInfo.IsSet());
		FMultiUser_ChangeRemote_StreamUpdatedEvent ProgressEvent { bProcessed };
		if (bProcessed)
		{
			ProgressEvent.StreamChangeResponse = Response.SubmissionInfo->Response;
		}
		
		InProgressOperation->bFinishedStream = true;
		ConcertSession->SendCustomEvent(ProgressEvent, InProgressOperation->RequestingEndpointId, EConcertMessageFlags::ReliableOrdered);
		FinishOperationIfDone();
	}

	void FRemoteSubmissionListener::HandleAuthorityChangeDone(const FSubmitAuthorityChangesResponse& Response)
	{
		const bool bProcessed = Response.ErrorCode == EAuthoritySubmissionResponseErrorCode::Success && ensure(Response.Response.IsSet());
		FMultiUser_ChangeRemote_AuthorityUpdatedEvent ProgressEvent { bProcessed };
		if (bProcessed)
		{
			ProgressEvent.AuthorityChangeResponse = *Response.Response;
		}

		InProgressOperation->bFinishedAuthority = true;
		ConcertSession->SendCustomEvent(ProgressEvent, InProgressOperation->RequestingEndpointId, EConcertMessageFlags::ReliableOrdered);
		FinishOperationIfDone();
	}

	void FRemoteSubmissionListener::FinishOperationIfDone()
	{
		const bool bIsStreamDone = InProgressOperation->bFinishedStream || InProgressOperation->Request.StreamChangeRequest.IsEmpty();
		const bool bIsAuthorityDone = InProgressOperation->bFinishedAuthority || InProgressOperation->Request.AuthorityRequest.IsEmpty();
		if (bIsStreamDone && bIsAuthorityDone)
		{
			CleanUpOperation();
		}
	}

	bool FRemoteSubmissionListener::CanProcessRequest(const FMultiUser_ChangeRemote_Request& Request)
	{
		// TODO UE-201118: Are these changes free of any authority conflicts?
		return true;
	}

	void FRemoteSubmissionListener::CancelOperation(EMultiUserCancelRemoteChangeReason Reason)
	{
		ConcertSession->SendCustomEvent(FMultiUser_ChangeRemote_Cancelled{ Reason }, InProgressOperation->RequestingEndpointId, EConcertMessageFlags::ReliableOrdered);
		CleanUpOperation();
	}

	void FRemoteSubmissionListener::CleanUpOperation()
	{
		InProgressOperation.Reset();
	}
}
