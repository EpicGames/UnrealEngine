// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSubmissionMessages.h"
#include "Replication/IToken.h"
#include "Replication/Submission/Queue/DeferredSubmitter.h"

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"


class IConcertClientSession;

enum class EConcertSessionResponseCode : uint8;

struct FConcertSessionContext;

namespace UE::MultiUserClient::Replication
{
	class IClientStreamSynchronizer;
	struct FSubmitAuthorityChangesResponse;
	struct FSubmitStreamChangesResponse;
	
	/**
	 * Handles stream and authority requests made by other clients towards the local client.
	 *
	 * For security reasons, the server only allows clients to change their own streams. Hence if client 1 wants to change something about client 2,
	 * client 1 must kindly ask client 2 to do the change. In this case client 1's FSubmissionWorkflow_RemoteClient does the asking and client 2's
	 * FRemoteSubmissionListener the answering.
	 * 
	 * The typical source of such requests is FSubmissionWorkflow_RemoteClient though theoretically the request can from any system with access
	 * to a Concert endpoint.
	 *
	 * @see FSubmissionWorkflow_RemoteClient
	 */
	class FRemoteSubmissionListener
		: public FNoncopyable
		, FSelfUnregisteringDeferredSubmitter
	{
	public:
		
		FRemoteSubmissionListener(TSharedRef<IConcertClientSession> InConcertSession, IClientStreamSynchronizer& InStreamSynchronizer, FSubmissionQueue& InSubmissionQueue);
		~FRemoteSubmissionListener();

	private:
		
		/** Latent operations keep a weak reference to this to detect whether this FSubmissionWorkflow_LocalClient was destroyed. */
		const TSharedRef<FToken> LifetimeToken = FToken::Make();

		/** Used to register and unregister event handlers. This object's owner is supposed to destroy us before the session is destroyed. */
		TSharedRef<IConcertClientSession> ConcertSession;

		/** Used to validate a request against the client's server state. */
		IClientStreamSynchronizer& StreamSynchronizer;
		/** Used to serve the incoming change requests. */
		FSubmissionQueue& SubmissionQueue;

		struct FOperationData
		{
			const FMultiUser_ChangeRemote_Request Request;
			const FGuid RequestingEndpointId;

			bool bFinishedStream = false;
			bool bFinishedAuthority = false;
		};
		/** Data for current in progress operation */
		TOptional<FOperationData> InProgressOperation;

		/** Handles remote change event. */
		EConcertSessionResponseCode HandleChangeRemoteRequest(const FConcertSessionContext& Context, const FMultiUser_ChangeRemote_Request& Request, FMultiUser_ChangeRemote_Response& Response);
		
		/** Checks whether the local submit operation is ready. If not, we subscribe to it finishing and try again. */
		void SubmitNowOrWaitUntilReady(int32 NumTriesSoFar = 0);

		//~ Begin IDeferredSubmission Interface
		virtual void PerformSubmission_GameThread(ISubmissionWorkflow& Workflow) override;
		//~ End IDeferredSubmission Interface

		/** The submission workflow is ready for submission. Do the submission. */
		void SubmitRequest();

		void HandleStreamChangeDone(const FSubmitStreamChangesResponse& Response);
		void HandleAuthorityChangeDone(const FSubmitAuthorityChangesResponse& Response);
		void FinishOperationIfDone();

		/** @return Whether this remote request should be processed. Returns false if there are any authority conflicts. */
		bool CanProcessRequest(const FMultiUser_ChangeRemote_Request& Request);

		/** Sends a cancel event to the requester and cleans up the operation.*/
		void CancelOperation(EMultiUserCancelRemoteChangeReason Reason);
		/** Resets InProgressOperation and does any other clean-up. */
		void CleanUpOperation();
	};
}

