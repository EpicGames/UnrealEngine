// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Messages/ChangeStream.h"
#include "RemoteSubmissionMessages.generated.h"

// IMPORTANT: This peer-to-peer workflow is subject to be replaced, possibly in 5.5, by a server feature.
// The peer-to-peer approach cannot guarantee atomicity if client A is trying to edit clients B and C as one operation: How to revert when B succeeds and C fails? You can't.
// TODO UE-201136.

/** A kind request from one client to another client to change their server registered state. The receiver can reject the request. */
USTRUCT()
struct FMultiUser_ChangeRemote_Request
{
	GENERATED_BODY()

	/** Stream changes to apply. Applied before authority changes. */
	UPROPERTY()
	FConcertReplication_ChangeStream_Request StreamChangeRequest;

	/** Authority changes to apply. Applied after stream changes. */
	UPROPERTY()
	FConcertReplication_ChangeAuthority_Request AuthorityRequest;
};

UENUM()
enum class EMultiUserChangeRemoteRequestError : uint8
{
	/** The requests will be executed. */
	Accepted,

	/** The client rejected the change. */
	RejectedChange,
	
	/** The request was invalid when it was received because the client determined that the change would generate a conflict given the client's known server state. */
	PredictedConflict,
	/** The change was rejected because the client was progressing a request from another client. */
	OtherInProgress
};

/** Indicates whether the request was enqueued. */
USTRUCT()
struct FMultiUser_ChangeRemote_Response
{
	GENERATED_BODY()

	/** Default result for default constructed response. Concert default constructs timed out responses. */
	UPROPERTY()
	bool bTimeout = true;
	
	/** Indicates whether the request was enqueued */
	UPROPERTY()
	EMultiUserChangeRemoteRequestError Error = EMultiUserChangeRemoteRequestError::Accepted;
};

/** Sent by serving client to the requesting client to confirm completion of the stream part of the request. */
USTRUCT()
struct FMultiUser_ChangeRemote_StreamUpdatedEvent
{
	GENERATED_BODY()

	/** Whether the change was submitted to the server. False indicates that there was some kind of submission failure. True does NOT imply that StreamChangeResponse did not fail. */
	UPROPERTY()
	bool bProcessedSuccessfully = true;

	/** Set if bProcessedSuccessfully == true. */
	UPROPERTY()
	FConcertReplication_ChangeStream_Response StreamChangeResponse;
};

/** Sent by serving client to the requesting client to confirm completion of the authority part of the request. */
USTRUCT()
struct FMultiUser_ChangeRemote_AuthorityUpdatedEvent
{
	GENERATED_BODY()
	
	/** Whether the change was submitted to the server. False indicates that there was some kind of submission failure. True does NOT imply that AuthorityChangeResponse did not fail. */
	UPROPERTY()
	bool bProcessedSuccessfully = true;

	/** Set if bProcessedSuccessfully == true. */
	UPROPERTY()
	FConcertReplication_ChangeAuthority_Response AuthorityChangeResponse;
};

UENUM()
enum class EMultiUserCancelRemoteChangeReason : uint8
{
	/**
	 * Internal error: SubmitChanges failed.
	 * No changes were made.
	 */
	FailedToCreate,

	/**
	 * The serving client had to wait with executing the event and after so doing determined that the request would a conflict. 
	 * No changes were made.
	 */
	PredictedConflict
};

/** Send by serving client to the requesting client to cancel the request after it was confirmed. */
USTRUCT()
struct FMultiUser_ChangeRemote_Cancelled
{
	GENERATED_BODY()

	UPROPERTY()
	EMultiUserCancelRemoteChangeReason Reason = EMultiUserCancelRemoteChangeReason::FailedToCreate;
};