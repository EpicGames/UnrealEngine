// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmissionWorkflow_RemoteClient.h"

#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "RemoteSubmissionMessages.h"
#include "Replication/Client/ClientUtils.h"
#include "Replication/Misc/Util/StreamRequestUtils.h"
#include "Replication/Submission/SingleClientSubmissionOperation.h"

namespace UE::MultiUserClient::Replication
{
	FSubmissionWorkflow_RemoteClient::FSubmissionWorkflow_RemoteClient(
		TSharedRef<IConcertClientSession> InConcertSession,
		const FGuid& InRemoteClientEndpointId
		)
		: RemoteClientEndpointId(InRemoteClientEndpointId)
		, ConcertSession(MoveTemp(InConcertSession))
	{
		// The other client will update as progress is made.
		// Note that handles cannot be registered & unregistered dynamically while the handler is executing (that would change the handler array while it is being iterated).
		ConcertSession->RegisterCustomEventHandler<FMultiUser_ChangeRemote_StreamUpdatedEvent>(this, &FSubmissionWorkflow_RemoteClient::OnStreamRemoteChangeEvent);
		ConcertSession->RegisterCustomEventHandler<FMultiUser_ChangeRemote_AuthorityUpdatedEvent>(this, &FSubmissionWorkflow_RemoteClient::OnAuthorityRemoteChangeEvent);
	}

	FSubmissionWorkflow_RemoteClient::~FSubmissionWorkflow_RemoteClient()
	{
		ConcertSession->UnregisterCustomEventHandler<FMultiUser_ChangeRemote_StreamUpdatedEvent>(this);
		ConcertSession->UnregisterCustomEventHandler<FMultiUser_ChangeRemote_AuthorityUpdatedEvent>(this);
		// InProgressOperation destructor chain execute any pending promises
	}

	TSharedPtr<ISubmissionOperation> FSubmissionWorkflow_RemoteClient::SubmitChanges(FSubmissionParams Params)
	{
		if (Params.IsEmpty() || !CanSubmit())
		{
			return nullptr;
		}
		
		const bool bIsModifyingStream = !Params.IsStreamChangeEmpty();
		const TSharedRef<FSingleClientSubmissionOperation> OperationResult = MakeShared<FSingleClientSubmissionOperation>(bIsModifyingStream);

		// Since SendCustomRequest can return immediately, emplace relevant promises before
		EarlyCompletePromisesForUnchangedData(Params, OperationResult);
		const FMultiUser_ChangeRemote_Request Request
		{
			Params.StreamRequest ? *Params.StreamRequest : FConcertReplication_ChangeStream_Request{},
			Params.AuthorityRequest ? *Params.AuthorityRequest : FConcertReplication_ChangeAuthority_Request{},
		};
		InProgressOperation = { MoveTemp(Params), OperationResult };
		
		ConcertSession->SendCustomRequest<FMultiUser_ChangeRemote_Request, FMultiUser_ChangeRemote_Response>(Request, RemoteClientEndpointId)
			.Next([this, WeakToken = LifetimeToken.ToWeakPtr()](FMultiUser_ChangeRemote_Response&& Response)
			{
				if (WeakToken.Pin())
				{
					ProcessSubmissionResponse(Response);
				}
			});
		
		return OperationResult;
	}

	EChangeUploadability FSubmissionWorkflow_RemoteClient::GetUploadability() const
	{
		if (InProgressOperation.IsSet())
		{
			return EChangeUploadability::InProgress;
		}

		// TODO DP UE-200927: Figure out whether remote client is configured to reject changes
		return EChangeUploadability::Ready;
	}

	void FSubmissionWorkflow_RemoteClient::ProcessSubmissionResponse(const FMultiUser_ChangeRemote_Response& Response)
	{
		if (Response.bTimeout)
		{
			TimeoutAndClean();
		}
		else if (Response.Error != EMultiUserChangeRemoteRequestError::Accepted)
		{
			HandleFailureResponse(Response.Error);
		}

		UE_LOGF(LogConcert, Log, "Remote client %ls accepted change request", *GetRemoteClientName());
	}
	
	void FSubmissionWorkflow_RemoteClient::TimeoutAndClean()
	{
		UE_LOGF(LogConcert, Warning, "Change request to client %ls timed out", *GetRemoteClientName());
		
		TimeoutStreamChangeIfUnset();
		TimeoutAuthorityChangeIfUnset();
		CleanUpSubmissionProcess();
	}

	void FSubmissionWorkflow_RemoteClient::TimeoutStreamChangeIfUnset()
	{
		if (!InProgressOperation->Parameters.IsStreamChangeEmpty())
		{
			const FSubmitStreamChangesResponse StreamResponse { EStreamSubmissionErrorCode::Timeout };
			GetOperation().EmplaceStreamPromise(StreamResponse);
			StreamRequestCompletedDelegate.Broadcast(StreamResponse);
		}
	}

	void FSubmissionWorkflow_RemoteClient::TimeoutAuthorityChangeIfUnset()
	{
		if (!InProgressOperation->ExposedOperation->HasSetAuthorityResponsePromise())
		{
			// Sending authority request was technically successful.
			const FSubmitAuthorityChangesRequest AuthorityRequest { EAuthoritySubmissionRequestErrorCode::Success, InProgressOperation->Parameters.AuthorityRequest };
			const FSubmitAuthorityChangesResponse AuthorityResponse { EAuthoritySubmissionResponseErrorCode::Timeout };
			GetOperation().EmplaceAuthorityResponsePromise(AuthorityResponse);
			AuthorityRequestCompletedDelegate.Broadcast(AuthorityRequest, AuthorityResponse);
		}
	}

	void FSubmissionWorkflow_RemoteClient::HandleFailureResponse(EMultiUserChangeRemoteRequestError ErrorFlags)
	{
		UE_LOGF(LogConcert, Warning, "Change request to client %ls failed (error flags: %d)", *GetRemoteClientName(), static_cast<int32>(ErrorFlags));
		
		switch (ErrorFlags)
		{
		case EMultiUserChangeRemoteRequestError::PredictedConflict:
			// TODO DP UE-200926: Retry when remote client predicts conflict, if applicable
		case EMultiUserChangeRemoteRequestError::OtherInProgress:
			// TODO DP UE-200924: Retry when client is busy serving another request
		case EMultiUserChangeRemoteRequestError::RejectedChange:
			TimeoutStreamChangeIfUnset();
			TimeoutAuthorityChangeIfUnset();
			CleanUpSubmissionProcess();
			break;
			
		case EMultiUserChangeRemoteRequestError::Accepted:
		default:
			checkNoEntry();
		}
	}

	void FSubmissionWorkflow_RemoteClient::CleanUpSubmissionProcess()
	{
		if (InProgressOperation)
		{
			const TSharedPtr<FSingleClientSubmissionOperation> ExposedOperation = InProgressOperation->ExposedOperation;
			InProgressOperation.Reset();
		
			ExposedOperation->EmplaceCompleteOperationPromise(ESubmissionOperationCompletedCode::Processed);
			OnSubmitOperationCompletedDelegate.Broadcast();
		}
	}
	
	void FSubmissionWorkflow_RemoteClient::EarlyCompletePromisesForUnchangedData(const FSubmissionParams& Params, const TSharedRef<FSingleClientSubmissionOperation>& OperationResult) const
	{
		// Complete stream event if nothing is being changed
		if (Params.IsStreamChangeEmpty())
		{
			const FSubmitStreamChangesResponse StreamResponse { EStreamSubmissionErrorCode::NoChange };
			OperationResult->EmplaceStreamPromise(StreamResponse);
			StreamRequestCompletedDelegate.Broadcast(StreamResponse);
		}

		// Complete the authority promise if nothing is being changed
		if (Params.IsAuthorityChangeEmpty())
		{
			FSubmitAuthorityChangesRequest RequestEvent { EAuthoritySubmissionRequestErrorCode::NoChange, Params.AuthorityRequest };
			FSubmitAuthorityChangesResponse ResponseEvent { EAuthoritySubmissionResponseErrorCode::NoChange };
			OperationResult->EmplaceAuthorityRequestPromise(RequestEvent);
			OperationResult->EmplaceAuthorityResponsePromise(ResponseEvent);
			AuthorityRequestCompletedDelegate.Broadcast(RequestEvent, ResponseEvent);
		}
		else
		{
			// Opposed to the local case, sending authority request is sent instantly in the remote case, so emplace the promise now.
			OperationResult->EmplaceAuthorityRequestPromise({ EAuthoritySubmissionRequestErrorCode::Success, Params.AuthorityRequest });
		}
	}

	void FSubmissionWorkflow_RemoteClient::OnStreamRemoteChangeEvent(const FConcertSessionContext& Context, const FMultiUser_ChangeRemote_StreamUpdatedEvent& EventData)
	{
		// Might receive from other clients due to other registered FSubmissionWorkflow_RemoteClients on this same editor instance
		if (!InProgressOperation.IsSet() ||Context.SourceEndpointId != RemoteClientEndpointId)
		{
			return;
		}

		FSingleClientSubmissionOperation& Operation = GetOperation();
		if (EventData.bProcessedSuccessfully)
		{
			const FCompletedChangeSubmission StreamChangeSubmission { *InProgressOperation->Parameters.StreamRequest, { EventData.StreamChangeResponse } };
			const FSubmitStreamChangesResponse StreamResponse { EStreamSubmissionErrorCode::Success, StreamChangeSubmission };
			Operation.EmplaceStreamPromise(StreamResponse);
			StreamRequestCompletedDelegate.Broadcast(StreamResponse);
		}
		else
		{
			const FSubmitStreamChangesResponse StreamResponse{ EStreamSubmissionErrorCode::Timeout };
			Operation.EmplaceStreamPromise(StreamResponse);
			StreamRequestCompletedDelegate.Broadcast(StreamResponse);
		}
		
		// TODO UE-200925: Display better error handling to user
		UE_CLOGF(!EventData.bProcessedSuccessfully, LogConcert, Error, "Remote stream change to client %ls failed", *GetRemoteClientName());
		FinishSubmissionIfDone();
	}

	void FSubmissionWorkflow_RemoteClient::OnAuthorityRemoteChangeEvent(const FConcertSessionContext& Context, const FMultiUser_ChangeRemote_AuthorityUpdatedEvent& EventData)
	{
		// Might receive from other clients due to other registered FSubmissionWorkflow_RemoteClients on this same editor instance
		if (!InProgressOperation.IsSet() || Context.SourceEndpointId != RemoteClientEndpointId)
		{
			return;
		}
		
		FSingleClientSubmissionOperation& Operation = GetOperation();
		const FSubmitAuthorityChangesRequest AuthorityRequest { EAuthoritySubmissionRequestErrorCode::Success, InProgressOperation->Parameters.AuthorityRequest };
		if (EventData.bProcessedSuccessfully)
		{
			const FConcertReplication_ChangeAuthority_Response ReceivedResponse { EventData.AuthorityChangeResponse };
			const FSubmitAuthorityChangesResponse AuthorityResponse { EAuthoritySubmissionResponseErrorCode::Success, ReceivedResponse };
			Operation.EmplaceAuthorityResponsePromise(AuthorityResponse);
			AuthorityRequestCompletedDelegate.Broadcast(AuthorityRequest, AuthorityResponse);
		}
		else
		{
			const FSubmitAuthorityChangesResponse AuthorityResponse { EAuthoritySubmissionResponseErrorCode::Timeout };
			Operation.EmplaceAuthorityResponsePromise(AuthorityResponse);
			AuthorityRequestCompletedDelegate.Broadcast(AuthorityRequest, AuthorityResponse);
		}

		FinishSubmissionIfDone();
	}

	void FSubmissionWorkflow_RemoteClient::FinishSubmissionIfDone()
	{
		FSingleClientSubmissionOperation& Operation = GetOperation();
		const bool bIsStreamChangeDone = Operation.HasSetStreamPromise() || InProgressOperation->Parameters.StreamRequest->IsEmpty();
		const bool bIsAuthorityChangeDone = Operation.HasSetAuthorityResponsePromise() || InProgressOperation->Parameters.AuthorityRequest->IsEmpty();
		if (bIsStreamChangeDone && bIsAuthorityChangeDone)
		{
			CleanUpSubmissionProcess();
		}
	}

	FString FSubmissionWorkflow_RemoteClient::GetRemoteClientName() const
	{
		return ClientUtils::GetClientDisplayName(*ConcertSession, RemoteClientEndpointId);
	}
}
