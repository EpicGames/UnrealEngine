// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertSessionHandler.h"
#include "RemoteSubmissionMessages.h"
#include "Replication/IToken.h"
#include "Replication/Submission/ISubmissionWorkflow.h"

#include "Misc/Optional.h"
#include "Templates/UnrealTemplate.h"

class IConcertClientSession;
class IConcertSyncClient;

enum class EConcertClientStatus : uint8;

namespace UE::MultiUserClient::Replication
{
	class FSingleClientSubmissionOperation;
	
	/**
	 * Sends change requests to a remote client, whose FRemoteSubmissionListener will handle the requests.
	 * @see FRemoteSubmissionListener
	 */
	class FSubmissionWorkflow_RemoteClient
		: public FSubmissionWorkflowBase
		, public FNoncopyable
	{
	public:
		
		FSubmissionWorkflow_RemoteClient(TSharedRef<IConcertClientSession> InConcertSession, const FGuid& InRemoteClientEndpointId);
		virtual ~FSubmissionWorkflow_RemoteClient() override;

		//~ Begin ISubmissionWorkflow Interface
		virtual TSharedPtr<ISubmissionOperation> SubmitChanges(FSubmissionParams Params) override;
		virtual EChangeUploadability GetUploadability() const override;
		//~ End ISubmissionWorkflow Interface

	private:
		
		/** Latent operations keep a weak reference to this to detect whether this FSubmissionWorkflow_LocalClient was destroyed. */
		const TSharedRef<FToken> LifetimeToken = FToken::Make();

		/** Id of the remote client that requests are sent to. */
		const FGuid RemoteClientEndpointId;
		
		/** Used to sending messages */
		const TSharedRef<IConcertClientSession> ConcertSession;

		struct FOperationData
		{
			/** Parameters the operation was started with. */
			const FSubmissionParams Parameters;
			/** Automatically cancels pending promises when destroyed. */
			const TSharedRef<FSingleClientSubmissionOperation> ExposedOperation;
		};
		/** Set for as long as there is a SubmitChanges operation in progress. */
		TOptional<FOperationData> InProgressOperation;

		/** Handles a response from a remote client. */
		void ProcessSubmissionResponse(const FMultiUser_ChangeRemote_Response& Response);
		
		/** Handles timeout of submission response. */
		void TimeoutAndClean();
		void TimeoutStreamChangeIfUnset();
		void TimeoutAuthorityChangeIfUnset();
		
		/** Handles a response from the client that indicates failure. */
		void HandleFailureResponse(EMultiUserChangeRemoteRequestError ErrorFlags);
		/** Cleans up side effects from last submission process */
		void CleanUpSubmissionProcess();

		FSingleClientSubmissionOperation& GetOperation() { return *InProgressOperation->ExposedOperation; }

		/** Checks which data is not being changed by Params and completes the corresponding promises on OperationResult. */
		void EarlyCompletePromisesForUnchangedData(const FSubmissionParams& Params, const TSharedRef<FSingleClientSubmissionOperation>& OperationResult) const;
		
		/** Called by remote client to update us about progress. */
		void OnStreamRemoteChangeEvent(const FConcertSessionContext& Context, const FMultiUser_ChangeRemote_StreamUpdatedEvent& EventData);
		void OnAuthorityRemoteChangeEvent(const FConcertSessionContext& Context, const FMultiUser_ChangeRemote_AuthorityUpdatedEvent& EventData);
		void FinishSubmissionIfDone();

		FString GetRemoteClientName() const;
	};
}

