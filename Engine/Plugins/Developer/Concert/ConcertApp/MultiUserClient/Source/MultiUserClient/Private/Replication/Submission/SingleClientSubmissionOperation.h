// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertLogGlobal.h"
#include "ISubmissionOperation.h"

#include "Templates/UnrealTemplate.h"

namespace UE::MultiUserClient::Replication
{
	/** Keeps a promise for each relevant event. Handles cancelling them when destroyed. */
	class FSingleClientSubmissionOperation
		: public ISubmissionOperation
		, public FNoncopyable
	{
	public:

		FSingleClientSubmissionOperation(bool bInModifiesStreams)
			: bModifiesStreams(bInModifiesStreams)
		{}
		
		virtual ~FSingleClientSubmissionOperation() override
		{
			if (!bStreamPromiseWasSet)
			{
				StreamChangesPromise.EmplaceValue(FSubmitStreamChangesResponse{ EStreamSubmissionErrorCode::Cancelled });
			}
			if (!bAuthorityRequestPromiseWasSet)
			{
				AuthorityChangeRequestPromise.EmplaceValue(FSubmitAuthorityChangesRequest{ EAuthoritySubmissionRequestErrorCode::Cancelled });
			}
			if (!bAuthorityResponsePromiseWasSet)
			{
				AuthorityChangeResponsePromise.EmplaceValue(FSubmitAuthorityChangesResponse{ EAuthoritySubmissionResponseErrorCode::Cancelled });
			}
			if (!bCompleteOperationPromiseWasSet)
			{
				CompleteOperationPromise.EmplaceValue(ESubmissionOperationCompletedCode::Cancelled);
			}
		}

		void EmplaceStreamPromise(FSubmitStreamChangesResponse Result)
		{
			bStreamPromiseWasSet = true;
			StreamChangesPromise.EmplaceValue(MoveTemp(Result));
			
			UE_LOGF(LogConcert, Log, "Submission operation: Completed stream request stage.");
		}
		void EmplaceAuthorityRequestPromise(FSubmitAuthorityChangesRequest Result)
		{
			bAuthorityRequestPromiseWasSet = true;
			AuthorityChangeRequestPromise.EmplaceValue(MoveTemp(Result));
			
			UE_LOGF(LogConcert, Log, "Submission operation: Complete authority request stage.");
		}
		void EmplaceAuthorityResponsePromise(FSubmitAuthorityChangesResponse Result)
		{
			bAuthorityResponsePromiseWasSet = true;
			AuthorityChangeResponsePromise.EmplaceValue(MoveTemp(Result));
			
			UE_LOGF(LogConcert, Log, "Submission operation: Completed authority response stage.");
		}
		void EmplaceCompleteOperationPromise(ESubmissionOperationCompletedCode Result)
		{
			// The following enforces the API contract that CompleteOperationPromise finishes as last future.
			if (!ensureMsgf(bStreamPromiseWasSet, TEXT("All other stages should be explicitly completed before calling EmplaceCompleteOperationPromise. Check implementation logic.")))
			{
				EmplaceStreamPromise(FSubmitStreamChangesResponse{ EStreamSubmissionErrorCode::Cancelled });
			}
			if (!ensureMsgf(bAuthorityRequestPromiseWasSet, TEXT("All other stages should be explicitly completed before calling EmplaceCompleteOperationPromise. Check implementation logic.")))
			{
				EmplaceAuthorityRequestPromise(FSubmitAuthorityChangesRequest{ EAuthoritySubmissionRequestErrorCode::Cancelled });
			}
			if (!ensureMsgf(bAuthorityResponsePromiseWasSet, TEXT("All other stages should be explicitly completed before calling EmplaceCompleteOperationPromise. Check implementation logic.")))
			{
				EmplaceAuthorityResponsePromise(FSubmitAuthorityChangesResponse{ EAuthoritySubmissionResponseErrorCode::Cancelled });
			}
			
			bCompleteOperationPromiseWasSet = true;
			CompleteOperationPromise.EmplaceValue(Result);
			
			UE_LOGF(LogConcert, Log, "Submission operation: Completed operation.");
		}

		bool HasSetStreamPromise() const { return bStreamPromiseWasSet; }
		bool HasSetAuthorityRequestPromise() const { return bAuthorityRequestPromiseWasSet; }
		bool HasSetAuthorityResponsePromise() const { return bAuthorityResponsePromiseWasSet; }
		bool HasSetCompleteOperationPromise() const { return bCompleteOperationPromiseWasSet; }
		
		//~ Begin ISubmissionOperation Interface
		virtual bool IsModifyingStreams() const override { return bModifiesStreams; }
		virtual TFuture<FSubmitStreamChangesResponse> OnCompleteStreamChangesFuture_AnyThread() override { return StreamChangesPromise.GetFuture();  }
		virtual TFuture<FSubmitAuthorityChangesRequest> OnRequestAuthorityChangeFuture_AnyThread() override { return AuthorityChangeRequestPromise.GetFuture(); }
		virtual TFuture<FSubmitAuthorityChangesResponse> OnCompleteAuthorityChangeFuture_AnyThread() override { return AuthorityChangeResponsePromise.GetFuture(); }
		virtual TFuture<ESubmissionOperationCompletedCode> OnCompletedOperation_AnyThread() override { return CompleteOperationPromise.GetFuture(); }
		//~ End ISubmissionOperation Interface

	private:

		const bool bModifiesStreams;

		bool bStreamPromiseWasSet = false;
		bool bAuthorityRequestPromiseWasSet = false;
		bool bAuthorityResponsePromiseWasSet = false;
		bool bCompleteOperationPromiseWasSet = false;
		
		// All fulfilled by the owning FSingleClientSubmissionWorkflow
		TPromise<FSubmitStreamChangesResponse> StreamChangesPromise;
		TPromise<FSubmitAuthorityChangesRequest> AuthorityChangeRequestPromise;
		TPromise<FSubmitAuthorityChangesResponse> AuthorityChangeResponsePromise;
		TPromise<ESubmissionOperationCompletedCode> CompleteOperationPromise;
	};
}
