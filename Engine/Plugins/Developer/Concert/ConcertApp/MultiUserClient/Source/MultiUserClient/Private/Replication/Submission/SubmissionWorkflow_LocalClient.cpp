// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmissionWorkflow_LocalClient.h"

#include "IConcertSyncClient.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Misc/Util/StreamRequestUtils.h"
#include "Replication/Client/ClientUtils.h"

namespace UE::MultiUserClient::Replication
{
	FSubmissionWorkflow_LocalClient::FSubmissionWorkflow_LocalClient(TSharedRef<IConcertSyncClient> InClient)
		: Client(MoveTemp(InClient))
	{}

	EChangeUploadability FSubmissionWorkflow_LocalClient::GetUploadability() const
	{
		const bool bOperationInProgress = InProgressOperation.IsSet(); 
		return bOperationInProgress ? EChangeUploadability::InProgress : EChangeUploadability::Ready;
	}

	TSharedPtr<ISubmissionOperation> FSubmissionWorkflow_LocalClient::SubmitChanges(FSubmissionParams Params)
	{
		using namespace ConcertSyncClient::Replication;
		
		IConcertClientReplicationManager* ReplicationManager = Client->GetReplicationManager();
		if (Params.IsEmpty() || !CanSubmit() || !ensure(ReplicationManager))
		{
			return nullptr;
		}
		
		const TOptional<FConcertReplication_ChangeStream_Request>& OptionalStreamRequest = Params.StreamRequest;
		TOptional<FConcertReplication_ChangeAuthority_Request>& OptionalAuthorityRequest = Params.AuthorityRequest;
		
		const bool bIsStreamChangeEmpty = Params.IsStreamChangeEmpty();
		const bool bModifyStreams = !bIsStreamChangeEmpty;
		const TSharedRef<FSingleClientSubmissionOperation> Operation = MakeShared<FSingleClientSubmissionOperation>(bModifyStreams);
		InProgressOperation = { Operation };
		
		if (bIsStreamChangeEmpty)
		{
			const FSubmitStreamChangesResponse CompletedChange { EStreamSubmissionErrorCode::NoChange };
			Operation->EmplaceStreamPromise(CompletedChange);
			StreamRequestCompletedDelegate.Broadcast(CompletedChange);
			
			HandlePendingAuthorityChangeRequest(MoveTemp(*OptionalAuthorityRequest));
		}
		else
		{
			ReplicationManager->ChangeStream(*OptionalStreamRequest)
				.Next([this, DestructionDetection = LifetimeToken->AsWeak(), OptionalStreamRequest, AuthorityRequest = MoveTemp(OptionalAuthorityRequest)](FConcertReplication_ChangeStream_Response&& Response) mutable
				{
					const FSubmitStreamChangesResponse SubmissionResult { EStreamSubmissionErrorCode::Success, { FCompletedChangeSubmission{*OptionalStreamRequest, Response } } };
					// The request might execute after we're destroyed, e.g. by leaving session while request is on the way.
					// In that case, the Concert session triggers the OnSessionConnectionChanged which destroys us. Only after that, all the requests are timed out.
					if (DestructionDetection.IsValid())
					{
						OnStreamChangeCompleted(*OptionalStreamRequest, Response, MoveTemp(AuthorityRequest));
					}

					return SubmissionResult;
				});
		}

		// Note that InProgressOperation might already be unset due to ChangeStream failing instantly
		return Operation;
	}

	void FSubmissionWorkflow_LocalClient::OnStreamChangeCompleted(
		const FConcertReplication_ChangeStream_Request& StreamChangeRequest,
		const FConcertReplication_ChangeStream_Response& ChangeStreamResponse,
		TOptional<FConcertReplication_ChangeAuthority_Request> AuthorityChangeRequest
		)
	{
		const TSharedRef<FSingleClientSubmissionOperation> Operation = InProgressOperation->Operation;
		
		const EStreamSubmissionErrorCode ErrorCode = ChangeStreamResponse.ErrorCode == EReplicationResponseErrorCode::Handled
			? EStreamSubmissionErrorCode::Success
			: EStreamSubmissionErrorCode::Timeout;
		const FSubmitStreamChangesResponse CompletedChange{ ErrorCode, { FCompletedChangeSubmission{ StreamChangeRequest, ChangeStreamResponse }} };
		Operation->EmplaceStreamPromise(CompletedChange);
		StreamRequestCompletedDelegate.Broadcast(CompletedChange);
		
		if (ChangeStreamResponse.IsSuccess())
		{
			HandlePendingAuthorityChangeRequest(MoveTemp(AuthorityChangeRequest));
		}
		else
		{
			SkipAuthorityStage(EAuthoritySubmissionRequestErrorCode::CancelledDueToStreamUpdate, EAuthoritySubmissionResponseErrorCode::CancelledDueToStreamUpdate);
			CleanUpSubmissionOperation();
		}
	}

	void FSubmissionWorkflow_LocalClient::HandlePendingAuthorityChangeRequest(TOptional<FConcertReplication_ChangeAuthority_Request> AuthorityChangeRequest)
	{
		using namespace ConcertSyncClient::Replication;
		IConcertClientReplicationManager* ReplicationManager = Client->GetReplicationManager();
		if (!ensure(ReplicationManager))
		{
			SkipAuthorityStage(EAuthoritySubmissionRequestErrorCode::Cancelled, EAuthoritySubmissionResponseErrorCode::Cancelled);
			CleanUpSubmissionOperation();
			return;
		}

		const TSharedRef<FSingleClientSubmissionOperation> Operation = InProgressOperation->Operation;
		const bool bHasNoChanges = !AuthorityChangeRequest || AuthorityChangeRequest->IsEmpty();
		if (bHasNoChanges)
		{
			SkipAuthorityStage(EAuthoritySubmissionRequestErrorCode::NoChange, EAuthoritySubmissionResponseErrorCode::NoChange);
			CleanUpSubmissionOperation();
			return;
		}
		
		FSubmitAuthorityChangesRequest Request{ EAuthoritySubmissionRequestErrorCode::Success, AuthorityChangeRequest };
		Operation->EmplaceAuthorityRequestPromise(Request);
		ReplicationManager->RequestAuthorityChange(MoveTemp(*AuthorityChangeRequest))
			.Next([this, Request = MoveTemp(Request), DestructionDetection = LifetimeToken->AsWeak()](FConcertReplication_ChangeAuthority_Response&& Response)
			{
				if (DestructionDetection.IsValid())
				{
					const EAuthoritySubmissionResponseErrorCode ErrorCode = Response.ErrorCode == EReplicationResponseErrorCode::Handled
						? EAuthoritySubmissionResponseErrorCode::Success
						: EAuthoritySubmissionResponseErrorCode::Timeout;
					const FSubmitAuthorityChangesResponse Result { ErrorCode, MoveTemp(Response) };
					
					InProgressOperation->Operation->EmplaceAuthorityResponsePromise(Result);
					AuthorityRequestCompletedDelegate.Broadcast(Request, Result);
					CleanUpSubmissionOperation();
				}
			});
	}
	
	void FSubmissionWorkflow_LocalClient::SkipAuthorityStage(EAuthoritySubmissionRequestErrorCode RequestCode, EAuthoritySubmissionResponseErrorCode ResponseCode)
	{
		const TSharedRef<FSingleClientSubmissionOperation>& Operation = InProgressOperation->Operation;
		const FSubmitAuthorityChangesRequest Request{ RequestCode };
		const FSubmitAuthorityChangesResponse Response{ ResponseCode };
			
		Operation->EmplaceAuthorityRequestPromise(Request);
		Operation->EmplaceAuthorityResponsePromise(Response);
		AuthorityRequestCompletedDelegate.Broadcast(Request, Response);
	}

	void FSubmissionWorkflow_LocalClient::CleanUpSubmissionOperation()
	{
		const TSharedRef<FSingleClientSubmissionOperation> Operation = InProgressOperation->Operation;
		InProgressOperation.Reset();
		
		Operation->EmplaceCompleteOperationPromise(ESubmissionOperationCompletedCode::Processed);
		OnSubmitOperationCompletedDelegate.Broadcast();
	}
}
