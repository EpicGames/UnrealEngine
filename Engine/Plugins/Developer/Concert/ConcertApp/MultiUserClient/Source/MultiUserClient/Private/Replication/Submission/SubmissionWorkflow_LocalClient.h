// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISubmissionWorkflow.h"

#include "Replication/Stream/IClientStreamSynchronizer.h"
#include "Replication/IToken.h"
#include "SingleClientSubmissionOperation.h"

#include "Templates/UnrealTemplate.h"

class IConcertSyncClient;

namespace UE::MultiUserClient::Replication
{
	class IClientAuthoritySynchronizer;
	class IClientStreamSynchronizer;
	class FAuthorityChangeTracker;
	class FGlobalAuthorityCache;
	class FStreamChangeTracker;
	
	/** Handles the submission workflow for a single client. */
	class FSubmissionWorkflow_LocalClient : public FSubmissionWorkflowBase, public FNoncopyable
	{
	public:
		
		FSubmissionWorkflow_LocalClient(TSharedRef<IConcertSyncClient> InClient);
		
		//~ Begin ISubmissionWorkflow Interface
		virtual TSharedPtr<ISubmissionOperation> SubmitChanges(FSubmissionParams Params) override;
		virtual EChangeUploadability GetUploadability() const override;
		//~ End ISubmissionWorkflow Interface
	
	private:

		/** Latent operations keep a weak reference to this to detect whether this FSubmissionWorkflow_LocalClient was destroyed. */
		const TSharedRef<FToken> LifetimeToken = FToken::Make();
		
		/** Used to send authority requests to the server. */
		const TSharedRef<IConcertSyncClient> Client;

		struct FOperationData
		{
			TSharedRef<FSingleClientSubmissionOperation> Operation;
		};
		/**
		 * Set for as long as there is a SubmitChanges operation in progress.
		 * Automatically cancels pending promises when destroyed.
		 */
		TOptional<FOperationData> InProgressOperation;

		/** Advances the request by requesting authority. */
		void OnStreamChangeCompleted(
			const FConcertReplication_ChangeStream_Request& StreamChangeRequest,
			const FConcertReplication_ChangeStream_Response& ChangeStreamResponse,
			TOptional<FConcertReplication_ChangeAuthority_Request> AuthorityChangeRequest
			);
		void HandlePendingAuthorityChangeRequest(TOptional<FConcertReplication_ChangeAuthority_Request> AuthorityChangeRequest);
		
		void SkipAuthorityStage(EAuthoritySubmissionRequestErrorCode RequestCode, EAuthoritySubmissionResponseErrorCode ResponseCode);
		void CleanUpSubmissionOperation();
	};
}

