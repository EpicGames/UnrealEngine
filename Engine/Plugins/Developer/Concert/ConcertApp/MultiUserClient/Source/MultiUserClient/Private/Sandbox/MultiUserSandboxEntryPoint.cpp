// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserSandboxEntryPoint.h"

#include "IConcertSyncClient.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "FMultiUserSandboxEntryPoint"

namespace UE::MultiUserClient
{
FMultiUserSandboxEntryPoint::FMultiUserSandboxEntryPoint(const TSharedRef<IConcertSyncClient>& InMultiUserClient)
	: MultiUserClient(InMultiUserClient)
{}

void FMultiUserSandboxEntryPoint::SummonProviderUI()
{
	OnSummonProviderUIDelegate.Broadcast(); 
}

FText FMultiUserSandboxEntryPoint::GetEntryPointLabel() const
{
	return LOCTEXT("EntryPoint.Label", "Multi User");
}

bool FMultiUserSandboxEntryPoint::OwnsSandbox(const FileSandboxCore::ISandboxInstance& InSandbox) const
{
	return MultiUserClient->GetConcertClient()->GetCurrentSession().IsValid();
}
}

#undef LOCTEXT_NAMESPACE