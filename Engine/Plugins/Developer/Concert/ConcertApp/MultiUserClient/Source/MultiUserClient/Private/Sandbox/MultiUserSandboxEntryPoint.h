// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "EntryPoint/ISandboxEntryPoint.h"

class IConcertSyncClient;

namespace UE::MultiUserClient
{
/** Represents Multi User as entry point so it becomes discoverable by other editor systems. */
class FMultiUserSandboxEntryPoint : public FileSandboxUI::ISandboxEntryPoint
{
public:
	
	explicit FMultiUserSandboxEntryPoint(const TSharedRef<IConcertSyncClient>& InMultiUserClient);
	
	//~ Begin ISandboxEntryPoint Interface
	virtual void SummonProviderUI() override;
	virtual FText GetEntryPointLabel() const override;
	virtual bool OwnsSandbox(const FileSandboxCore::ISandboxInstance& InSandbox) const override;
	//~ End ISandboxEntryPoint Interface
	
	/** Event invoked when the UI is supposed to be summoned. */
	FSimpleMulticastDelegate& OnSummonProviderUI() { return OnSummonProviderUIDelegate; }
	
private:
	
	/** Multi User's client instance. Used to get the current session: if client is in a session, we'll report that the sandbox is owned by Multi User. */
	const TSharedRef<IConcertSyncClient> MultiUserClient;
	
	/** Event invoked when the UI is supposed to be summoned. */
	FSimpleMulticastDelegate OnSummonProviderUIDelegate;
};
}

