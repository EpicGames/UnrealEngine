// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserSandboxIntegration.h"

#include "EntryPoint/EntryPointWidgetFactory.h"
#include "EntryPoint/ISandboxEntryPointRegistry.h"
#include "IFileSandboxUIModule.h"

namespace UE::MultiUserClient
{
FMultiUserSandboxIntegration::FMultiUserSandboxIntegration(const TSharedRef<IConcertSyncClient>& InMultiUserClient)
	: EntryPoint(MakeShared<FMultiUserSandboxEntryPoint>(InMultiUserClient))
	, ExternalSandboxViewModel(FileSandboxUI::MakeExternalSandboxActiveViewModel(EntryPoint))
{
	using namespace UE::FileSandboxUI;
	ISandboxEntryPointRegistry& EntryPointsRegistry = IFileSandboxUIModule::Get().GetEntryPointRegistry();
	EntryPointsRegistry.RegisterEntryPoint(EntryPoint);
}

FMultiUserSandboxIntegration::~FMultiUserSandboxIntegration()
{
	using namespace UE::FileSandboxUI;
	if (IFileSandboxUIModule::IsAvailable())
	{
		ISandboxEntryPointRegistry& EntryPointsRegistry = IFileSandboxUIModule::Get().GetEntryPointRegistry();
		EntryPointsRegistry.UnregisterEntryPoint(EntryPoint);
	}
}
}
