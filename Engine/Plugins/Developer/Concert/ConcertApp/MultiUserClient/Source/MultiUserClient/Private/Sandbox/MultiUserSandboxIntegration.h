// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MultiUserSandboxEntryPoint.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxUI { class IExternalSandboxActiveViewModel; }

namespace UE::MultiUserClient
{
/** Responsible for registering Multi User with the sandbox system. */
class FMultiUserSandboxIntegration : public FNoncopyable
{
public:
	
	FMultiUserSandboxIntegration(const TSharedRef<IConcertSyncClient>& InMultiUserClient);
	~FMultiUserSandboxIntegration();
	
	const TSharedRef<FMultiUserSandboxEntryPoint>& GetEntryPoint() const { return EntryPoint; }
	const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel>& GetExternalSandboxViewModel() const { return ExternalSandboxViewModel; }
	
private:
	
	/** Represents Multi User as entry point so it becomes discoverable by other editor systems. */
	const TSharedRef<FMultiUserSandboxEntryPoint> EntryPoint;
	
	/** This view model is used to overlay a widget onto the browser when the engine is already sandboxed. */
	const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel> ExternalSandboxViewModel;
};
}

