// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrequencyDefaultRules.h"

#include "Algo/AnyOf.h"
#include "GameFramework/Actor.h"

bool FMultiUserActorFrequencyStackEntry::Matches(UObject& Object) const
{
	const FSoftClassPath ObjectClass = Object.GetClass();
	const bool bMatchesAnyClass = Algo::AnyOf(MatchingClasses, [&ObjectClass](const FSoftClassPath& Class){ return ObjectClass == Class; });
	if (bMatchesAnyClass || Object.IsA<AActor>())
	{
		return bMatchesAnyClass;
	}

	AActor* ObjectOwner = Object.GetTypedOuter<AActor>();
	if (!ObjectOwner)
	{
		return false;
	}

	bool bFoundObject = false;
	SubobjectMatchingRules.MatchSubobjectsRecursivelyBreakable(*ObjectOwner, [&Object, &bFoundObject](UObject& MatchedObject)
	{
		bFoundObject |= &MatchedObject == &Object;
		return bFoundObject ? EBreakBehavior::Break : EBreakBehavior::Continue;
	});
	return bFoundObject;
}

TOptional<FConcertObjectReplicationSettings> FMultiUserFrequencyDefaultRules::FindOverrideSettings(UObject& Object) const
{
	for (const FMultiUserActorFrequencyStackEntry& Entry : FrequencyRuleStack)
	{
		if (Entry.Matches(Object))
		{
			return Entry.ReplicationSettings;
		}
	}
	return {};
}
