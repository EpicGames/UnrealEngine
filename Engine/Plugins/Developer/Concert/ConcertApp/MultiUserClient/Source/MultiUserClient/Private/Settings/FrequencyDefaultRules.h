// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ReplicationFrequencySettings.h"
#include "Replication/Settings/ConcertStreamObjectAutoBindingRules.h"
#include "Misc/Optional.h"
#include "FrequencyDefaultRules.generated.h"

class UObject;

/**
 * Used to build a stack of rules. The first stack entry that matches an object, is used to determine the object's frequency settings.
 */
USTRUCT()
struct FMultiUserActorFrequencyStackEntry
{
	GENERATED_BODY()

	/** The settings to apply to objects matched via this stack entry. */
	UPROPERTY(Category = "Replication Settings|Frequency", EditAnywhere)
	FConcertObjectReplicationSettings ReplicationSettings;

	/**
	 * If an object's class is contained in this set, ReplicationSettings are applied to it.
	 * @see bAllowRecursiveClassSearch.
	 */
	UPROPERTY(Category = "Replication Settings|Frequency", EditAnywhere)
	TSet<FSoftClassPath> MatchingClasses;

	/**
	 * These rules are applied only to objects that are not actors, e.g. components.
	 * The matching rules are applied to the object's owning actor. If the object is found by these rules, ReplicationSettings are applied to it.
	 */
	UPROPERTY(Category = "Replication Settings|Frequency", EditAnywhere)
	FConcertPerClassSubobjectMatchingRules SubobjectMatchingRules;

	/**
	 * Determines whether parent classes are included when matching against MatchingClasses.
	 * 
	 * Example: MatchingClasses = UStaticMeshComponent.
	 * Object 1 is UStaticMeshComponent > matches.
	 * Object 2 is UInstancedStaticMeshComponent > if bAllowRecursiveClassSearch == true, then UInstancedStaticMeshComponent is matched. Otherwise, not. 
	 */
	UPROPERTY(Category = "Replication Settings|Frequency", EditAnywhere, AdvancedDisplay)
	bool bAllowRecursiveClassSearch = true;

	/** @return Whether Object matches the these rules. */
	bool Matches(UObject& Object) const;
};

/**
 * Defines rules determining an object's frequency settings.
 */
USTRUCT()
struct FMultiUserFrequencyDefaultRules
{
	GENERATED_BODY()

	/**
	 * Default frequency settings that are used by objects when they do not match any specific rules in FrequencyRuleStack.
	 * These are the settings that are set for the Multi User stream's FConcertBaseStreamInfo::FrequencySettings::Defaults member.
	 */
	UPROPERTY(Category = "Replication Settings|Frequency", EditAnywhere)
	FConcertObjectReplicationSettings DefaultObjectFrequencySettings { EConcertObjectReplicationMode::SpecifiedRate, 30 };

	/**
	 * Use this to assign specific frequency settings to objects.
	 * 
	 * When an object is added to the replication stream, its frequency settings are looked up as follows:
	 *	1. If the object is an actor,
	 *		1.1 go through FrequencyRuleStack from top to bottom
	 *		1.2 if the entry's MatchingClasses contains the actor's class, use the ReplicationSettings in that entry.
	 *	2. If the object is not an actor,
	*		2.1 go through FrequencyRuleStack from top to bottom
	 *		2.2 if the entry's MatchingClasses contains the object's class, use the ReplicationSettings in that entry. 
	 *		2.3 get the object's owning actor
	 *		2.4 if the entry's SubobjectMatchingRules finds the object by searching the owning actor, use the ReplicationSettings in that entry.
	 */
	UPROPERTY(Category = "Replication Settings|Frequency", EditAnywhere)
	TArray<FMultiUserActorFrequencyStackEntry> FrequencyRuleStack;

	/**
	 * Goes through FrequencyRuleStack and returns the FConcertObjectReplicationSettings of the entry that was matched.
	 * 
	 * If none was matched, the object should not have any frequency override and should default to the stream's
	 * FConcertBaseStreamInfo::FrequencySettings::Defaults, which should be set to DefaultObjectFrequencySettings. 
	 */
	TOptional<FConcertObjectReplicationSettings> FindOverrideSettings(UObject& Object) const;
};
