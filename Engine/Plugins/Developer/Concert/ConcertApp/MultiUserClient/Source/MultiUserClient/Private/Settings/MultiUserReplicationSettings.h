// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrequencyDefaultRules.h"
#include "Replication/Settings/ConcertStreamObjectAutoBindingRules.h"

#include "Misc/Optional.h"
#include "UObject/Object.h"

#include "MultiUserReplicationSettings.generated.h"

UCLASS(config=MultiUserClient)
class UMultiUserReplicationSettings : public UObject
{
	GENERATED_BODY()
public:

	static UMultiUserReplicationSettings* Get() { return GetMutableDefault<UMultiUserReplicationSettings>(); }

	/**
	 * When you add an object via the stream editor, you may want to automatically bind properties and add additional subobjects.
	 * Here you can specify the rules to achieve this.
	 */
	UPROPERTY(Config, Category = "Replication Settings", EditAnywhere)
	FConcertStreamObjectAutoBindingRules ReplicationEditorSettings;

	/** Determines the frequency settings applied to an object set by default when it is added to the stream. */
	UPROPERTY(Config, Category = "Replication Settings|Frequency", EditAnywhere)
	FMultiUserFrequencyDefaultRules FrequencyRules;

	/**
	 * Checks Object against these settings and returns override settings to set for this object.
	 * @param Object An object that the user just added to the stream
	 */
	TOptional<FConcertObjectReplicationSettings> DetermineObjectFrequencySettings(UObject& Object) const { return FrequencyRules.FindOverrideSettings(Object); }
};