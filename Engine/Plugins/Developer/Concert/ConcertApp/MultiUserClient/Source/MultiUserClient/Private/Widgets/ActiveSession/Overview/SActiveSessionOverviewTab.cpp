// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ActiveSession/Overview/SActiveSessionOverviewTab.h"

#include "ConcertActionDefinition.h"
#include "ConcertClientFrontendUtils.h"
#include "ConcertClientSettings.h"
#include "ConcertFrontendUtils.h"
#include "ConcertMessageData.h"
#include "ConcertMessages.h"
#include "Dialog/SMessageDialog.h"
#include "IConcertClientPresenceManager.h"
#include "IConcertClient.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "Widgets/Client/ClientInfoHelpers.h"
#include "Widgets/Client/SClientName.h"
#include "Widgets/ClientSessionHistoryController.h"

#include "Algo/Transform.h"
#include "FileHelpers.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/PackageName.h"
#include "Dialog/SCustomDialog.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "HAL/PlatformFileManager.h"

#define LOCTEXT_NAMESPACE "SActiveSessionOverviewTab"

namespace ActiveSessionDetailsUI
{
	static const FName DisplayNameColumnName(TEXT("DisplayName"));
	static const FName PresenceColumnName(TEXT("Presence"));
	static const FName LevelColumnName(TEXT("Level"));
}


class SActiveSessionDetailsRow : public SMultiColumnTableRow<TSharedPtr<FConcertSessionClientInfo>>
{
public:
	
	SLATE_BEGIN_ARGS(SActiveSessionDetailsRow) {}
	SLATE_END_ARGS()

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InClientInfo The client displayed by this row.
	 * @param InClientSession The session in which the client is, used to determine if the client is the local one, so that we can suffix it with a "you".
	 * @param InOwnerTableView The table to which the row must be added.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<IConcertSyncClient> InSyncClient, TWeakPtr<IConcertClientSession> InClientSession, TSharedPtr<FConcertSessionClientInfo> InClientInfo, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		SyncClient = MoveTemp(InSyncClient);
		ClientSession = MoveTemp(InClientSession);
		SessionClientInfo = MoveTemp(InClientInfo);
		SMultiColumnTableRow<TSharedPtr<FConcertSessionClientInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		// Set the tooltip for the entire row. Will show up unless there is another item with a tooltip hovered in the row, such as the "presence" icons.
		SetToolTipText(MakeAttributeSP(this, &SActiveSessionDetailsRow::GetRowToolTip));
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ActiveSessionDetailsUI::DisplayNameColumnName)
		{
			const TSharedPtr<IConcertSyncClient> SyncClientPin = SyncClient.Pin();
			const TSharedPtr<FConcertSessionClientInfo> SessionClientInfoPin = SessionClientInfo.Pin();
			if (!SyncClientPin)
			{
				return SNew(STextBlock).Text(LOCTEXT("Unavailable", "Unavailable"));
			}

			const bool bIsLocalClient = SyncClientPin->GetConcertClient()->GetCurrentSession()->GetSessionClientEndpointId() == SessionClientInfoPin->ClientEndpointId;
			const FText ParenthesesContent = bIsLocalClient ? UE::ConcertSharedSlate::ParenthesesClientNameContent::LocalClient : FText::GetEmpty();
			return SNew(UE::ConcertSharedSlate::SClientName)
				.ClientInfo(UE::ConcertClientSharedSlate::MakeClientInfoAttribute(SyncClientPin->GetConcertClient(), SessionClientInfoPin->ClientEndpointId))
				.ParenthesisContent(ParenthesesContent);
		}
		else if (ColumnName == ActiveSessionDetailsUI::PresenceColumnName)
		{
			// Displays a set of icons corresponding to the client presence. The set may be extended later to include other functionalities.
			TArray<FConcertActionDefinition> ActionDefs;
			TSharedRef<SHorizontalBox> PresenceCell = SNew(SHorizontalBox);

			TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
			TSharedPtr<IConcertSyncClient> SyncClientPin = SyncClient.Pin();
			if (ClientInfoPin.IsValid() && SyncClientPin.IsValid())
			{
				SyncClientPin->GetSessionClientActions(*ClientInfoPin, ActionDefs);
				ConcertClientFrontendUtils::AppendButtons(PresenceCell, ActionDefs);
			}
			return PresenceCell;
		}
		else // LevelColumnName
		{
			check(ColumnName == ActiveSessionDetailsUI::LevelColumnName); // If this fail, was a column added/removed/renamed ?

			TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
			TSharedPtr<IConcertClientSession> ClientSessionPin = ClientSession.Pin();
			TSharedPtr<SWidget> LevelWidget;

			// If the row is not for "this client", then add a hyper link to open the other client level.
			if (ClientInfoPin.IsValid() && ClientSessionPin.IsValid() && ClientInfoPin->ClientEndpointId != ClientSessionPin->GetSessionClientEndpointId())
			{
				LevelWidget = SNew(SOverlay)
					+SOverlay::Slot()
					[
						SNew(SHyperlink)
						.Visibility_Lambda([this]() { return GetOtherLevelVisibility(); })
						.Text(this, &SActiveSessionDetailsRow::GetLevel)
						.ToolTipText(this, &SActiveSessionDetailsRow::GetOtherLevelTooltip)
						.OnNavigate(this, &SActiveSessionDetailsRow::OnOtherLevelClicked)
					]
					+SOverlay::Slot()
					[
						SNew(STextBlock)
						.Visibility_Lambda([this]() { return GetOtherLevelVisibility() == EVisibility::Visible ? EVisibility::Collapsed : EVisibility::Visible; })
						.Text(LOCTEXT("LevelNotAvailableInGame", "N/A (-game)")) // The only known reason for the level to be unknown and invisible is b/c the user is in -game.
						.ToolTipText(LOCTEXT("LevelNotAvailableInGame_Tooltip", "The presence/level information is not emitted by clients running Editor in -game mode."))
					];
			}
			else
			{
				LevelWidget = SNew(STextBlock)
					.Text(this, &SActiveSessionDetailsRow::GetLevel)
					.ToolTipText(FText(LOCTEXT("YourLevel", "The level opened in your editor.")));
			}

			// Displays which "level" the client is editing, playing (PIE) or simulating (SIE).
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(0.0f, 4.0f, 2.0f, 6.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("LevelEditor.Tabs.Levels"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0f, 4.0f))
				[
					LevelWidget.ToSharedRef()
				];
		}
	}

	FText GetRowToolTip() const
	{
		// This is a tooltip for the entire row. Like display name, the tooltip will not update in real time if the user change its
		// settings. See GetDisplayName() for more info.
		TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
		return ClientInfoPin.IsValid() ? ClientInfoPin->ToDisplayString() : FText();
	}

	FText GetDisplayName() const
	{
		TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
		if (ClientInfoPin.IsValid())
		{
			// NOTE: The display name doesn't update in real time at the moment because the concert setting are not propagated
			//       until the client disconnect/reconnect. Since those settings should not change often, this should not
			//       be a major deal breaker for the users.
			TSharedPtr<IConcertClientSession> ClientSessionPin = ClientSession.Pin();
			if (ClientSessionPin.IsValid() && ClientInfoPin->ClientEndpointId == ClientSessionPin->GetSessionClientEndpointId())
			{
				return FText::Format(LOCTEXT("ClientDisplayNameIsYouFmt", "{0} (You)"), FText::FromString(ClientSessionPin->GetLocalClientInfo().DisplayName));
			}

			// Return the ClientInfo cached.
			return FText::FromString(ClientInfoPin->ClientInfo.DisplayName);
		}

		return FText();
	}

	EVisibility GetOtherLevelVisibility() const
	{
		TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
		TSharedPtr<IConcertSyncClient> SyncClientPin = SyncClient.Pin();

		if (ClientInfoPin && SyncClientPin)
		{
			EEditorPlayMode EditorPlayMode = EEditorPlayMode::None;
			if (IConcertClientPresenceManager* PresenceManager = SyncClientPin->GetPresenceManager())
			{
				if (!PresenceManager->GetPresenceWorldPath(ClientInfoPin->ClientEndpointId, EditorPlayMode).IsEmpty())
				{
					return EVisibility::Visible;
				}
			}
		}

		return EVisibility::Hidden;
	}

	FText GetOtherLevelTooltip() const
	{
		if (TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin())
		{
			return FText::Format(LOCTEXT("OtherUserLevel_Tooltip", "The level opened in {0} editor. Click to open this level."), FText::FromString(ClientInfoPin->ClientInfo.DisplayName));
		}

		return FText::GetEmpty();
	}

	FText GetLevel() const
	{
		FString WorldPath;
		
		TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
		TSharedPtr<IConcertSyncClient> SyncClientPin = SyncClient.Pin();

		if (ClientInfoPin && SyncClientPin)
		{
			// The world path returned by the function below is always the one from the editor context, never the one from the PIE/SIE context, so
			// it will not contain the prefix added to the world when in PIE/SIE.
			EEditorPlayMode EditorPlayMode = EEditorPlayMode::None;
			if (IConcertClientPresenceManager* PresenceManager = SyncClientPin->GetPresenceManager())
			{
				WorldPath = PresenceManager->GetPresenceWorldPath(ClientInfoPin->ClientEndpointId, EditorPlayMode);
			}
			
			// The world path is returned as something like /Game/MyMap.MyMap, but we are only interested to keep the
			// string left to the '.' to display "/Game/MyMap"
			WorldPath = FPackageName::ObjectPathToPackageName(WorldPath);

			if (EditorPlayMode == EEditorPlayMode::PIE)
			{
				WorldPath += TEXT(" (PIE)");
			}
			else if (EditorPlayMode == EEditorPlayMode::SIE)
			{
				WorldPath += TEXT(" (SIE)");
			}
		}

		return FText::FromString(WorldPath);
	}

	void OnOtherLevelClicked()
	{
		TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
		TSharedPtr<IConcertClientSession> ClientSessionPin = ClientSession.Pin();
		TSharedPtr<IConcertSyncClient> ConcertSyncClientPin = SyncClient.Pin();

		// If the user clicks on a client row other than the one that represents them.
		if (ClientInfoPin && ClientSessionPin && ConcertSyncClientPin && ConcertSyncClientPin->GetPresenceManager() &&
			ClientInfoPin->ClientEndpointId != ClientSessionPin->GetSessionClientEndpointId())
		{
			IConcertClientPresenceManager* PresenceManager = ConcertSyncClientPin->GetPresenceManager();
			EEditorPlayMode EditorPlayMode; // The result stored in this variable is not used.
			FString RowWorldPath = PresenceManager->GetPresenceWorldPath(ClientInfoPin->ClientEndpointId, EditorPlayMode);
			FString ThisClientSessionWorldPath = PresenceManager->GetPresenceWorldPath(ClientSessionPin->GetSessionClientEndpointId(), EditorPlayMode);

			// If this client is not editing the same world as the client represented by this row.
			if (ThisClientSessionWorldPath != RowWorldPath)
			{
				// If there are any unsaved changes to the current level, see if the user wants to save those first.
				bool bPromptUserToSave = true;
				bool bSaveMapPackages = true;
				bool bSaveContentPackages = false;
				if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages) == false)
				{
					return;
				}
				
				// Open the level.
				FString WorlPathname;
				const bool bLoadAsTemplate = false;
				const bool bShowProgress = true;
				FPackageName::TryConvertLongPackageNameToFilename(RowWorldPath, WorlPathname);
				FEditorFileUtils::LoadMap(WorlPathname, bLoadAsTemplate, bShowProgress);

				// Jump to the client position/orientation to view the same thing as client corresponding to the clicked row currently look at.
				PresenceManager->InitiateJumpToPresence(ClientInfoPin->ClientEndpointId);
			}
		}
	}

	FSlateFontInfo GetAvatarFont() const
	{
		// This font is used to render a small square box filled with the avatar color.
		FSlateFontInfo ClientIconFontInfo = FAppStyle::Get().GetFontStyle(ConcertClientFrontendUtils::ButtonIconSyle);
		ClientIconFontInfo.Size = 8;
		ClientIconFontInfo.OutlineSettings.OutlineSize = 1;

		TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
		if (ClientInfoPin.IsValid())
		{
			FLinearColor ClientOutlineColor = ClientInfoPin->ClientInfo.AvatarColor * 0.6f; // Make the font outline darker.
			ClientOutlineColor.A = ClientInfoPin->ClientInfo.AvatarColor.A; // Put back the original alpha.
			ClientIconFontInfo.OutlineSettings.OutlineColor = ClientOutlineColor;
		}
		else
		{
			ClientIconFontInfo.OutlineSettings.OutlineColor = FLinearColor(0.75, 0.75, 0.75); // This is an arbitrary color.
		}

		return ClientIconFontInfo;
	}

	FSlateColor GetAvatarColor() const
	{
		TSharedPtr<FConcertSessionClientInfo> ClientInfoPin = SessionClientInfo.Pin();
		if (ClientInfoPin.IsValid())
		{
			return ClientInfoPin->ClientInfo.AvatarColor;
		}

		return FSlateColor(FLinearColor(0.75, 0.75, 0.75)); // This is an arbitrary color.
	}

private:
	TWeakPtr<IConcertSyncClient> SyncClient;
	TWeakPtr<IConcertClientSession> ClientSession;
	TWeakPtr<FConcertSessionClientInfo> SessionClientInfo;
};

void SActiveSessionOverviewTab::Construct(const FArguments& InArgs, TSharedPtr<IConcertSyncClient> InConcertSyncClient)
{
	WeakConcertSyncClient = InConcertSyncClient;
	SessionHistoryController = MakeShared<FClientSessionHistoryController>(InConcertSyncClient.ToSharedRef());

	if (InConcertSyncClient.IsValid())
	{
		IConcertClientRef ConcertClient = InConcertSyncClient->GetConcertClient();
		ConcertClient->OnSessionStartup().AddSP(this, &SActiveSessionOverviewTab::HandleSessionStartup);
		ConcertClient->OnSessionShutdown().AddSP(this, &SActiveSessionOverviewTab::HandleSessionShutdown);

		if (TSharedPtr<IConcertClientSession> ClientSession = ConcertClient->GetCurrentSession())
		{
			WeakSessionPtr = ClientSession;
			ClientInfo = MakeShared<FConcertSessionClientInfo>(FConcertSessionClientInfo{ClientSession->GetSessionClientEndpointId(), ClientSession->GetLocalClientInfo()});
			ClientSession->OnSessionClientChanged().AddSP(this, &SActiveSessionOverviewTab::HandleSessionClientChanged);
			InConcertSyncClient->GetTransactionBridge()->OnConflictResolutionForPendingSend().AddSP(this, &SActiveSessionOverviewTab::OnSendConflict);
		}

		TSharedPtr<IConcertClientWorkspace> Workspace = InConcertSyncClient->GetWorkspace();
		Workspace->OnActivityAddedOrUpdated().AddSP(this, &SActiveSessionOverviewTab::ActivityUpdated);
		Workspace->AddWorkspaceCanProcessPackagesDelegate(
			TEXT("SActiveSession"), FCanProcessPendingPackages::CreateSP(this, &SActiveSessionOverviewTab::CanProcessPendingPackages));
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		// Clients + History
		+SVerticalBox::Slot()
		.FillHeight(1.0)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)

			// Clients.
			+SSplitter::Slot()
			.SizeRule(TAttribute<SSplitter::ESizeRule>(this, &SActiveSessionOverviewTab::GetClientAreaSizeRule))
			.Value(0.3)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.0f)
				[
					SAssignNew(ClientArea, SExpandableArea)
					.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
					.BorderImage_Lambda([this]() { return ConcertFrontendUtils::GetExpandableAreaBorderImage(*ClientArea); })
					.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					.BodyBorderBackgroundColor(FLinearColor::White)
					.OnAreaExpansionChanged(this, &SActiveSessionOverviewTab::OnClientAreaExpansionChanged)
					.Padding(0.0f)
					.HeaderContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SessionConnectedClients", "Clients"))
						.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
						.ShadowOffset(FVector2D(1.0f, 1.0f))
					]
					.BodyContent()
					[
						SAssignNew(ClientsListView, SListView<TSharedPtr<FConcertSessionClientInfo>>)
						.SelectionMode(ESelectionMode::Single)
						.ListItemsSource(&Clients)
						.OnGenerateRow(this, &SActiveSessionOverviewTab::HandleGenerateRow)
						.HeaderRow
						(
							SNew(SHeaderRow)
							+SHeaderRow::Column(ActiveSessionDetailsUI::DisplayNameColumnName)
							.DefaultLabel(LOCTEXT("UserDisplayName", "Display Name"))
							+SHeaderRow::Column(ActiveSessionDetailsUI::PresenceColumnName)
							.DefaultLabel(LOCTEXT("UserPresence", "Presence"))
							+SHeaderRow::Column(ActiveSessionDetailsUI::LevelColumnName)
							.DefaultLabel(LOCTEXT("UserLevel", "Level"))
						)
					]
				]
			]

			// History (activity feed).
			+SSplitter::Slot()
			.SizeRule(TAttribute<SSplitter::ESizeRule>(this, &SActiveSessionOverviewTab::GetHistoryAreaSizeRule))
			.Value(0.7)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.0f)
				[
					SAssignNew(HistoryArea, SExpandableArea)
					.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
					.BorderImage_Lambda([this]() { return ConcertFrontendUtils::GetExpandableAreaBorderImage(*HistoryArea); })
					.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					.BodyBorderBackgroundColor(FLinearColor::White)
					.OnAreaExpansionChanged(this, &SActiveSessionOverviewTab::OnHistoryAreaExpansionChanged)
					.Padding(0.0f)
					.HeaderContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SessionHistory", "History"))
						.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
						.ShadowOffset(FVector2D(1.0f, 1.0f))
					]
					.BodyContent()
					[
						SessionHistoryController->GetSessionHistory()
					]
				]
			]
		]
	];

	// Create a timer to periodically poll the this client(s) info to detect if it changed its display name or avatar color because
	// IConcertClientsession::OnSessionClientChanged() doesn't trigger when the 'local' client changes. This needs to be polled.
	RegisterActiveTimer(1.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SActiveSessionOverviewTab::HandleLocalClientInfoChangePollingTimer));

	UpdateSessionClientListView();
}

TSharedRef<ITableRow> SActiveSessionOverviewTab::HandleGenerateRow(TSharedPtr<FConcertSessionClientInfo> InClientInfo, const TSharedRef<STableViewBase>& OwnerTable) const
{
	// Generate a row for the client corresponding to InClientInfo.
	return SNew(SActiveSessionDetailsRow, WeakConcertSyncClient, WeakSessionPtr, InClientInfo, OwnerTable);
}

void SActiveSessionOverviewTab::HandleSessionStartup(TSharedRef<IConcertClientSession> InClientSession)
{
	WeakSessionPtr = InClientSession;
	InClientSession->OnSessionClientChanged().AddSP(this, &SActiveSessionOverviewTab::HandleSessionClientChanged);
	if (TSharedPtr<IConcertSyncClient> SyncClient = WeakConcertSyncClient.Pin())
	{
		SyncClient->GetTransactionBridge()->OnConflictResolutionForPendingSend().AddSP(this, &SActiveSessionOverviewTab::OnSendConflict);
	}

	ClientInfo = MakeShared<FConcertSessionClientInfo>(FConcertSessionClientInfo{InClientSession->GetSessionClientEndpointId(), InClientSession->GetLocalClientInfo()});

	UpdateSessionClientListView();

	if (SessionHistoryController.IsValid())
	{
		SessionHistoryController->ReloadActivities();
	}
}

void SActiveSessionOverviewTab::HandleSessionShutdown(TSharedRef<IConcertClientSession> InClientSession)
{
	if (InClientSession == WeakSessionPtr)
	{
		WeakSessionPtr.Reset();
		InClientSession->OnSessionClientChanged().RemoveAll(this);
		if (TSharedPtr<IConcertSyncClient> SyncClient = WeakConcertSyncClient.Pin())
		{
			SyncClient->GetTransactionBridge()->OnConflictResolutionForPendingSend().RemoveAll(this);
		}
		Clients.Reset();

		if (ClientsListView.IsValid())
		{
			ClientsListView->RequestListRefresh();
		}

		if (SessionHistoryController.IsValid())
		{
			SessionHistoryController->ReloadActivities();
		}
	}
}

void SActiveSessionOverviewTab::HandleSessionClientChanged(IConcertClientSession&, EConcertClientStatus ClientStatus, const FConcertSessionClientInfo& InClientInfo)
{
	// Update the view for a specific client.
	UpdateSessionClientListView(&InClientInfo, ClientStatus);
}

EActiveTimerReturnType SActiveSessionOverviewTab::HandleLocalClientInfoChangePollingTimer(double InCurrentTime, float InDeltaTime)
{
// This switch is used to test if updating the session FConcertClientInfo dynamically correctly applies across the board. Don't
// turn it on by default, it would overwrite runtime changes made to the info from some state/logic with the one from the settings.
// For testing: Keep the MU browser tab open, join a session, open the MU settings and change one of the updatable field tracked below.
#define MU_TEST_UPDATE_CLIENT_INFO_FROM_SETTINGS 0

	TSharedPtr<IConcertClientSession> Session = WeakSessionPtr.Pin();
	if (Session.IsValid() && ClientInfo.IsValid())
	{
#if MU_TEST_UPDATE_CLIENT_INFO_FROM_SETTINGS
		if (TSharedPtr<IConcertSyncClient> SyncClientPin = WeakConcertSyncClient.Pin())
		{
			// User changed its settings.
			const FConcertClientInfo& LatestClientInfo = Session->GetLocalClientInfo();
			const UConcertClientConfig* ConcertClientConfig = SyncClientPin->GetConcertClient()->GetConfiguration();
			if (ConcertClientConfig->ClientSettings.AvatarColor != LatestClientInfo.AvatarColor ||
				ConcertClientConfig->ClientSettings.DisplayName != LatestClientInfo.DisplayName ||
				ConcertClientConfig->ClientSettings.DesktopAvatarActorClass != LatestClientInfo.DesktopAvatarActorClass ||
				ConcertClientConfig->ClientSettings.VRAvatarActorClass != LatestClientInfo.VRAvatarActorClass)
			{
				FConcertClientInfoUpdate ClientInfoUpdate;
				ClientInfoUpdate.DisplayName = ConcertClientConfig->ClientSettings.DisplayName;
				ClientInfoUpdate.AvatarColor = ConcertClientConfig->ClientSettings.AvatarColor;
				ClientInfoUpdate.DesktopAvatarActorClass = ConcertClientConfig->ClientSettings.DesktopAvatarActorClass.ToString();
				ClientInfoUpdate.VRAvatarActorClass = ConcertClientConfig->ClientSettings.VRAvatarActorClass.ToString();
				Session->UpdateLocalClientInfo(ClientInfoUpdate);
			}
		}
#endif

		// Check if the local client info cached as class member is out dated with respect to the one held by the session changes. Just check the info displayed by this panel.
		const FConcertClientInfo& LatestClientInfo = Session->GetLocalClientInfo();
		if (LatestClientInfo.DisplayName != ClientInfo->ClientInfo.DisplayName || LatestClientInfo.AvatarColor != ClientInfo->ClientInfo.AvatarColor)
		{
			// Update the view for this client info.
			ClientInfo->ClientInfo = LatestClientInfo;
			UpdateSessionClientListView(ClientInfo.Get(), EConcertClientStatus::Updated);
		}
	}

	return EActiveTimerReturnType::Continue;
}

void SActiveSessionOverviewTab::UpdateSessionClientListView(const FConcertSessionClientInfo* InClientInfo, EConcertClientStatus ClientStatus)
{
	// We expect the UI to be constructed in Construct() function, prior this function gets called.
	check(ClientsListView.IsValid());

	// Try pinning the session.
	TSharedPtr<IConcertClientSession> Session = WeakSessionPtr.Pin();
	if (Session.IsValid())
	{
		// Remember the element selected in the list view to reselect it later.
		TArray<TSharedPtr<FConcertSessionClientInfo>> SelectedItems = ClientsListView->GetSelectedItems();
		checkf(SelectedItems.Num() <= 1, TEXT("ActiveSession's client list view should not support multiple selection."));

		// If this is about a specific client update. (i.e. responding to a IConcertClientSession::OnSessionClientChanged() event)
		if (InClientInfo)
		{
			if (ClientStatus == EConcertClientStatus::Connected)
			{
				Clients.Emplace(MakeShared<FConcertSessionClientInfo>(*InClientInfo));
			}
			else
			{
				int32 Index = Clients.IndexOfByPredicate([InClientInfo](const TSharedPtr<FConcertSessionClientInfo>& Visited) { return InClientInfo->ClientEndpointId == Visited->ClientEndpointId; });
				if (Index != INDEX_NONE)
				{
					if (ClientStatus == EConcertClientStatus::Disconnected)
					{
						Clients.RemoveAt(Index); // We want to preserve items relative order.
					}
					else if (ensure(ClientStatus == EConcertClientStatus::Updated)) // Ensure we handled all status in EConcertClientStatus.
					{
						*Clients[Index] = *InClientInfo; // Update the client info.
					}
				}
			}
		}
		else // Sync everything.
		{
			// Convert the list of clients to a list of shared pointer to client (the list view model asks for shared pointers).
			TArray<FConcertSessionClientInfo> OtherConnectedClients = Session->GetSessionClients();
			TArray<TSharedPtr<FConcertSessionClientInfo>> UpdatedClientList;
			UpdatedClientList.Reserve(OtherConnectedClients.Num() + 1); // +1 is to include this local client (see below).
			Algo::Transform(OtherConnectedClients, UpdatedClientList, [](FConcertSessionClientInfo InClient)
			{
				return MakeShared<FConcertSessionClientInfo>(MoveTemp(InClient));
			});

			// Add this local client as it is not part of the list returned by GetSessionClients(). (The client connected from this process).
			if (ClientInfo.IsValid())
			{
				UpdatedClientList.Emplace(ClientInfo);
			}

			// Merge the list used by the list view (the model) with the updated list, removing clients who left and adding the one who joined.
			ConcertFrontendUtils::SyncArraysByPredicate(Clients, MoveTemp(UpdatedClientList), [](const TSharedPtr<FConcertSessionClientInfo>& ClientToFind)
			{
				return [ClientToFind](const TSharedPtr<FConcertSessionClientInfo>& PotentialClient)
				{
					return PotentialClient->ClientEndpointId == ClientToFind->ClientEndpointId;
				};
			});
		}

		// Sort the list by display name alphabetically.
		Clients.StableSort([](const TSharedPtr<FConcertSessionClientInfo>& ClientOne, const TSharedPtr<FConcertSessionClientInfo>& ClientTwo)
		{
			return ClientOne->ClientInfo.DisplayName < ClientTwo->ClientInfo.DisplayName;
		});

		// If a client row was selected, select it back (if still available).
		if (SelectedItems.Num() > 0)
		{
			ClientsListView->SetSelection(SelectedItems[0]);
		}
	}
	else // The session appears to be invalid.
	{
		// Clear the list of clients.
		Clients.Reset();
	}

	ClientsListView->RequestListRefresh();
}

void SActiveSessionOverviewTab::OnSendConflict(const FConcertConflictDescriptionBase& ConflictDescription)
{
	FAsyncTaskNotificationConfig Config;
	Config.bKeepOpenOnFailure = true;
	Config.TitleText = ConflictDescription.GetConflictTitle();
	Config.ExpireDuration = 5.0f;

	FAsyncTaskNotification Notification(Config);
	Notification.SetHyperlink(FSimpleDelegate(), LOCTEXT("TransactionConflictDetails", "See Details..."));
	Notification.SetComplete(Config.TitleText, ConflictDescription.GetConflictDetails(), false);
}

void SActiveSessionOverviewTab::SetSelectedClient(const FGuid& InClientEndpointId, ESelectInfo::Type SelectInfo)
{
	if (ClientsListView.IsValid())
	{
		TSharedPtr<FConcertSessionClientInfo> NewSelectedClient = FindAvailableClient(InClientEndpointId);

		if (NewSelectedClient.IsValid())
		{
			ClientsListView->SetSelection(NewSelectedClient, SelectInfo);
		}
		else
		{
			ClientsListView->ClearSelection();
		}
	}
}

TSharedPtr<FConcertSessionClientInfo> SActiveSessionOverviewTab::FindAvailableClient(const FGuid& InClientEndpointId) const
{
	const TSharedPtr<FConcertSessionClientInfo>* FoundClientPtr = Clients.FindByPredicate([&InClientEndpointId](const TSharedPtr<FConcertSessionClientInfo>& PotentialClient)
	{
		return PotentialClient->ClientEndpointId == InClientEndpointId;
	});

	return FoundClientPtr ? *FoundClientPtr : nullptr;
}

bool SActiveSessionOverviewTab::CanProcessPendingPackages() const
{
	return bCanHotReload;
}

void SActiveSessionOverviewTab::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bCanHotReload && CanReloadDialog)
	{
		CanReloadDialog = {};
	}
}

namespace UE::MultiUserClient::Private
{
static bool IsMapAsset(FName PackageName)
{
	FString FullPath;
	if (FPackageName::TryConvertLongPackageNameToFilename(PackageName.ToString(), FullPath, FPackageName::GetMapPackageExtension()))
	{
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*FullPath))
		{
			return true;
		}
	}
	return false;
}

static TSharedPtr<SMessageDialog> MakeReloadDialog(FName PackageName)
{
	static const FText DialogTitle = LOCTEXT("ActiveSessionPendingHotReloadTitle", "A level hot reload is pending.");

	static const FText ReloadButton = LOCTEXT("ActiveSessionPendingHotReloadButton", "Reload");

	const FText ReloadMessage =
		FText::Format(
			LOCTEXT("ActiveSessionPendingHotReload","A hot reload on {0} is pending. Please dismiss this notification to continue with reload. All transactions will be queued until this occurs."),
			 FText::FromName(PackageName)
			);

	return SNew(SMessageDialog)
		.Title(DialogTitle)
		.Message(ReloadMessage)
		.Buttons( { SCustomDialog::FButton(ReloadButton) } );
}

}
void SActiveSessionOverviewTab::ActivityUpdated(const FConcertClientInfo& InClientInfo, const FConcertSyncActivity& InActivity, const FStructOnScope& ActivitySummary)
{
	if (InActivity.EventType == EConcertSyncActivityEventType::Package)
	{
		TSharedPtr<IConcertSyncClient> SyncClientPin = WeakConcertSyncClient.Pin();
		if (!WeakSessionPtr.IsValid() || !SyncClientPin || CanReloadDialog)
		{
			return;
		}

		if (InActivity.EndpointId == WeakSessionPtr.Pin()->GetSessionClientEndpointId())
		{
			return;
		}

		check(ActivitySummary.GetStruct()->IsChildOf(FConcertSyncPackageActivitySummary::StaticStruct()));
		const FConcertSyncPackageActivitySummary* ActivitySummaryPtr =
			(FConcertSyncPackageActivitySummary*)ActivitySummary.GetStructMemory();

		// User changed its settings.
		const UConcertClientConfig* ConcertClientConfig = SyncClientPin->GetConcertClient()->GetConfiguration();
		if (ConcertClientConfig->bShouldPromptForHotReloadOnLevel)
		{
			const bool bIsAMap = UE::MultiUserClient::Private::IsMapAsset(ActivitySummaryPtr->PackageName);
			if (bIsAMap)
			{
				bCanHotReload = false;
				CanReloadDialog = UE::MultiUserClient::Private::MakeReloadDialog(ActivitySummaryPtr->PackageName);
				CanReloadDialog->GetOnWindowClosedEvent().AddLambda([this](const TSharedRef<SWindow>& ClosedWindow)
					{
						bCanHotReload = true;
					});

				CanReloadDialog->Show();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE /* SActiveSession */
