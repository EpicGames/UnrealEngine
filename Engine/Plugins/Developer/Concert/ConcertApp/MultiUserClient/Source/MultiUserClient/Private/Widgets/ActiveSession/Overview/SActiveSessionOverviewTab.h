// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessages.h"
#include "Widgets/ClientSessionHistoryController.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"

class IConcertClientSession;
class IConcertSyncClient;

class FAsyncTaskNotification;
struct FConcertConflictDescriptionBase;
struct FConcertSessionClientInfo;

class SCustomDialog;
class SDockTab;
class SExpandableArea;


/**
 * Displays the multi-users active session's clients and activities.
 */
class SActiveSessionOverviewTab : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SActiveSessionOverviewTab) { }
	SLATE_END_ARGS();

	/**
	* Constructs the active session tab.
	*
	* @param InArgs The Slate argument list.
	*/
	void Construct(const FArguments& InArgs, TSharedPtr<IConcertSyncClient> InConcertSyncClient);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	
	/** Pointer on the client sync. */
	TWeakPtr<IConcertSyncClient> WeakConcertSyncClient;

	/** Holds a concert client session. */
	TWeakPtr<IConcertClientSession> WeakSessionPtr;

	/** List view for AvailableClients. */
	TSharedPtr<SListView<TSharedPtr<FConcertSessionClientInfo>>> ClientsListView;

	/** List of clients for the current session. */
	TArray<TSharedPtr<FConcertSessionClientInfo>> Clients;

	/** Information about the machine's client. */
	TSharedPtr<FConcertSessionClientInfo> ClientInfo;

	/** Holds a concert activity log. */
	TSharedPtr<FClientSessionHistoryController> SessionHistoryController;

	/** The 'Clients' expandable area. */
	TSharedPtr<SExpandableArea> ClientArea;

	/** The 'History' expandable area. */
	TSharedPtr<SExpandableArea> HistoryArea;

	/** Notification handler for hot reload. */
	TSharedPtr<SCustomDialog> CanReloadDialog;

	/** Flag to indicate if it is OK to hotreload the packages. */
	bool bCanHotReload = true;

	/** Keeps the status of 'Clients' area expansion. */
	bool bClientAreaExpanded = true;

	/** Keep the status of 'History' area expansion. */
	bool bHistoryAreaExpanded = true;

	/** Generate a new client row */
	TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FConcertSessionClientInfo> InClientInfo, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Handle a session startup */
	void HandleSessionStartup(TSharedRef<IConcertClientSession> InClientSession);

	/** Handle a session shutdown */
	void HandleSessionShutdown(TSharedRef<IConcertClientSession> InClientSession);

	/** Handle a session client change */
	void HandleSessionClientChanged(IConcertClientSession&, EConcertClientStatus ClientStatus, const FConcertSessionClientInfo& ClientInfo);

	/** Polls the local client info and detect if it changed in order to update its representation in real time.*/
	EActiveTimerReturnType HandleLocalClientInfoChangePollingTimer(double InCurrentTime, float InDeltaTime);

	/** Update the list of clients while keeping the alphabetical sorting */
	void UpdateSessionClientListView(const FConcertSessionClientInfo* Client = nullptr, EConcertClientStatus Status = EConcertClientStatus::Updated);

	/** Set the selected client in the clients list view*/
	void SetSelectedClient(const FGuid& InClientEndpointId, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Find a client with its endpoint id */
	TSharedPtr<FConcertSessionClientInfo> FindAvailableClient(const FGuid& InClientEndpointId) const;

	/** Handles how much space the 'Clients' area uses with respect to its expansion state. */
	SSplitter::ESizeRule GetClientAreaSizeRule() const { return bClientAreaExpanded ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent; }
	void OnClientAreaExpansionChanged(bool bExpanded) { bClientAreaExpanded = bExpanded; }

	/** Handles how much space the 'History' area uses with respect to its expansion state. */
	SSplitter::ESizeRule GetHistoryAreaSizeRule() const { return bHistoryAreaExpanded ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent; }
	void OnHistoryAreaExpansionChanged(bool bExpanded) { bHistoryAreaExpanded = bExpanded; }

	/** Delegate handler when a conflict between inbound transaction and those stored pending outbound transactions. */
	void OnSendConflict(const FConcertConflictDescriptionBase& ConflictMsg);

	/** Delegate handler on inbound transactions that focus specifically on packages. */
	void OnPackageChangeActivity();

	/** Delegate handler to indicate if we can hot reload an inbound package. */
	bool CanProcessPendingPackages() const;

	/** Delegate handler invoked when an activity has been received by multi-user. */
	void ActivityUpdated(const FConcertClientInfo& InClientInfo, const FConcertSyncActivity& InActivity, const FStructOnScope& /*unused*/);
};
