// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

class IConcertClient;
class FMenuBuilder;

namespace UE::ConcertSharedSlate
{
	class IMultiReplicationStreamEditor;
	class IObjectHierarchyModel;
}
namespace UE::MultiUserClient::Replication
{
	class FReassignObjectPropertiesLogic;
	class FOnlineClientManager;
}

namespace UE::MultiUserClient::Replication::ContextMenuUtils
{
	/** Adds menu entries for reassigning the object to another client. */
	void AddReassignmentOptions(
		FMenuBuilder& MenuBuilder,
		const TSoftObjectPtr<>& ContextObject,
		const IConcertClient& ConcertClient,
		const FOnlineClientManager& ReplicationManager,
		ConcertSharedSlate::IObjectHierarchyModel& ObjectHierarchy,
		FReassignObjectPropertiesLogic& ReassignmentLogic,
		ConcertSharedSlate::IMultiReplicationStreamEditor& MultiStreamEditor
		);
	
	/** Adds an edit box for batch reassigning the select object's frequencies for all replicating clients. */
	void AddFrequencyOptionsForMultipleClients(FMenuBuilder& MenuBuilder, const FSoftObjectPath& ContextObjects, FOnlineClientManager& InClientManager);
	/** Adds an edit box for batch reassigning the select object's frequencies for all replicating clients. */
	inline void AddFrequencyOptionsIfOneContextObject_MultiClient(FMenuBuilder& MenuBuilder, TConstArrayView<TSoftObjectPtr<>> ContextObjects, FOnlineClientManager& InClientManager)
	{
		if (ContextObjects.Num() == 1)
		{
			AddFrequencyOptionsForMultipleClients(MenuBuilder, ContextObjects[0].GetUniqueID(), InClientManager);
		}
	}
}