// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextMenuUtils.h"

#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Misc/Frequency/FrequencyUtils.h"
#include "Replication/Misc/Util/SynchronizedRequestUtils.h"

#include "Algo/AnyOf.h"
#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Replication/ObjectNameUtils.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SScaleBox.h"

#define LOCTEXT_NAMESPACE "FrequencyContextMenuUtils"

namespace UE::MultiUserClient::Replication::ContextMenuUtils
{
	namespace Private
	{
		using FInlineClientArray = FrequencyUtils::FInlineClientArray;

		static FText GetRealtimeTooltip() { return LOCTEXT("ListView_Realtime.ToolTip", "The object will replicate as fast as possible (every tick)."); }
		static FText GetSpecifiedRateTooltip() { return LOCTEXT("ListView_SpecifiedRate.ToolTip", "The object replicates at the specified rate per second."); }
		static FText GetEditBoxInstructionsTooltip() { return LOCTEXT("Frequency.ToolTip", "Specifies how often this object should replicate in 1 second."); }

		/** Displayed in context menu for changing an object's replication frequency. */
		class SFrequencyNumericBox : public SCompoundWidget
		{
		public:

			SLATE_BEGIN_ARGS(SFrequencyNumericBox)
			{}
				SLATE_ARGUMENT(FSoftObjectPath, SelectedObject)
				SLATE_ATTRIBUTE(FInlineClientArray, Clients)
			SLATE_END_ARGS()
			
			void Construct(const FArguments& InArgs, FOnlineClientManager& InClientManager)
			{
				ContextObject = InArgs._SelectedObject;
				ClientsAttribute = InArgs._Clients;
				ClientManager = &InClientManager;
				
				ChildSlot
				[
					SNew(SBox)
					.MinDesiredWidth(100.f)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.FillWidth(1.f)
						[
							SNew(SNumericEntryBox<uint8>)
							.AllowSpin(false)
							.IsEnabled(this, &SFrequencyNumericBox::GetIsEnabled)
							.Value(this, &SFrequencyNumericBox::GetValue)
							.OnValueCommitted(this, &SFrequencyNumericBox::OnValueCommitted)
							.ToolTipText(this, &SFrequencyNumericBox::GetToolTipText)
							.UndeterminedString(LOCTEXT("Single.Undetermined", "n/a"))
						]
						
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SScaleBox)
							[
								SNew(SThrobber)
								.Visibility_Lambda([this](){ return InProgressChange.IsSet() ? EVisibility::HitTestInvisible : EVisibility::Collapsed; })
							]
						]
					]
				];
			}

		private:

			FSoftObjectPath ContextObject;
			TAttribute<FInlineClientArray> ClientsAttribute;
			FOnlineClientManager* ClientManager = nullptr;

			struct FChangeOperation
			{
				TSharedPtr<IParallelSubmissionOperation> ChangeOperation;
				uint8 ValueBeingSet;
			};
			/** Set for as long a change operation is in progress. */
			TOptional<FChangeOperation> InProgressChange;

			/** @return Gets all clients that have the given mode set for ContextObject. */
			FInlineClientArray GetClientsWithMode(const EConcertObjectReplicationMode Mode) const
			{
				FInlineClientArray Result = ClientsAttribute.Get();
				for (auto It = Result.CreateIterator(); It; ++It)
				{
					const TOptional<EConcertObjectReplicationMode> ClientMode = FrequencyUtils::FindSharedReplicationMode(ContextObject, { *It }, *ClientManager);
					if (!ClientMode || *ClientMode != Mode)
					{
						It.RemoveCurrent();
					}
				}
				return Result;
			}

			/** @return Whether all clients with specified rate can be changed. */
			bool GetIsEnabled() const
			{
				const FInlineClientArray Clients = GetClientsWithMode(EConcertObjectReplicationMode::SpecifiedRate);
				return !InProgressChange && !Clients.IsEmpty() && FrequencyUtils::CanChangeFrequencySettings(ContextObject, Clients, *ClientManager);
			}
			
			void OnValueCommitted(uint8 NewValue, ETextCommit::Type CommitType)
			{
				if (CommitType != ETextCommit::OnEnter || NewValue == 0 || InProgressChange.IsSet())
				{
					return;
				}
			
				const TSharedPtr<IParallelSubmissionOperation> ChangeOperation = FrequencyUtils::SetFrequencySettingForClients(ContextObject, GetClientsWithMode(EConcertObjectReplicationMode::SpecifiedRate), *ClientManager,
					[NewValue](FConcertObjectReplicationSettings& Override)
					{
						Override.ReplicationRate = NewValue;
					});
				InProgressChange = { ChangeOperation, NewValue };

				// For better UX, while we're waiting on the server to progress the request this widget should show what the user entered last.
				// Once the request completes, the widget will show what the server state (so it may revert if the request failed, e.g. due to time out).
				InProgressChange->ChangeOperation->OnCompletedFuture_AnyThread()
					.Next([WeakThis = TWeakPtr<SFrequencyNumericBox>(SharedThis(this))](FParallelExecutionResult&& ExecutionResult)
					{
						// User may have close the menu containing this widget before the request completed.
						if (const TSharedPtr<SFrequencyNumericBox> ThisPin = WeakThis.Pin())
						{
							// GuardedExecuteOnGameThread internally checks that "this" is valid before executing lambda
							ThisPin->GuardedExecuteOnGameThread([WeakThis]() { WeakThis.Pin()->InProgressChange.Reset(); });
						}
					});
			}

			void GuardedExecuteOnGameThread(TUniqueFunction<void()> Callback) const
			{
				if (IsInGameThread())
				{
					Callback();
				}
				else
				{
					AsyncTask(ENamedThreads::GameThread, [Callback = MoveTemp(Callback), WeakThis = AsWeak()]()
					{
						if (WeakThis.IsValid())
						{
							Callback();
						}
					});
				}
			}

			TOptional<uint8> GetValue() const
			{
				// For better UX, if a change is in progress show whatever the user specified for the request. Once the request finishes, show whatever the state is on the server.
				return InProgressChange
					? InProgressChange->ValueBeingSet
					: FrequencyUtils::FindSharedFrequencyRate(ContextObject, GetClientsWithMode(EConcertObjectReplicationMode::SpecifiedRate), *ClientManager);
			}
			
			FText GetToolTipText() const
			{
				if (GetValue())
				{
					return GetEditBoxInstructionsTooltip();
				}

				const TOptional<EConcertObjectReplicationMode> Mode = FrequencyUtils::FindSharedReplicationMode(ContextObject, GetClientsWithMode(EConcertObjectReplicationMode::SpecifiedRate), *ClientManager);
				return Mode
					? LOCTEXT("NotApplicable.Mixed", "Multiple clients are replicating this object with different rates.")
					: LOCTEXT("NotApplicable.AllRealtime", "This object is repliacting in realtime");
			}
		};

		static void AddReplicationModeSubMenu(FMenuBuilder& MenuBuilder, const FSoftObjectPath& ContextObject, TAttribute<FInlineClientArray> Clients, FOnlineClientManager& InClientManager)
		{
			const auto SetReplicationMode = [ContextObject, Clients, &InClientManager](const EConcertObjectReplicationMode ModeToSet)
			{
				FrequencyUtils::SetFrequencySettingForClients(ContextObject, Clients.Get(), InClientManager,
					[ModeToSet](FConcertObjectReplicationSettings& SettingToOverride)
					{
						SettingToOverride.ReplicationMode = ModeToSet;
					});
			};

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ListView_Realtime.Label", "Realtime"),
				GetRealtimeTooltip(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([SetReplicationMode]()
					{
						SetReplicationMode(EConcertObjectReplicationMode::Realtime);
					}),
					FCanExecuteAction::CreateLambda([ContextObject, Clients, &InClientManager]()
					{
						return FrequencyUtils::CanChangeFrequencySettings(ContextObject, Clients.Get(), InClientManager);
					}),
					FIsActionChecked::CreateLambda([ContextObject, Clients, &InClientManager]()
					{
						return FrequencyUtils::AllClientsHaveMode(EConcertObjectReplicationMode::Realtime, ContextObject, Clients.Get(), InClientManager);
					})
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
			
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ListView_SpecifiedRate.Label", "Specified rate"),
				GetSpecifiedRateTooltip(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([SetReplicationMode]()
					{
						SetReplicationMode(EConcertObjectReplicationMode::SpecifiedRate);
					}),
					FCanExecuteAction::CreateLambda([ContextObject, Clients, &InClientManager]()
					{
						return FrequencyUtils::CanChangeFrequencySettings(ContextObject, Clients.Get(), InClientManager);
					}),
					FIsActionChecked::CreateLambda([ContextObject, Clients, &InClientManager]()
					{
						return FrequencyUtils::AllClientsHaveMode(EConcertObjectReplicationMode::SpecifiedRate, ContextObject, Clients.Get(), InClientManager);
					})
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
				);
		}

		static void AppendFrequencyEditingToMenu(
			FMenuBuilder& MenuBuilder,
			const FSoftObjectPath& ContextObject,
			TAttribute<FInlineClientArray> GetClientsAttribute,
			FOnlineClientManager& InClientManager
			)
		{
			// Change to Realtime / Specified Rate
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("Frequency.Mode", "Mode"));
			AddReplicationModeSubMenu(MenuBuilder, ContextObject, GetClientsAttribute, InClientManager);
			MenuBuilder.EndSection();

			MenuBuilder.AddSeparator();

			// Replication rate, only enabled if Specified Rate
			constexpr bool bNoIndent = false;
			constexpr bool bSearchable = true;
			MenuBuilder.AddWidget(
				SNew(Private::SFrequencyNumericBox, InClientManager)
					.SelectedObject(ContextObject)
					.Clients(MoveTemp(GetClientsAttribute)),
				LOCTEXT("Frequency.Label", "Replication Rate"),
				bNoIndent,
				bSearchable,
				GetEditBoxInstructionsTooltip()
				);
		}
		
		static void AddSubMenuForEachObjectInHierarchy(
			FMenuBuilder& MenuBuilder,
			const FSoftObjectPath& ContextObject,
			TFunction<FInlineClientArray(const FSoftObjectPath& Object)> GetClientsForObjectFunc,
			FOnlineClientManager& InClientManager
			)
		{
			bool bAddedSubmenu = false;
			const auto OnAddSubmenu = [&MenuBuilder, &bAddedSubmenu]()
			{
				if (!bAddedSubmenu)
				{
					bAddedSubmenu = true;
					MenuBuilder.BeginSection(NAME_None, LOCTEXT("Frequency.Change", "Change Frequency"));
				}
			};
			ON_SCOPE_EXIT{ if(bAddedSubmenu){ MenuBuilder.EndSection(); } };
			
			const auto MakeSubmenu = [&MenuBuilder, &GetClientsForObjectFunc, &InClientManager, &OnAddSubmenu](const FSoftObjectPath& Object)
			{
				OnAddSubmenu();

				constexpr bool bOpenOnClick = false;
				constexpr bool bCloseWindowAfterMenuSelection = false;
				MenuBuilder.AddSubMenu(
					ConcertClientSharedSlate::GetObjectDisplayName(TSoftObjectPtr(Object)),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateLambda([Object, GetClientsForObjectFunc, &InClientManager](FMenuBuilder& MenuBuilder)
					{
						TAttribute<FInlineClientArray> ClientAttribute = TAttribute<FInlineClientArray>::CreateLambda([Object, GetClientsForObjectFunc]()
						{
							return GetClientsForObjectFunc(Object);
						});
						AppendFrequencyEditingToMenu(MenuBuilder, Object, MoveTemp(ClientAttribute), InClientManager);
					}), bOpenOnClick, FSlateIcon(), bCloseWindowAfterMenuSelection);
			};

			if (!GetClientsForObjectFunc(ContextObject).IsEmpty())
			{
				MakeSubmenu(ContextObject);
			}
			
			using namespace ConcertSyncCore;
			const FObjectPathHierarchy& Hierarchy = InClientManager.GetAuthorityCache().GetStreamObjectHierarchy();
			Hierarchy.TraverseTopToBottom([&MakeSubmenu](const FChildRelation& Relation)
			{
				MakeSubmenu(Relation.Child.Object);
				return ETreeTraversalBehavior::Continue;
			}, ContextObject);
		}
	}

	void AddFrequencyOptionsForMultipleClients(FMenuBuilder& MenuBuilder, const FSoftObjectPath& ContextObject, FOnlineClientManager& InClientManager)
	{
		using namespace Private;
		const auto GetClientsForObject = [&InClientManager](const FSoftObjectPath& Object)
		{
			FInlineClientArray Result;
			InClientManager.ForEachClient([&Object, &Result](const FOnlineClient& Client)
			{
				// Only clients that have the object registered should be considered
				if (Client.GetStreamSynchronizer().GetServerState().HasProperties(Object))
				{
					Result.Add(Client.GetEndpointId());
				}
				return EBreakBehavior::Continue;
			});
			return Result;
		};
		
		AddSubMenuForEachObjectInHierarchy(MenuBuilder, ContextObject, GetClientsForObject, InClientManager);
	}
}

#undef LOCTEXT_NAMESPACE