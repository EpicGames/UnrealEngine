// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextMenuUtils.h"

#include "Replication/Client/ClientUtils.h"
#include "Replication/Client/Online/OnlineClient.h"
#include "Replication/Editor/Model/Object/IObjectHierarchyModel.h"
#include "Replication/Editor/View/IMultiReplicationStreamEditor.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"
#include "Replication/Submission/MultiEdit/ReassignObjectPropertiesLogic.h"
#include "Widgets/ActiveSession/Replication/Client/Multi/Columns/MultiStreamColumns.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Textures/SlateIcon.h"

#define LOCTEXT_NAMESPACE "ReassignmentContextMenuUtils"

namespace UE::MultiUserClient::Replication::ContextMenuUtils
{
	namespace Private
	{
		using FInlineAllocator = TInlineAllocator<24>;
		using FInlineObjectPathArray = TArray<FSoftObjectPath, FInlineAllocator>;
		
		static FInlineObjectPathArray GetChildrenOfManagedObject(ConcertSharedSlate::IObjectHierarchyModel& ObjectHierarchy, TSoftObjectPtr<> ManagedObject)
		{
			FInlineObjectPathArray Result;
			Algo::Transform(ObjectHierarchy.GetChildrenRecursive<FInlineAllocator>(ManagedObject), Result, [](const TSoftObjectPtr<>& Object)
			{
				return Object.GetUniqueID();
			});
			return Result;
		}
		
		static void AddReassignSection(
			FMenuBuilder& MenuBuilder,
			const TArray<const FOnlineClient*>& SortedClients,
			TAttribute<FInlineObjectPathArray> ObjectsToAssign,
			const IConcertClient& ConcertClient,
			FReassignObjectPropertiesLogic& ReassignmentLogic,
			ConcertSharedSlate::IMultiReplicationStreamEditor& MultiStreamEditor
			)
		{
			for (const FOnlineClient* Client : SortedClients)
			{
				const FGuid& ClientId = Client->GetEndpointId();
				MenuBuilder.AddMenuEntry(
					FText::FromString(ClientUtils::GetClientDisplayName(ConcertClient, Client->GetEndpointId())),
					TAttribute<FText>::CreateLambda([ClientId, ObjectsToAssign, &ReassignmentLogic]()
					{
						FText CannotEditReason;
						const bool bCanReassign = ReassignmentLogic.CanReassignAnyTo(ObjectsToAssign.Get(), ClientId, &CannotEditReason);
						return bCanReassign ? LOCTEXT("DoReassign", "Reassign to this client") : CannotEditReason;
					}),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([ObjectsToAssign, ClientId, &ReassignmentLogic, &MultiStreamEditor]()
						{
							TArray<FSoftObjectPath> ObjectsNotInline;
							Algo::Transform(ObjectsToAssign.Get(), ObjectsNotInline, [](const FSoftObjectPath& ObjectPath){ return ObjectPath; });
							ReassignmentLogic.ReassignAllTo(ObjectsNotInline, ClientId);
							MultiStreamEditor.GetEditorBase().RequestObjectColumnResort(MultiStreamColumns::AssignedClientsColumnId);
						}),
						FCanExecuteAction::CreateLambda([ObjectsToAssign, ClientId, &ReassignmentLogic](){ return ReassignmentLogic.CanReassignAnyTo(ObjectsToAssign.Get(), ClientId); }),
						FIsActionChecked::CreateLambda([ObjectsToAssign, ClientId, &ReassignmentLogic](){ return ReassignmentLogic.OwnsAnyOf(ObjectsToAssign.Get(), ClientId); })
						),
						NAME_None,
						EUserInterfaceActionType::Check
					);
			}
		}
	}
	
	void AddReassignmentOptions(
		FMenuBuilder& MenuBuilder,
		const TSoftObjectPtr<>& ContextObject,
		const IConcertClient& ConcertClient,
		const FOnlineClientManager& ReplicationManager,
		ConcertSharedSlate::IObjectHierarchyModel& ObjectHierarchy,
		FReassignObjectPropertiesLogic& ReassignmentLogic,
		ConcertSharedSlate::IMultiReplicationStreamEditor& MultiStreamEditor
		)
	{
		const TArray<const FOnlineClient*> SortedClients = ClientUtils::GetSortedClientList(ConcertClient, ReplicationManager);
		
		MenuBuilder.BeginSection(TEXT("Reassign.All"), LOCTEXT("Reassign.All", "Reassign all to"));
		Private::AddReassignSection(MenuBuilder, SortedClients,
			TAttribute<Private::FInlineObjectPathArray>::CreateLambda([ContextObject, &ObjectHierarchy]()
			{
				Private::FInlineObjectPathArray Result = Private::GetChildrenOfManagedObject(ObjectHierarchy, ContextObject);
				Result.Add(ContextObject.GetUniqueID());
				return Result;
			}),
			ConcertClient, ReassignmentLogic, MultiStreamEditor
			);
		MenuBuilder.EndSection();
	}
}

#undef LOCTEXT_NAMESPACE