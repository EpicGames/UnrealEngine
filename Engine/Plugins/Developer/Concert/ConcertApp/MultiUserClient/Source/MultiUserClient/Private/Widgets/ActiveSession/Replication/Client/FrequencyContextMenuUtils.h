// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "UObject/SoftObjectPath.h"

class FMenuBuilder;
struct FGuid;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
}

namespace UE::MultiUserClient::Replication::FrequencyContextMenuUtils
{
	/** Adds an edit box for changing the selected object's frequencies. */
	void AddFrequencyOptionsForSingleClient(FMenuBuilder& MenuBuilder, const FSoftObjectPath& ContextObjects, const FGuid& ClientId, FOnlineClientManager& InClientManager);
	/** Adds an edit box for batch reassigning the select object's frequencies for all replicating clients. */
	void AddFrequencyOptionsForMultipleClients(FMenuBuilder& MenuBuilder, const FSoftObjectPath& ContextObjects, FOnlineClientManager& InClientManager);
	
	/** Adds an edit box for changing the selected object's frequencies. */
	inline void AddFrequencyOptionsIfOneContextObject_SingleClient(FMenuBuilder& MenuBuilder, TConstArrayView<FSoftObjectPath> ContextObjects, const FGuid& ClientId, FOnlineClientManager& InClientManager)
	{
		if (ContextObjects.Num() == 1)
		{
			AddFrequencyOptionsForSingleClient(MenuBuilder, ContextObjects[0], ClientId, InClientManager);
		}
	}
	/** Adds an edit box for batch reassigning the select object's frequencies for all replicating clients. */
	inline void AddFrequencyOptionsIfOneContextObject_MultiClient(FMenuBuilder& MenuBuilder, TConstArrayView<FSoftObjectPath> ContextObjects, FOnlineClientManager& InClientManager)
	{
		if (ContextObjects.Num() == 1)
		{
			AddFrequencyOptionsForMultipleClients(MenuBuilder, ContextObjects[0], InClientManager);
		}
	}
}
