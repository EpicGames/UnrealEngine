// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssignPropertyModel.h"

#include "ConcertLogGlobal.h"
#include "Replication/Editor/Model/PropertyUtils.h"
#include "Replication/Client/UnifiedClientView.h"
#include "Widgets/ActiveSession/Replication/Client/Multi/ViewOptions/MultiViewOptions.h"

#include "GameFramework/Actor.h"
#include "ScopedTransaction.h"

#include <type_traits>

#include "Misc/ObjectUtils.h"

#define LOCTEXT_NAMESPACE "FAssignPropertyModel"

namespace UE::MultiUserClient::Replication::MultiStreamColumns::AssignPropertyModel
{
	static void AssignPropertyTo(
		ConcertSharedSlate::IEditableReplicationStreamModel& ClientEditModel,
		TConstArrayView<TSoftObjectPtr<>> Objects,
		const FConcertPropertyChain& Property
		)
	{
		for (const TSoftObjectPtr<>& Object : Objects)
		{
			const FSoftObjectPath& ObjectPath = Object.GetUniqueID();
			if (!ClientEditModel.ContainsObjects({ ObjectPath }))
			{
				ClientEditModel.AddObjects({ Object.Get() });
			}

			const FSoftClassPath ClassPath = ClientEditModel.GetObjectClass(ObjectPath);
			TArray AddedProperties { Property };
			ConcertClientSharedSlate::PropertyUtils::AppendAdditionalPropertiesToAdd(ClassPath, AddedProperties);
			ClientEditModel.AddProperties(ObjectPath, AddedProperties);
		}
	}

	static void AddOwningActorIfHierarchyIsEmpty(ConcertSharedSlate::IEditableReplicationStreamModel& ClientEditModel, const FSoftObjectPath& ObjectPath)
	{
		const TOptional<FSoftObjectPath> ActorPath = ConcertSyncCore::GetActorPathIn(ObjectPath);
		UObject* ResolvedObject = ActorPath ? ActorPath->ResolveObject() : nullptr;
		if (!ResolvedObject || ClientEditModel.ContainsObjects({ *ActorPath }))
		{
			return;
		}

		bool bHasChildren = false;
		ClientEditModel.ForEachSubobject(*ActorPath, [&bHasChildren](auto)
		{
			bHasChildren = true;
			return bHasChildren ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		if (!bHasChildren)
		{
			ClientEditModel.AddObjects({ ResolvedObject });
		}
	}
	
	static void RemovePropertiesFromClient(
		ConcertSharedSlate::IEditableReplicationStreamModel& ClientEditModel,
		TConstArrayView<TSoftObjectPtr<>> Objects,
		const FConcertPropertyChain& Property,
		bool bIsLocalClient
		)
	{
		for (const TSoftObjectPtr<>& Object : Objects)
		{
			const FSoftObjectPath& ObjectPath = Object.GetUniqueID();
			const FSoftClassPath ClassPath = ClientEditModel.GetObjectClass(ObjectPath);
			ClientEditModel.RemoveProperties(ObjectPath, { Property });
					
			if (ClientEditModel.HasAnyPropertyAssigned(ObjectPath))
			{
				continue;
			}

			// We want to remove subobjects that have no properties. Retain actors because they cause their entire component / subobject hierarchy to be displayed.
			// Skipping this check would close the entire property tree view and remove the actor hierarchy from the view.
			// That would feel very unnatural / unexpected for the user. 
			// If the user does not want the actor anymore, they should click it and delete it.
			const UClass* ObjectClass = ClassPath.IsValid() ? ClassPath.TryLoadClass<UObject>() : nullptr;
			UE_CLOGF(ClassPath.IsValid() && !ObjectClass, LogConcert, Warning, "SAssignPropertyComboBox: Failed to resolve class %ls", *ClassPath.ToString());
			const bool bIsSubobject = ObjectClass && !ObjectClass->IsChildOf<AActor>();
			if (bIsSubobject)
			{
				ClientEditModel.RemoveObjects({ ObjectPath });
			}

			// Scenario: 1. We had nothing assigned, 2. Remote client assigns some property of some component to us 3. Now, the property is cleared.
			// The remote assignment op from step 2 does not add the owning actor.
			// If the local client is clearing the property, we'd now remove the last object from the hierarchy, thus removing it from the UI.
			// That feels unnatural. To prevent it, add the owning actor to keep the hierarchy in the UI.
			if (bIsLocalClient)
			{
				AddOwningActorIfHierarchyIsEmpty(ClientEditModel, ObjectPath);
			}
		}
	}
	
	template<typename TShouldRemove>
	requires std::is_invocable_r_v<bool, TShouldRemove, const FGuid&>
	static void UnassignPropertyFromClients(
		const FUnifiedClientView& ClientView,
		TConstArrayView<TSoftObjectPtr<>> Objects,
		const FConcertPropertyChain& Property,
		TShouldRemove&& ShouldRemoveFromClient
		)
	{
		ClientView.ForEachOnlineClient([&ClientView, &Objects, &Property, &ShouldRemoveFromClient](const FGuid& EndpointId)
		{
			const TSharedPtr<ConcertSharedSlate::IEditableReplicationStreamModel> Stream = ClientView.GetEditableClientStreamById(EndpointId);
			if (Stream && ShouldRemoveFromClient(EndpointId))
			{
				const bool bIsLocalClient = EndpointId == ClientView.GetLocalClient();
				RemovePropertiesFromClient(*Stream, Objects, Property, bIsLocalClient);
			}
			return EBreakBehavior::Continue;
		});
	}
}

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	void FAssignPropertyModel::AssignPropertyTo(
		const FUnifiedClientView& ClientView,
		const FGuid& ClientId,
		TConstArrayView<TSoftObjectPtr<>> Objects,
		const FConcertPropertyChain& Property
		)
	{
		if (const TSharedPtr<ConcertSharedSlate::IEditableReplicationStreamModel> StreamModel = ClientView.GetEditableClientStreamById(ClientId))
		{
			// Remove the property from all clients but the one we'll assign to ...
			AssignPropertyModel::UnassignPropertyFromClients(ClientView, Objects, Property,
				[ClientId](const FGuid& ClientToRemoveFrom){ return ClientId != ClientToRemoveFrom; }
				);

			// ... and then assign the property
			AssignPropertyModel::AssignPropertyTo(*StreamModel, Objects, Property);
		}
	}

	FAssignPropertyModel::FAssignPropertyModel(FUnifiedClientView& InClientView, FMultiViewOptions& InViewOptions)
		: ClientView(InClientView)
		, ViewOptions(InViewOptions)
	{
		ClientView.OnClientsChanged().AddRaw(this, &FAssignPropertyModel::BroadcastOnOwnershipChanged);
		ClientView.GetStreamCache().OnCacheChanged().AddRaw(this, &FAssignPropertyModel::BroadcastOnOwnershipChanged);
		ViewOptions.OnOptionsChanged().AddRaw(this, &FAssignPropertyModel::BroadcastOnOwnershipChanged);
	}

	FAssignPropertyModel::~FAssignPropertyModel()
	{
		ClientView.OnClientsChanged().RemoveAll(this);
		ClientView.GetStreamCache().OnCacheChanged().RemoveAll(this);
		ViewOptions.OnOptionsChanged().RemoveAll(this);
	}

#define SET_REASON(Text) if (Reason) { *Reason = Text; }
	bool FAssignPropertyModel::CanChangePropertyFor(const FGuid& ClientId, FText* Reason) const
	{
		const TOptional<EClientType> ClientType = ClientView.GetClientType(ClientId);
		if (!ClientType || !IsOnlineClient(*ClientType))
		{
			SET_REASON(LOCTEXT("ClientDisconnected", "Client is not online."));
			return false;
		}
		
		return true;
	}
#undef SET_REASON

	bool FAssignPropertyModel::CanClear(TConstArrayView<TSoftObjectPtr<>> Objects, const FConcertPropertyChain& Property) const
	{
		bool bIsAssignedToAnyClient = false;
		ClientView.ForEachOnlineClient([this, &Objects, &Property, &bIsAssignedToAnyClient](const FGuid& EndpointId)
		{
			const TSharedPtr<const ConcertSharedSlate::IReplicationStreamModel> StreamModel = ClientView.GetClientStreamById(EndpointId);
			check(StreamModel);
			
			for (const TSoftObjectPtr<>& EditedObject : Objects)
			{
				if (bIsAssignedToAnyClient)
				{
					break;
				}
				
				const bool bHasProperty = StreamModel->HasProperty(EditedObject.GetUniqueID(), Property);
				bIsAssignedToAnyClient |= bHasProperty;
			}
			
			return bIsAssignedToAnyClient ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		return bIsAssignedToAnyClient;
	}

	EPropertyOnObjectsOwnershipState FAssignPropertyModel::GetPropertyOwnershipState(const FGuid& ClientId, TConstArrayView<TSoftObjectPtr<>> Objects, const FConcertPropertyChain& Property) const
	{
		const TSharedPtr<const ConcertSharedSlate::IReplicationStreamModel> StreamModel = ClientView.GetClientStreamById(ClientId);
		const TOptional<EClientType> ClientType = ClientView.GetClientType(ClientId);
		// Remote clients can disconnect after the combo-box is opened.
		if (!ClientType || !IsOnlineClient(*ClientType) || !StreamModel)
		{
			return EPropertyOnObjectsOwnershipState::NotOwnedOnAllObjects;
		}

		EPropertyOnObjectsOwnershipState Result = EPropertyOnObjectsOwnershipState::Mixed;
		for (const TSoftObjectPtr<>& ObjectPath : Objects)
		{
			const bool bHasProperty = StreamModel->HasProperty(ObjectPath.GetUniqueID(), Property);
			const EPropertyOnObjectsOwnershipState ExpectedState = bHasProperty
				? EPropertyOnObjectsOwnershipState::OwnedOnAllObjects
				: EPropertyOnObjectsOwnershipState::NotOwnedOnAllObjects;
			
			if (Result == EPropertyOnObjectsOwnershipState::Mixed)
			{
				Result = ExpectedState;
				continue;
			}

			if (Result != ExpectedState)
			{
				return EPropertyOnObjectsOwnershipState::Mixed;
			}
		}
		return Result;
	}

	void FAssignPropertyModel::TogglePropertyFor(
		const FGuid& ClientId,
		TConstArrayView<TSoftObjectPtr<>> Objects,
		const FConcertPropertyChain& Property
		) const
	{
		// Remote clients can disconnect after the combo-box is opened.
		if (!ClientView.GetEditableClientStreamById(ClientId))
		{
			return;
		}
		
		const FText TransactionText = FText::Format(
			LOCTEXT("AllClientsAssignFmt", "Assign {0} property"),
			FText::FromString(Property.ToString(FConcertPropertyChain::EToStringMethod::LeafProperty))
			);
		FScopedTransaction Transaction(TransactionText);

		const EPropertyOnObjectsOwnershipState OwnershipState = GetPropertyOwnershipState(ClientId, Objects, Property);
		const bool bRemovePropertyFromEditedClient = OwnershipState == EPropertyOnObjectsOwnershipState::OwnedOnAllObjects;
		
		// To make it simpler for the user, at most one client is supposed to be assigned to the object at any given time so ...
		if (bRemovePropertyFromEditedClient)
		{
			// ... remove property from all clients
			ClearProperty(Objects, Property);
		}
		else
		{
			AssignPropertyTo(ClientView, ClientId, Objects, Property);
		}
	}

	void FAssignPropertyModel::ClearProperty(TConstArrayView<TSoftObjectPtr<>> Objects, const FConcertPropertyChain& Property) const
	{
		if (CanClear(Objects, Property))
		{
			const FText TransactionText = FText::Format(
				LOCTEXT("ClearAllClientsFmt", "Clear {0} property"),
				FText::FromString(Property.ToString(FConcertPropertyChain::EToStringMethod::LeafProperty))
				);
			FScopedTransaction Transaction(TransactionText);
			
			AssignPropertyModel::UnassignPropertyFromClients(ClientView, Objects, Property, [](auto&){ return true; });
		}
	}

	void FAssignPropertyModel::ForEachAssignedClient(
		const FConcertPropertyChain& DisplayedProperty,
		 const TArray<TSoftObjectPtr<>>& EditedObjects,
		 TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback
		 ) const
	{
		const EClientEnumerationMode Mode = ViewOptions.ShouldShowOfflineClients()
			? EClientEnumerationMode::SkipOfflineClientsThatFullyOverlapWithOnlineClients
			: EClientEnumerationMode::SkipOfflineClients;
		for (const TSoftObjectPtr<>& Object : EditedObjects)
		{
			ClientView.GetStreamCache().EnumerateClientsWithObjectAndProperty(
				Object.ToSoftObjectPath(),
				DisplayedProperty,
				[&Callback](const FGuid& ClientId){ return Callback(ClientId); },
				Mode
			);
		}
	}
}

#undef LOCTEXT_NAMESPACE