// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/SoftObjectPtr.h"

enum class EBreakBehavior : uint8;
struct FConcertPropertyChain;

namespace UE::ConcertSharedSlate { class IEditableReplicationStreamModel; }
namespace UE::MultiUserClient::Replication
{
	class FMultiViewOptions;
	class FOnlineClient;
	class FOnlineClientManager;
	class FUnifiedClientView;
}

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	enum class EPropertyOnObjectsOwnershipState : uint8
	{
		/** Clients owns the property on all objects */
		OwnedOnAllObjects,
		/** does not own the property on any object */
		NotOwnedOnAllObjects,
		
		Mixed
	};

	/** Implements the model in the MVC pattern for the property assignment column. */
	class FAssignPropertyModel : public FNoncopyable
	{
	public:

		/** Assigns the property to the given client and unassigns it from all others. */
		static void AssignPropertyTo(
			const FUnifiedClientView& ClientView,
			const FGuid& ClientId,
			TConstArrayView<TSoftObjectPtr<>> Objects,
			const FConcertPropertyChain& Property
		);
		
		FAssignPropertyModel(FUnifiedClientView& InClientView UE_LIFETIMEBOUND, FMultiViewOptions& InViewOptions UE_LIFETIMEBOUND);
		~FAssignPropertyModel();

		/** @return Whether property ownership can be changed for the given client. */
		bool CanChangePropertyFor(const FGuid& ClientId, FText* Reason = nullptr) const;
		/** @return Whether it is valid to call ClearProperty with this parameters. */
		bool CanClear(TConstArrayView<TSoftObjectPtr<>> Objects, const FConcertPropertyChain& Property) const;
		/** @return How the property is owned by ClientId */
		EPropertyOnObjectsOwnershipState GetPropertyOwnershipState(const FGuid& ClientId, TConstArrayView<TSoftObjectPtr<>> Objects, const FConcertPropertyChain& Property) const;

		/** Assigns the property to ClientId or unassigns it. Removes the property from all other clients in both cases. */
		void TogglePropertyFor(const FGuid& ClientId, TConstArrayView<TSoftObjectPtr<>> Objects, const FConcertPropertyChain& Property) const;
		/** Removes the property from all clients. */
		void ClearProperty(TConstArrayView<TSoftObjectPtr<>> Objects, const FConcertPropertyChain& Property) const;

		/** Iterates every client that has DisplayedProperty assigned for any of EditedObjects. */
		void ForEachAssignedClient(
			const FConcertPropertyChain& DisplayedProperty,
			const TArray<TSoftObjectPtr<>>& EditedObjects,
			TFunctionRef<EBreakBehavior(const FGuid& ClientId)> Callback
			) const;
		
		/** Broadcasts when property ownership may have changed. */
		FSimpleMulticastDelegate& OnOwnershipChanged() { return OnOwnershipChangedDelegate; }

	private:

		/**
		 * Used to
		 * - get all online clients that can be assigned / reassigned to,
		 * - detect changes made to client content for broadcasting OnOwnershipChangedDelegate
		 */
		FUnifiedClientView& ClientView;
		/** Controls whether offline clients should be listed by ForEachAssignedClient. */
		FMultiViewOptions& ViewOptions;

		/** Broadcasts when property ownership may have changed. */
		FSimpleMulticastDelegate OnOwnershipChangedDelegate;
		
		void BroadcastOnOwnershipChanged() const { OnOwnershipChangedDelegate.Broadcast(); }
	};
}
