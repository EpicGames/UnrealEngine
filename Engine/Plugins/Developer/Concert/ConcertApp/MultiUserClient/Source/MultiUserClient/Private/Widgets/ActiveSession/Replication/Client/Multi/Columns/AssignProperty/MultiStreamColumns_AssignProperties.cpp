// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssignPropertyModel.h"

#include "MultiUserReplicationStyle.h"
#include "Replication/Editor/Model/IEditableMultiReplicationStreamModel.h"
#include "Replication/Client/UnifiedClientViewExtensions.h"
#include "Replication/Editor/View/IMultiReplicationStreamEditor.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"
#include "Replication/Editor/View/Column/IPropertyTreeColumn.h"
#include "Replication/Editor/View/Column/ReplicationColumnDelegates.h"
#include "SAssignPropertyComboBox.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "AssignPropertyColumn"

namespace UE::MultiUserClient::Replication::MultiStreamColumns::AssignPropertyColumnUtils
{
	template<typename TLambda> requires std::is_invocable_v<TLambda, const FGuid&>
	static void ForEachClientAssignedToProperty(
		const TArray<TSoftObjectPtr<>>& Objects,
		const FConcertPropertyChain& Property,
		const FUnifiedClientView& ClientView,
		const ConcertSharedSlate::IMultiReplicationStreamEditor& MultiEditor,
		TLambda&& Callback
	)
	{
		MultiEditor.GetMultiStreamModel().ForEachStream(
			[&Objects, &Property,  &ClientView, &Callback](const TSharedRef<ConcertSharedSlate::IReplicationStreamModel>& Stream)
			{
				for (const TSoftObjectPtr<>& Object : Objects)
				{
					if (!Stream->HasProperty(Object.ToSoftObjectPath() , Property))
					{
						continue;
					}
				
					const TOptional<FGuid> ClientEndpointId = FindClientIdByStream(ClientView, *Stream);
					if (ensure(ClientEndpointId))
					{
						Callback(*ClientEndpointId);
						return EBreakBehavior::Continue;
					}
				}
				return EBreakBehavior::Continue;
			}
		);
	}
}

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	const FName AssignPropertyColumnId(TEXT("AssignPropertyColumn"));
	
	ConcertSharedSlate::FPropertyColumnEntry AssignPropertyColumn(
		TAttribute<TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>> InMultiStreamEditor,
		FUnifiedClientView& InClientView,
		FMultiViewOptions& InViewOptions,
		const int32 InColumnsSortPriority
		)
	{
		using namespace ConcertSharedSlate;
		class FPropertyColumn_AssignPropertyColumn : public IPropertyTreeColumn
		{
		public:

			FPropertyColumn_AssignPropertyColumn(
				TAttribute<TSharedPtr<IMultiReplicationStreamEditor>> MultiStreamEditor,
				FUnifiedClientView& ClientView UE_LIFETIMEBOUND,
				FMultiViewOptions& InViewOptions UE_LIFETIMEBOUND
				)
				: MultiStreamEditor(MoveTemp(MultiStreamEditor))
				, Model(ClientView, InViewOptions)
				, ClientView(ClientView)
			{}
			
			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(AssignPropertyColumnId)
					.DefaultLabel(LOCTEXT("Owner.Label", "Author"))
					.ToolTipText(LOCTEXT("Owner.ToolTip", "Client that should replicate this property"))
					.FillSized(FMultiUserReplicationStyle::Get()->GetFloat(TEXT("AllClients.Property.OwnerSize")));
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				const TArray<TSoftObjectPtr<>> DisplayedObjects = InArgs.RowItem.RowData.GetContextObjects();
				return SNew(SAssignPropertyComboBox, Model, ClientView)
					.DisplayedProperty(InArgs.RowItem.RowData.GetProperty())
					.EditedObjects(DisplayedObjects)
					.HighlightText(InArgs.HighlightText)
					.OnPropertyAssignmentChanged_Lambda([this]()
					{
						if (const TSharedPtr<IMultiReplicationStreamEditor> Editor = MultiStreamEditor.Get())
						{
							Editor->GetEditorBase().RequestPropertyColumnResort(AssignPropertyColumnId);
						}
					});
			}
			
			virtual void PopulateSearchString(const FPropertyTreeRowContext& InItem, TArray<FString>& InOutSearchStrings) const override
			{
				AssignPropertyColumnUtils::ForEachClientAssignedToProperty(
					InItem.RowData.GetContextObjects(),
					InItem.RowData.GetProperty(),
					ClientView,
					*MultiStreamEditor.Get(),
					[this, &InOutSearchStrings](const FGuid& ClientId)
					{
						InOutSearchStrings.Add(GetClientDisplayString(ClientView, ClientId));
					});
			}

			virtual bool CanBeSorted() const override { return true; }
			virtual bool IsLessThan(const FPropertyTreeRowContext& Left, const FPropertyTreeRowContext& Right) const override
			{
				const TOptional<FString> LeftClientDisplayString = SAssignPropertyComboBox::GetDisplayString(
					Model, ClientView, Left.RowData.GetProperty(), Left.RowData.GetContextObjects()
					);
				const TOptional<FString> RightClientDisplayString = SAssignPropertyComboBox::GetDisplayString(
				Model, ClientView, Right.RowData.GetProperty(), Right.RowData.GetContextObjects()
				);
			
				if (LeftClientDisplayString && RightClientDisplayString)
				{
					return *LeftClientDisplayString < *RightClientDisplayString;
				}
				// Our rule: set < unset. This way unassigned appears last.
				return LeftClientDisplayString.IsSet() && !RightClientDisplayString.IsSet();
			}

		private:

			/** Used to refresh the sort state of the editor, if sorting by this column. */
			const TAttribute<TSharedPtr<IMultiReplicationStreamEditor>> MultiStreamEditor;

			/** The model the view displays. */
			FAssignPropertyModel Model;
			/** Used to get display information about clients. */
			FUnifiedClientView& ClientView;
		};
		
		check(InMultiStreamEditor.IsBound() || InMultiStreamEditor.IsSet());
		return {
			TReplicationColumnDelegates<FPropertyTreeRowContext>::FCreateColumn::CreateLambda(
				[MultiStreamEditor = MoveTemp(InMultiStreamEditor), &InClientView, &InViewOptions]()
				{
					return MakeShared<FPropertyColumn_AssignPropertyColumn>(MultiStreamEditor, InClientView, InViewOptions);
				}),
			AssignPropertyColumnId,
			{ InColumnsSortPriority }
		};
	}
}

#undef LOCTEXT_NAMESPACE