// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssignPropertyComboBox.h"

#include "AssignPropertyModel.h"
#include "Replication/Client/Online/OnlineClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Client/UnifiedClientViewExtensions.h"
#include "Replication/Editor/Model/PropertyUtils.h"
#include "Widgets/ActiveSession/Replication/Misc/SNoClients.h"
#include "Widgets/Client/ClientInfoHelpers.h"
#include "Widgets/Client/SHorizontalClientList.h"
#include "Widgets/Client/SClientName.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SComboButton.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "SAssignPropertyComboBox"

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	namespace AssignPropertyComboBox
	{
		static TArray<FGuid> GetDisplayedClients(
			const FAssignPropertyModel& Model,
			const FConcertPropertyChain& DisplayedProperty,
			const TArray<TSoftObjectPtr<>>& EditedObjects
			)
		{
			TArray<FGuid> Clients;
			Model.ForEachAssignedClient(DisplayedProperty, EditedObjects,
				[&Clients](const FGuid& ClientId){ Clients.AddUnique(ClientId); return EBreakBehavior::Continue; }
				);
			return Clients;
		}
	}
	
	TOptional<FString> SAssignPropertyComboBox::GetDisplayString(
		const FAssignPropertyModel& Model,
		const FUnifiedClientView& ClientView,
		const FConcertPropertyChain& DisplayedProperty,
		const TArray<TSoftObjectPtr<>>& EditedObjects)
	{
		using SWidgetType = ConcertSharedSlate::SHorizontalClientList;
		const TArray<FGuid> Clients = AssignPropertyComboBox::GetDisplayedClients(Model, DisplayedProperty, EditedObjects);
		
		const ConcertSharedSlate::FGetClientParenthesesContent GetParenthesesContent =
			MakeLocalAndOfflineParenthesesContentGetter(ClientView);
		const auto SortPredicate = [&GetParenthesesContent](const FConcertSessionClientInfo& Left, const FConcertSessionClientInfo& Right)
		{
			return ConcertSharedSlate::SortLocalClientParenthesesFirstThenThenAlphabetical(Left, Right, GetParenthesesContent);
		};
		
		return SWidgetType::GetDisplayString(
			Clients,
			MakeOnlineThenOfflineClientInfoGetter(ClientView),
			ConcertSharedSlate::FClientSortPredicate::CreateLambda(SortPredicate),
			GetParenthesesContent
			);
	}

	void SAssignPropertyComboBox::Construct(
		const FArguments& InArgs,
	    FAssignPropertyModel& InModel,
	    FUnifiedClientView& InClientView
	)
	{
		Model = &InModel;
		ClientView = &InClientView;
		
		Property = InArgs._DisplayedProperty;
		EditedObjects = InArgs._EditedObjects;
		HighlightText = InArgs._HighlightText;
		check(!EditedObjects.IsEmpty());

		OnOptionClickedDelegate = InArgs._OnPropertyAssignmentChanged;
		
		ChildSlot
		[
			SNew(SComboButton)
			.HasDownArrow(true)
			.ButtonContent()
			[
				SAssignNew(ClientListWidget, ConcertSharedSlate::SHorizontalClientList)
				.GetClientParenthesesContent(MakeLocalAndOfflineParenthesesContentGetter(InClientView))
				.GetClientInfo(MakeOnlineThenOfflineClientInfoGetter(InClientView))
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.HighlightText_Lambda([this](){ return HighlightText ? *HighlightText : FText::GetEmpty(); })
				.EmptyListSlot() [ SNew(SNoClients) ]
			]
			.OnGetMenuContent(this, &SAssignPropertyComboBox::GetMenuContent)
			.ToolTipText_Raw(this, &SAssignPropertyComboBox::GetComboBoxToolTipText)
		];

		Model->OnOwnershipChanged().AddSP(this, &SAssignPropertyComboBox::RefreshContentBoxContent);
		RefreshContentBoxContent();
	}
	
	void SAssignPropertyComboBox::RefreshContentBoxContent() const
	{
		ClientListWidget->RefreshList(
			AssignPropertyComboBox::GetDisplayedClients(*Model, Property, EditedObjects)
			);
	}

	TSharedRef<SWidget> SAssignPropertyComboBox::GetMenuContent()
	{
		using namespace ConcertSharedSlate;
		
		FMenuBuilder MenuBuilder(true, nullptr);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Clear.Label", "Clear"),
			LOCTEXT("Clear.Tooltip", "Stop this property from being replicated"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAssignPropertyComboBox::OnClickClear),
				FCanExecuteAction::CreateSP(this, &SAssignPropertyComboBox::CanClickClear)
				),
			NAME_None,
			EUserInterfaceActionType::Button
		);
		
		const auto MakeWidget = [this](const FGuid& EndpointId) -> TSharedRef<SWidget>
		{
			return SNew(SClientName)
				.ClientInfo_Lambda([this, EndpointId]{ return ClientView->GetClientInfoByEndpoint(EndpointId); })
				.ParenthesisContent_Lambda([this, EndpointId]{ return GetParenthesesContent(*ClientView, EndpointId); })
				.HighlightText_Lambda([this](){ return HighlightText ? *HighlightText : FText::GetEmpty(); })
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
		};
		
		MenuBuilder.BeginSection(TEXT("AssignTo"), LOCTEXT("AssignTo", "Assign to online client"));
		for (const FGuid& EndpointId : GetSortedOnlineClients(*ClientView))
		{
			TAttribute<FText> Tooltip = TAttribute<FText>::CreateLambda([this, EndpointId]()
			{
				FText Reason;
				const bool bCanClick = CanClickOptionWithReason(EndpointId, &Reason);
				if (!bCanClick)
				{
					return Reason;
				}

				switch (GetOptionCheckState(EndpointId))
				{
				case ECheckBoxState::Unchecked: return LOCTEXT("Action.Unchecked", "Assign property to client and remove it from all others.");
				case ECheckBoxState::Undetermined:  return LOCTEXT("Action.Undetermined", "Assign property to client for all selected objects.");
				case ECheckBoxState::Checked: return LOCTEXT("Action.Checked", "Remove property from client and remove it from all others.");
				default: return FText::GetEmpty();
				}
			});
			
			MenuBuilder.AddMenuEntry(
				FUIAction(
					FExecuteAction::CreateSP(this, &SAssignPropertyComboBox::OnClickOption, EndpointId),
					FCanExecuteAction::CreateSP(this, &SAssignPropertyComboBox::CanClickOption, EndpointId),
					FGetActionCheckState::CreateSP(this, &SAssignPropertyComboBox::GetOptionCheckState, EndpointId)
					),
				MakeWidget(EndpointId),
				NAME_None,
				Tooltip,
				EUserInterfaceActionType::Check
				);
		}
		MenuBuilder.EndSection();
		
		return MenuBuilder.MakeWidget();
	}
	
	void SAssignPropertyComboBox::OnClickOption(const FGuid EndpointId) const
	{
		Model->TogglePropertyFor(EndpointId, EditedObjects, Property);
		OnOptionClickedDelegate.ExecuteIfBound();
	}

	bool SAssignPropertyComboBox::CanClickOptionWithReason(const FGuid& EndpointId, FText* Reason) const
	{
		return Model->CanChangePropertyFor(EndpointId, Reason);
	}
	
	ECheckBoxState SAssignPropertyComboBox::GetOptionCheckState(const FGuid EndpointId) const
	{
		switch (Model->GetPropertyOwnershipState(EndpointId,EditedObjects, Property))
		{
		case EPropertyOnObjectsOwnershipState::OwnedOnAllObjects: return ECheckBoxState::Checked;
		case EPropertyOnObjectsOwnershipState::NotOwnedOnAllObjects: return ECheckBoxState::Unchecked;
		case EPropertyOnObjectsOwnershipState::Mixed: return ECheckBoxState::Undetermined;
		default: return ECheckBoxState::Undetermined;
		}
	}

	void SAssignPropertyComboBox::OnClickClear()
	{
		Model->ClearProperty(EditedObjects, Property);
		OnOptionClickedDelegate.ExecuteIfBound();
	}

	bool SAssignPropertyComboBox::CanClickClear() const
	{
		return Model->CanClear(EditedObjects, Property);
	}

	FText SAssignPropertyComboBox::GetComboBoxToolTipText() const
	{
		bool bHasOfflineClients = false;
		int32 NumClients = 0;
		Model->ForEachAssignedClient(Property, EditedObjects,
			[this, &bHasOfflineClients, &NumClients](const FGuid& ClientId)
			{
				++NumClients;
				const TOptional<EClientType> ClientType = ClientView->GetClientType(ClientId);
				bHasOfflineClients |= ClientType && IsOfflineClient(*ClientType);
				return NumClients > 1 ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});

		if (bHasOfflineClients)
		{
			return FText::Format(
				LOCTEXT("AssignProperty.ToolTip.HasOfflineClients", "The assigned offline {0}|plural(one=client,other=clients) will replicate this property upon rejoining."),
				NumClients
				);
		}

		return FText::Format(
			LOCTEXT("AssignProperty.ToolTip.Normal", "The {0}|plural(one=client,other=clients) that {0}|plural(one=is,other=are) registered to replicate this property."),
			NumClients
			);
	}
}

#undef LOCTEXT_NAMESPACE