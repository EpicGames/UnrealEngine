// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"

#include "Containers/Array.h"
#include "Templates/Function.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IConcertClient;
enum class ECheckBoxState : uint8;
struct FGuid;

namespace UE::ConcertSharedSlate { class SHorizontalClientList; }
namespace UE::MultiUserClient::Replication { class FUnifiedClientView; }

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	class FAssignPropertyModel;
	
	/**
	 * Placed in every property row column to assign properties to clients.
	 * 
	 * Can be used to assign a property to at most one client: multiple clients cannot be assigned because it is an advanced workflow
	 * which might confuse basic users.
	 */
	class SAssignPropertyComboBox : public SCompoundWidget
	{
	public:

		DECLARE_DELEGATE(FOnPropertyAssignmentChanged);

		/** @return The display string this widget would have with the given state. If unset, no clients are displayed in the combobox.*/
		static TOptional<FString> GetDisplayString(
			const FAssignPropertyModel& Model,
			const FUnifiedClientView& ClientView,
			const FConcertPropertyChain& DisplayedProperty,
			const TArray<TSoftObjectPtr<>>& EditedObjects
			);
		
		SLATE_BEGIN_ARGS(SAssignPropertyComboBox)
		{}
			SLATE_ARGUMENT(FConcertPropertyChain, DisplayedProperty)
			SLATE_ARGUMENT(TArray<TSoftObjectPtr<>>, EditedObjects)
			SLATE_ARGUMENT(TSharedPtr<FText>, HighlightText)
			/** Called when the property assignment of any client(s) is changed by this widget. */
			SLATE_EVENT(FOnPropertyAssignmentChanged, OnPropertyAssignmentChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs,
           FAssignPropertyModel& InModel,
           FUnifiedClientView& InClientView
		);

	private:

		/** The model this view is displaying. */
		FAssignPropertyModel* Model = nullptr;
		/** Used to obtain display information about clients. */
		FUnifiedClientView* ClientView = nullptr;

		/** The objects for which the property is being displayed. */
		TArray<TSoftObjectPtr<>> EditedObjects;
		/** The property assigned to this column. */
		FConcertPropertyChain Property;

		/** The static menu content (when there is no drop-down). */
		TSharedPtr<ConcertSharedSlate::SHorizontalClientList> ClientListWidget;
		/** Passed to MakeListWidgetDelegate */
		TSharedPtr<FText> HighlightText;

		/** Called when the property assignment of any client(s) is changed by this widget. */
		FOnPropertyAssignmentChanged OnOptionClickedDelegate;

		/** Updates the content of the combo box */
		void RefreshContentBoxContent() const;
		
		/** Builds the drop-down menu */
		TSharedRef<SWidget> GetMenuContent();

		/** Called when a client is clicked in the drop-down menu */
		void OnClickOption(const FGuid EndpointId) const;
		/** @return Whether an item in the drop-down menu should be clickable (returns true if the stream is writable) */
		bool CanClickOption(const FGuid EndpointId) const { return CanClickOptionWithReason(EndpointId, nullptr); }
		bool CanClickOptionWithReason(const FGuid& EndpointId, FText* Reason = nullptr) const;
		/** @return Checkbox state for the item. Handles cases of multiple objects being property edited. */
		ECheckBoxState GetOptionCheckState(const FGuid EndpointId) const;

		/** Called when a client clicks the clear button */
		void OnClickClear();
		/** Whether the property is assigned to any client */
		bool CanClickClear() const;
		
		FText GetComboBoxToolTipText() const;
	};
}

