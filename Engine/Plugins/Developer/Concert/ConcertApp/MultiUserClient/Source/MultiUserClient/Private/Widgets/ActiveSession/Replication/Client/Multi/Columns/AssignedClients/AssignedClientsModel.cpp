// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssignedClientsModel.h"

#include "Replication/Client/UnifiedStreamCache.h"
#include "Replication/Editor/Model/Object/IObjectHierarchyModel.h"

#include "Containers/Array.h"
#include "Misc/Guid.h"

#include <type_traits>

#include "Widgets/ActiveSession/Replication/Client/Multi/ViewOptions/MultiViewOptions.h"

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	FAssignedClientsModel::FAssignedClientsModel(
		const ConcertSharedSlate::IObjectHierarchyModel& InObjectHierarchy,
		FUnifiedStreamCache& InStreamCache,
		FMultiViewOptions& InViewOptions 
		)
		: ObjectHierarchy(InObjectHierarchy)
		, StreamCache(InStreamCache)
		, ViewOptions(InViewOptions)
	{
		StreamCache.OnCacheChanged().AddRaw(this, &FAssignedClientsModel::BroadcastOwnershipChanged);
		ViewOptions.OnOptionsChanged().AddRaw(this, &FAssignedClientsModel::BroadcastOwnershipChanged);
	}

	FAssignedClientsModel::~FAssignedClientsModel()
	{
		StreamCache.OnCacheChanged().RemoveAll(this);
		ViewOptions.OnOptionsChanged().RemoveAll(this);
	}

	TArray<FGuid> FAssignedClientsModel::GetAssignedClients(const FSoftObjectPath& ObjectPath) const
	{
		TArray<FGuid> ClientsWithOwnership;

		const auto ProcessObject = [this, &ClientsWithOwnership](const FSoftObjectPath& ObjectPath)
		{
			const EClientEnumerationMode Mode = ViewOptions.ShouldShowOfflineClients()
				? EClientEnumerationMode::SkipOfflineClientsThatFullyOverlapWithOnlineClients
				: EClientEnumerationMode::SkipOfflineClients;
			StreamCache.EnumerateClientsWithObject(ObjectPath, [&ClientsWithOwnership](const FGuid& ClientId)
			{
				ClientsWithOwnership.AddUnique(ClientId);
				return EBreakBehavior::Continue;
			}, Mode);
		};
			
		ProcessObject(ObjectPath);
		ObjectHierarchy.ForEachChildRecursive(
			TSoftObjectPtr{ ObjectPath },
			[&ProcessObject](const TSoftObjectPtr<>&, const TSoftObjectPtr<>& ChildObject, ConcertSharedSlate::EChildRelationship)
			{
				ProcessObject(ChildObject.GetUniqueID());
				return EBreakBehavior::Continue;
			});
		
		return ClientsWithOwnership;
	}

	void FAssignedClientsModel::OnClientChanged(const FGuid&) const
	{
		BroadcastOwnershipChanged();
	}

	void FAssignedClientsModel::BroadcastOwnershipChanged() const
	{
		OnOwnershipChangedDelegate.Broadcast();
	}
}
