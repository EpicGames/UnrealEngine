// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"


struct FGuid;

namespace UE::ConcertSharedSlate { class IObjectHierarchyModel; }
namespace UE::MultiUserClient::Replication
{
	class FMultiViewOptions;
	class FUnifiedStreamCache;
	class FUnifiedClientView;
	class FOnlineClientManager;
	class FReassignObjectPropertiesLogic;
}

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	/** Implements the model in the MVC pattern for the assigned clients column. */
	class FAssignedClientsModel : public FNoncopyable
	{
	public:
		
		FAssignedClientsModel(
			const ConcertSharedSlate::IObjectHierarchyModel& InObjectHierarchy UE_LIFETIMEBOUND,
			FUnifiedStreamCache& InStreamCache UE_LIFETIMEBOUND,
			FMultiViewOptions& InViewOptions UE_LIFETIMEBOUND
			);
		~FAssignedClientsModel();

		/** @return The endpoint IDs of clients that have properties assigned to ObjecPath. */
		TArray<FGuid> GetAssignedClients(const FSoftObjectPath& ObjectPath) const;
		
		/** Broadcast when the result of GetAssignedClients may have changed. */
		FSimpleMulticastDelegate& OnOwnershipChanged() { return OnOwnershipChangedDelegate; }

	private:

		/** Used to compute clients's recursive ownership of subobjects for a managed object. */
		const ConcertSharedSlate::IObjectHierarchyModel& ObjectHierarchy;
		/** Holds the stream content of both online and offline clients. */
		FUnifiedStreamCache& StreamCache;

		/** Controls whether offline clients should be considered. */
		FMultiViewOptions& ViewOptions;
		
		/** Broadcast when the result of GetAssignedClients may have changed. */
		FSimpleMulticastDelegate OnOwnershipChangedDelegate;
		
		void OnClientChanged(const FGuid&) const;
		void BroadcastOwnershipChanged() const;
	};
}

