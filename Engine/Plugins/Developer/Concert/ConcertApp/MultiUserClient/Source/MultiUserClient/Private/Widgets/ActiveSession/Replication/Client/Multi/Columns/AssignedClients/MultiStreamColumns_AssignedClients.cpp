// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssignedClientsModel.h"

#include "IConcertClient.h"
#include "MultiUserReplicationStyle.h"
#include "Replication/Client/ClientUtils.h"
#include "Replication/Editor/View/IMultiReplicationStreamEditor.h"
#include "SAssignedClientsWidget.h"
#include "Replication/Client/UnifiedClientView.h"
#include "Widgets/ActiveSession/Replication/Client/Multi/Columns/MultiStreamColumns.h"
#include "Widgets/Client/ClientInfoHelpers.h"
#include "Widgets/Client/SHorizontalClientList.h"

#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AssignedClientsColumnId"

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	const FName AssignedClientsColumnId(TEXT("AssignedClientsColumn"));

	ConcertSharedSlate::FObjectColumnEntry AssignedClientsColumn(
		const TSharedRef<IConcertClient>& InConcertClient,
		TAttribute<TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>> InMultiStreamModelAttribute,
		const ConcertSharedSlate::IObjectHierarchyModel& InObjectHierarchy,
		FUnifiedClientView& InClientView,
		FMultiViewOptions& InViewOptions,
		const int32 InColumnsSortPriority
		)
	{
		class FObjectColumn_AssignedClients : public ConcertSharedSlate::IObjectTreeColumn
		{
		public:

			FObjectColumn_AssignedClients(
				const TSharedRef<IConcertClient>& InConcertClient,
				TAttribute<TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>> InMultiStreamModelAttribute,
				const ConcertSharedSlate::IObjectHierarchyModel& InObjectHierarchy UE_LIFETIMEBOUND,
				FUnifiedClientView& InClientView UE_LIFETIMEBOUND,
				FMultiViewOptions& InViewOptions UE_LIFETIMEBOUND
				)
				: ConcertClient(InConcertClient)
				, MultiStreamModelAttribute(MoveTemp(InMultiStreamModelAttribute))
				, Model(InObjectHierarchy, InClientView.GetStreamCache(), InViewOptions)
				, ClientView(InClientView)
			{}
			
			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(AssignedClientsColumnId)
					.DefaultLabel(LOCTEXT("Owner.Label", "Author"))
					.ToolTipText(LOCTEXT("Owner.ToolTip", "Clients that have registered properties for the object"))
					.FillSized(FMultiUserReplicationStyle::Get()->GetFloat(TEXT("AllClients.Object.OwnerSize")));
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				return SNew(SAssignedClientsWidget, Model, ClientView)
					.ManagedObject(InArgs.RowItem.RowData.GetObjectPath())
					.HighlightText_Lambda([HighlightText = InArgs.HighlightText](){ return HighlightText ? *HighlightText : FText::GetEmpty(); });
			}
			virtual void PopulateSearchString(const ConcertSharedSlate::FObjectTreeRowContext& InItem, TArray<FString>& InOutSearchStrings) const override
			{
				for (const FGuid& ClientId : Model.GetAssignedClients(InItem.RowData.GetObjectPath()))
				{
					InOutSearchStrings.Add(ClientUtils::GetClientDisplayName(*ConcertClient->GetCurrentSession(), ClientId));
				}
			}

			virtual bool CanBeSorted() const override { return true; }
			virtual bool IsLessThan(const ConcertSharedSlate::FObjectTreeRowContext& Left, const ConcertSharedSlate::FObjectTreeRowContext& Right) const override
			{
				const TOptional<FString> LeftClientDisplayString = GetDisplayString(Left.RowData.GetObjectPath());
				const TOptional<FString> RightClientDisplayString = GetDisplayString(Right.RowData.GetObjectPath());
			
				if (LeftClientDisplayString && RightClientDisplayString)
				{
					return *LeftClientDisplayString < *RightClientDisplayString;
				}
				// Our rule: set < unset. This way unassigned appears last.
				return LeftClientDisplayString.IsSet() && !RightClientDisplayString.IsSet();
			}

		private:
			
			const TSharedRef<IConcertClient> ConcertClient;
			const TAttribute<TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>> MultiStreamModelAttribute;

			/** The model that the view displays; the model of the MVC pattern. */
			FAssignedClientsModel Model;
			/** Passed to the UI. */
			FUnifiedClientView& ClientView;
			
			TOptional<FString> GetDisplayString(const FSoftObjectPath& ManagedObject) const
			{
				using SWidgetType = ConcertSharedSlate::SHorizontalClientList;
				const TArray<FGuid> Clients = Model.GetAssignedClients(ManagedObject);

				const ConcertSharedSlate::FGetClientParenthesesContent GetParenthesesContent =
					ConcertClientSharedSlate::MakeGetLocalClientParenthesesContent(ConcertClient);
				const auto SortPredicate = [&GetParenthesesContent](const FConcertSessionClientInfo& Left, const FConcertSessionClientInfo& Right)
				{
					return ConcertSharedSlate::SortLocalClientParenthesesFirstThenThenAlphabetical(Left, Right, GetParenthesesContent);
				};
				return SWidgetType::GetDisplayString(
					Clients,
					ConcertClientSharedSlate::MakeClientInfoGetter(ConcertClient),
					ConcertSharedSlate::FClientSortPredicate::CreateLambda(SortPredicate),
					GetParenthesesContent
					);
			}
		};

		return {
			ConcertSharedSlate::TReplicationColumnDelegates<ConcertSharedSlate::FObjectTreeRowContext>::FCreateColumn::CreateLambda(
				[InConcertClient, MultiStreamModelAttribute = MoveTemp(InMultiStreamModelAttribute), &InObjectHierarchy, &InClientView, &InViewOptions]
				{
					return MakeShared<FObjectColumn_AssignedClients>(
						InConcertClient, MultiStreamModelAttribute, InObjectHierarchy, InClientView, InViewOptions
						);
				}),
			AssignedClientsColumnId,
			{ InColumnsSortPriority }
		};
	}
}

#undef LOCTEXT_NAMESPACE