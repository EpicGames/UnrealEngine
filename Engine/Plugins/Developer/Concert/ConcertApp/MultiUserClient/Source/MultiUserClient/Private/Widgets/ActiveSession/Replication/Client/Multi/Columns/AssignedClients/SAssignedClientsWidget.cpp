// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssignedClientsWidget.h"

#include "AssignedClientsModel.h"
#include "Replication/Client/UnifiedClientViewExtensions.h"
#include "Widgets/Client/ClientInfoHelpers.h"
#include "Widgets/Client/SHorizontalClientList.h"

#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAssignedClientsWidget"

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	void SAssignedClientsWidget::Construct(
		const FArguments& InArgs,
		FAssignedClientsModel& InModel,
		const FUnifiedClientView& InClientView
		)
	{
		Model = &InModel;
		ManagedObject = InArgs._ManagedObject;
		
		ChildSlot
		[
			SAssignNew(ClientList, ConcertSharedSlate::SHorizontalClientList)
			.GetClientParenthesesContent(MakeLocalAndOfflineParenthesesContentGetter(InClientView))
			.GetClientInfo(MakeOnlineThenOfflineClientInfoGetter(InClientView))
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.HighlightText(InArgs._HighlightText)
			.ListToolTipText(LOCTEXT("Clients.ToolTip", "These clients will replicate their assigned properties when replication is active.\nYou can pause & resume replication at the beginnig of this row."))
			.EmptyListSlot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoClients.Label", "No assigned properties"))
				.ToolTipText(LOCTEXT("NoClients.ToolTip", "Click this row and then assign the properties to the client that should replicate them."))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
		];
		
		RefreshClientList();
		Model->OnOwnershipChanged().AddSP(this, &SAssignedClientsWidget::RefreshClientList);
	}
	
	SAssignedClientsWidget::~SAssignedClientsWidget()
	{
		Model->OnOwnershipChanged().RemoveAll(this);
	}

	void SAssignedClientsWidget::RefreshClientList() const
	{
		ClientList->RefreshList(Model->GetAssignedClients(ManagedObject));
	}
}

#undef LOCTEXT_NAMESPACE