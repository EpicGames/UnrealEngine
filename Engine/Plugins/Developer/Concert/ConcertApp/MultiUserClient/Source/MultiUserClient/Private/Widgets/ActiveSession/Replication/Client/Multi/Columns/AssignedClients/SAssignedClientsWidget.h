// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IConcertClient;

namespace UE::ConcertSharedSlate
{
	class IObjectHierarchyModel;
	class SHorizontalClientList;
}

namespace UE::MultiUserClient::Replication
{
	class FReassignObjectPropertiesLogic;
	class FUnifiedClientView;
}

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	class FAssignedClientsModel;

	/** Placed into object rows. Display the clients that own it and its children. */
	class SAssignedClientsWidget : public SCompoundWidget
	{
		
	public:
	
		SLATE_BEGIN_ARGS(SAssignedClientsWidget){}
			SLATE_ARGUMENT(FSoftObjectPath, ManagedObject)
			SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_END_ARGS()

		void Construct(
			const FArguments& InArgs,
			FAssignedClientsModel& InModel,
			const FUnifiedClientView& InClientView
			);
	
		virtual ~SAssignedClientsWidget() override;

		void RefreshClientList() const;

	private:

		/** The model this view displays. */
		FAssignedClientsModel* Model = nullptr;
		
		/** This widget displays the owning clients and is refreshed when the list of owning clients changes. */
		TSharedPtr<ConcertSharedSlate::SHorizontalClientList> ClientList;
		
		/** The object for which we're displaying the owning clients */
		FSoftObjectPath ManagedObject;
	};
}
