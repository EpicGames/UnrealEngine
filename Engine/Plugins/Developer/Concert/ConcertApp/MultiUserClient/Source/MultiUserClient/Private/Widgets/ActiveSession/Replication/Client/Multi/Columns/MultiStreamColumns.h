// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/Column/IPropertyTreeColumn.h"
#include "Replication/Editor/View/Column/IObjectTreeColumn.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

class IConcertClient;

namespace UE::MultiUserClient::Replication
{
	class FMultiViewOptions;
	class FMuteChangeTracker;
	class FReassignObjectPropertiesLogic;
	class FOnlineClientManager;
	class FUnifiedClientView;
}

namespace UE::ConcertSharedSlate
{
	class IObjectHierarchyModel;
	class IReplicationStreamModel;
	class IMultiReplicationStreamEditor;
	class IReplicationStreamViewer;
}

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	const extern FName MuteToggleColumnId;
	const extern FName AssignedClientsColumnId;
	const extern FName AssignPropertyColumnId;
	
	/* @see ETopLevelColumnOrder and EReplicationPropertyColumnOrder */
	enum class EColumnSortOrder
	{
		MuteToggle = 0,
		AssignPropertyColumn = 30,
		ReassignOwnership = 40
	};

	/**
	 * Mutes and unmutes the object and its child objects.
	 * 
	 * @param InMuteChangeTracker Tells us the mute state and changes it.
	 * @param InColumnsSortPriority The order relative to the other columns
	 * @return A checkbox with pause and unpause icons.
	 */
	ConcertSharedSlate::FObjectColumnEntry MuteToggleColumn(
		FMuteChangeTracker& InMuteChangeTracker UE_LIFETIMEBOUND,
		int32 InColumnsSortPriority = static_cast<int32>(EColumnSortOrder::MuteToggle)
		);

	/**
	 * A combo box which displays all current owners for an object and allows bulk reassigning properties.
	 *
	 * @param InConcertClient Used to look up client names
	 * @param InMultiStreamModelAttribute Used for requesting resorting the column
	 * @param InObjectHierarchy Used to display clients when a child object is replicated
	 * @param InClientView Used to access all clients' stream content for display in the column
	 * @param InViewOptions Controls whether offline clients are displayed
	 * @param InColumnsSortPriority The order relative to the other columns
	 */
	ConcertSharedSlate::FObjectColumnEntry AssignedClientsColumn(
		const TSharedRef<IConcertClient>& InConcertClient,
		TAttribute<TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>> InMultiStreamModelAttribute,
		const ConcertSharedSlate::IObjectHierarchyModel& InObjectHierarchy UE_LIFETIMEBOUND,
		FUnifiedClientView& InClientView UE_LIFETIMEBOUND,
		FMultiViewOptions& InViewOptions UE_LIFETIMEBOUND,
		int32 InColumnsSortPriority = static_cast<int32>(EColumnSortOrder::ReassignOwnership)
		);
	
	/**
	 * Creates a property column which assigns the property to the stream selected in the combo box.
	 * 
	 * @param InMultiStreamEditor Used to determine the selected objects.
	 * @param InClientView Used to obtain the clients that can be assigned to and display them
	 * @param InViewOptions Controls whether offline clients are displayed
	 * @param InColumnsSortPriority The order relative to the other columns
	 * 
	 * @return A column that spawns a combo box for assigning properties
	 */
	ConcertSharedSlate::FPropertyColumnEntry AssignPropertyColumn(
		TAttribute<TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>> InMultiStreamEditor,
		FUnifiedClientView& InClientView UE_LIFETIMEBOUND,
		FMultiViewOptions& InViewOptions UE_LIFETIMEBOUND,
		const int32 InColumnsSortPriority = static_cast<int32>(EColumnSortOrder::AssignPropertyColumn)
		);
}
