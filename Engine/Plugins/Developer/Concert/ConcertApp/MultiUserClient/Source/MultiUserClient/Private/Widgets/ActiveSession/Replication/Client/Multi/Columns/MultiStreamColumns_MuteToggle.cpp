// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiStreamColumns.h"

#include "MultiUserReplicationStyle.h"
#include "Replication/Editor/View/Column/IObjectTreeColumn.h"
#include "SMuteToggle.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "ReplicationToggle"

namespace UE::MultiUserClient::Replication::MultiStreamColumns
{
	const FName MuteToggleColumnId(TEXT("MuteToggleColumnId"));
	
	ConcertSharedSlate::FObjectColumnEntry MuteToggleColumn(
		FMuteChangeTracker& MuteChangeTracker UE_LIFETIMEBOUND,
		int32 ColumnsSortPriority
		)
	{
		class FObjectColumn_MuteToggle : public ConcertSharedSlate::IObjectTreeColumn
		{
		public:

			FObjectColumn_MuteToggle(FMuteChangeTracker& MuteChangeTracker UE_LIFETIMEBOUND)
				: MuteChangeTracker(MuteChangeTracker)
			{}
			
			virtual SHeaderRow::FColumn::FArguments CreateHeaderRowArgs() const override
			{
				return SHeaderRow::Column(MuteToggleColumnId)
					.DefaultLabel(FText::GetEmpty())
					.FixedWidth(FMultiUserReplicationStyle::Get()->GetFloat(TEXT("AllClients.Object.MuteToggle")))
					[
						SNew(SBox)
						.WidthOverride(16.f)
						.HeightOverride(16.f)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ToolTipText(LOCTEXT("Mute.ToolTip", "Whether an object is supposed to replicate or not."))
						[
							SNew(SImage)
							.Image(FMultiUserReplicationStyle::Get()->GetBrush(TEXT("MultiUser.Icons.Play")))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					];
			}
			
			virtual TSharedRef<SWidget> GenerateColumnWidget(const FBuildArgs& InArgs) override
			{
				return SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(2.f)
					[
						SNew(SMuteToggle, InArgs.RowItem.RowData.GetObjectPath(), MuteChangeTracker)
					];
			}

		private:

			FMuteChangeTracker& MuteChangeTracker;
		};

		return {
			ConcertSharedSlate::TReplicationColumnDelegates<ConcertSharedSlate::FObjectTreeRowContext>::FCreateColumn::CreateLambda(
				[&MuteChangeTracker]()
				{
					return MakeShared<FObjectColumn_MuteToggle>(MuteChangeTracker);
				}),
			MuteToggleColumnId,
			{ ColumnsSortPriority }
		};
	}
}

#undef LOCTEXT_NAMESPACE