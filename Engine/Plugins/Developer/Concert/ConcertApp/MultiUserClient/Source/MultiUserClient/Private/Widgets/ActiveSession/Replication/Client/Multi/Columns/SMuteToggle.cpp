// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMuteToggle.h"

#include "MultiUserReplicationStyle.h"
#include "Replication/Muting/MuteChangeTracker.h"

#include "Styling/SlateTypes.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SMuteToggle"

namespace UE::MultiUserClient::Replication
{
	void SMuteToggle::Construct(const FArguments& InArgs, FSoftObjectPath InObjectPath, FMuteChangeTracker& InMuteChangeTracker)
	{
		ObjectPath = InObjectPath;
		MuteChangeTracker = &InMuteChangeTracker;
		ChildSlot
		[
			SNew(SCheckBox)
			.IsChecked(this, &SMuteToggle::IsMuted)
			.Visibility(this, &SMuteToggle::GetMuteVisibility)
			.ToolTipText(this, &SMuteToggle::GetToolTipText)
			.OnCheckStateChanged(this, &SMuteToggle::OnCheckboxStateChanged)
			.Style(FMultiUserReplicationStyle::Get(), TEXT("AllClients.MuteToggle.Style"))
		];
	}

	ECheckBoxState SMuteToggle::IsMuted() const
	{
		return MuteChangeTracker->IsMuted(ObjectPath)
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	EVisibility SMuteToggle::GetMuteVisibility() const
	{
		return MuteChangeTracker->CanChangeMuteState(ObjectPath)
			? EVisibility::Visible
			: EVisibility::Hidden;
	}

	FText SMuteToggle::GetToolTipText() const
	{
		if (!MuteChangeTracker->CanChangeMuteState(ObjectPath))
		{
			return LOCTEXT("Mute.ToolTip.CannotMute", "Assign some properties to this object or a child object first.");
		}
		
		return MuteChangeTracker->IsMuted(ObjectPath)
			? LOCTEXT("Mute.ToolTip.Muted", "Paused. Press to resume replication.")
			: LOCTEXT("Mute.ToolTip.Unmuted", "Replicating. Press to pause replication.");
	}

	void SMuteToggle::OnCheckboxStateChanged(ECheckBoxState)
	{
		MuteChangeTracker->ToggleMuteState(ObjectPath);
	}
}

#undef LOCTEXT_NAMESPACE