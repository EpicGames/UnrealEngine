// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/SoftObjectPath.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FText;
enum class ECheckBoxState : uint8;
struct EVisibility;

namespace UE::MultiUserClient::Replication
{
	class FMuteChangeTracker;
	
	/** Pauses and resumes replication for an object */
	class SMuteToggle : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SMuteToggle) {}
		SLATE_END_ARGS()

		/**
		 * @param InArgs Widget arguments
		 * @param InObjectPath The path of the object for which to toggle the mute state
		 * @param InMuteChangeTracker The caller ensures it outlives the lifetime of the widget.
		 */
		void Construct(const FArguments& InArgs, FSoftObjectPath InObjectPath, FMuteChangeTracker& InMuteChangeTracker);

	private:

		/** The object that is being muted by this widget. */
		FSoftObjectPath ObjectPath; 
		
		/** Knows the mute state that should be displayed and is used to change it. */
		FMuteChangeTracker* MuteChangeTracker = nullptr;
		
		ECheckBoxState IsMuted() const;
		EVisibility GetMuteVisibility() const;
		FText GetToolTipText() const;
		void OnCheckboxStateChanged(ECheckBoxState CheckBoxState);
	};
}

