// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiStreamModel.h"

#include "Replication/Client/Offline/OfflineClientManager.h"
#include "Replication/Client/Online/OnlineClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Selection/ISelectionModel.h"
#include "ViewOptions/MultiViewOptions.h"

namespace UE::MultiUserClient::Replication
{
	FMultiStreamModel::FMultiStreamModel(
		IOnlineClientSelectionModel& InOnlineClientSelectionModel,
		IOfflineClientSelectionModel& InOfflineClientSelectionModel,
		FOnlineClientManager& InOnlineClientManager,
		FOfflineClientManager& InOfflineClientManager,
		FMultiViewOptions& InViewOptions
		)
		: OnlineClientSelectionModel(InOnlineClientSelectionModel)
		, OfflineClientSelectionModel(InOfflineClientSelectionModel)
		, OnlineClientManager(InOnlineClientManager)
		, OfflineClientManager(InOfflineClientManager)
		, ViewOptions(InViewOptions)
	{
		OnlineClientSelectionModel.OnSelectionChanged().AddRaw(this, &FMultiStreamModel::RebuildOnlineClients);
		OfflineClientSelectionModel.OnSelectionChanged().AddRaw(this, &FMultiStreamModel::RebuildOfflineClients);
		ViewOptions.OnOptionsChanged().AddRaw(this, &FMultiStreamModel::RebuildClients);
		
		RebuildClients();
	}

	FMultiStreamModel::~FMultiStreamModel()
	{
		UnsubscribeFromOnlineClients();
		UnsubscribeFromOfflineClients();
		ViewOptions.OnOptionsChanged().RemoveAll(this);
	}

	void FMultiStreamModel::ForEachDisplayedOnlineClient(TFunctionRef<EBreakBehavior(const FOnlineClient*)> ProcessClient) const
	{
		for (const FOnlineClient* WritableClient : CachedOnlineClients)
		{
			if (ProcessClient(WritableClient) == EBreakBehavior::Break)
			{
				return;
			}
		}
	}

	TSet<TSharedRef<ConcertSharedSlate::IReplicationStreamModel>> FMultiStreamModel::GetReadOnlyStreams() const
	{
		TSet<TSharedRef<ConcertSharedSlate::IReplicationStreamModel>> Result;
		Algo::Transform(CachedOfflineClients, Result, [](const FOfflineClient* Client){ return Client->GetStreamModel(); });
		return Result;
	}

	TSet<TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel>> FMultiStreamModel::GetEditableStreams() const
	{
		TSet<TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel>> Result;
		Algo::Transform(CachedOnlineClients, Result, [](const FOnlineClient* Client){ return Client->GetClientEditModel(); });
		return Result;
	}

	void FMultiStreamModel::RebuildOnlineClients()
	{
		// It is not safe to iterate through our cached client array because it may contain stale clients that were just removed.
		UnsubscribeFromOnlineClients();
		
		TSet<const FOnlineClient*> OnlineClients;
		OnlineClientSelectionModel.ForEachItem([this, &OnlineClients](FOnlineClient& Client)
		{
			const TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> Stream = Client.GetClientEditModel();
			Client.OnModelChanged().AddRaw(
				this, &FMultiStreamModel::HandleOnlineClientStreamExternallyChanged, Stream.ToWeakPtr()
				);
			
			OnlineClients.Add(&Client);
			return EBreakBehavior::Continue;
		});

		const bool bClientsStayedSame = CachedOnlineClients.Num() == OnlineClients.Num() && CachedOnlineClients.Includes(OnlineClients);
		if (!bClientsStayedSame)
		{
			CachedOnlineClients = MoveTemp(OnlineClients);
			OnStreamSetChangedDelegate.Broadcast();
		}
	}

	void FMultiStreamModel::RebuildOfflineClients()
	{
		// It is not safe to iterate through our cached client array because it may contain stale clients that were just removed.
		UnsubscribeFromOfflineClients();
		if (!ViewOptions.ShouldShowOfflineClients())
		{
			CachedOfflineClients.Reset();
			OnStreamSetChangedDelegate.Broadcast();
			return;
		}
		
		TSet<const FOfflineClient*> OfflineClients;
		OfflineClientSelectionModel.ForEachItem([this, &OfflineClients](FOfflineClient& Client)
		{
			Client.OnStreamPredictionChanged().AddRaw(
				this, &FMultiStreamModel::HandleOfflineClientStreamExternallyChanged, Client.GetStreamModel().ToWeakPtr()
				);
			OfflineClients.Add(&Client);
			return EBreakBehavior::Continue;
		});

		const bool bClientsStayedSame = CachedOfflineClients.Num() == OfflineClients.Num() && CachedOfflineClients.Includes(OfflineClients);
		if (!bClientsStayedSame)
		{
			CachedOfflineClients = MoveTemp(OfflineClients);
			OnStreamSetChangedDelegate.Broadcast();
		}
	}

	void FMultiStreamModel::UnsubscribeFromOnlineClients() const
	{
		OnlineClientManager.ForEachClient([this](FOnlineClient& Client)
		{
			Client.OnModelChanged().RemoveAll(this);
			return EBreakBehavior::Continue;
		});
	}

	void FMultiStreamModel::UnsubscribeFromOfflineClients() const
	{
		OfflineClientManager.ForEachClient([this](FOfflineClient& Client)
		{
			Client.OnStreamPredictionChanged().RemoveAll(this);
			return EBreakBehavior::Continue;
		});
	}

	void FMultiStreamModel::HandleOnlineClientStreamExternallyChanged(TWeakPtr<ConcertSharedSlate::IEditableReplicationStreamModel> ChangedStream)
	{
		if (const TSharedPtr<ConcertSharedSlate::IEditableReplicationStreamModel> ChangedStreamPin = ChangedStream.Pin())
		{
			OnStreamsExternallyChanged.Broadcast(ChangedStreamPin.ToSharedRef());
		}
	}

	void FMultiStreamModel::HandleOfflineClientStreamExternallyChanged(TWeakPtr<ConcertSharedSlate::IReplicationStreamModel> ChangedStream)
	{
		if (const TSharedPtr<ConcertSharedSlate::IReplicationStreamModel> ChangedStreamPin = ChangedStream.Pin())
		{
			OnStreamsExternallyChanged.Broadcast(ChangedStreamPin.ToSharedRef());
		}
	}
}
