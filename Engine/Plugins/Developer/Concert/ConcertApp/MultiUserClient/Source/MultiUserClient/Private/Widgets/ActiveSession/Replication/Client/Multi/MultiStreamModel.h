// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Replication/Editor/Model/IEditableMultiReplicationStreamModel.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Selection/SelectionModelFwd.h"

#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

enum class EBreakBehavior : uint8;

namespace UE::MultiUserClient::Replication
{
	class FMultiViewOptions;
	class FOfflineClientManager;
	class FOnlineClient;
	class FOnlineClientManager;

	/**
	 * Decides the client streams that are displayed in the multi-view.
	 * 
	 * This model, in turn, uses IOnlineClientSelectionModel and IOfflineClientSelectionModel to decide which online and offline clients are to be
	 * displayed.
	 */
	class FMultiStreamModel : public ConcertSharedSlate::IEditableMultiReplicationStreamModel
	{
	public:
		
		FMultiStreamModel(
			IOnlineClientSelectionModel& InOnlineClientSelectionModel UE_LIFETIMEBOUND,
			IOfflineClientSelectionModel& InOfflineClientSelectionModel UE_LIFETIMEBOUND,
			FOnlineClientManager& InOnlineClientManager UE_LIFETIMEBOUND,
			FOfflineClientManager& InOfflineClientManager UE_LIFETIMEBOUND,
			FMultiViewOptions& InViewOptions UE_LIFETIMEBOUND
			);
		~FMultiStreamModel();

		/** Enumerates every displayed online client. */
		void ForEachDisplayedOnlineClient(TFunctionRef<EBreakBehavior(const FOnlineClient*)> ProcessClient) const;

		//~ Begin IEditableMultiReplicationStreamModel Interface
		virtual TSet<TSharedRef<ConcertSharedSlate::IReplicationStreamModel>> GetReadOnlyStreams() const override;
		virtual TSet<TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel>> GetEditableStreams() const override;
		virtual FOnStreamExternallyChanged& OnStreamExternallyChanged() override { return OnStreamsExternallyChanged; }
		virtual FOnStreamSetChanged& OnStreamSetChanged() override { return OnStreamSetChangedDelegate; }
		//~ End IEditableMultiReplicationStreamModel Interface

	private:
		
		/** Gets all online clients that are supposed to be displayed. */
		IOnlineClientSelectionModel& OnlineClientSelectionModel;
		/** Gets all offline clients that are supposed to be displayed. */
		IOfflineClientSelectionModel& OfflineClientSelectionModel;
		
		/** Used to clean up subscriptions when client list changes. */
		FOnlineClientManager& OnlineClientManager;
		/** Used to clean up subscriptions when client list changes. */
		FOfflineClientManager& OfflineClientManager;

		/** Determines whether offline clients should be shown. */
		FMultiViewOptions& ViewOptions;

		TSet<const FOnlineClient*> CachedOnlineClients;
		TSet<const FOfflineClient*> CachedOfflineClients;

		FOnStreamExternallyChanged OnStreamsExternallyChanged;
		FOnStreamSetChanged OnStreamSetChangedDelegate;

		void RebuildOnlineClients();
		void RebuildOfflineClients();
		void RebuildClients()
		{
			RebuildOnlineClients();
			RebuildOfflineClients();
		}
		
		void UnsubscribeFromOnlineClients() const;
		void UnsubscribeFromOfflineClients() const;

		/** Handle read-only streams changing */
		void HandleOnlineClientStreamExternallyChanged(TWeakPtr<ConcertSharedSlate::IEditableReplicationStreamModel> ChangedStream);
		void HandleOfflineClientStreamExternallyChanged(TWeakPtr<ConcertSharedSlate::IReplicationStreamModel> ChangedStream);
	};
}

