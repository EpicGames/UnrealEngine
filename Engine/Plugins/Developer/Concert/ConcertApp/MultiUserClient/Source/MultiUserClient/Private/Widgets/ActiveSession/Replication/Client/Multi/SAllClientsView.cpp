// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAllClientsView.h"

#include "Selection/AllOnlineClientsSelectionModel.h"
#include "Replication/MultiUserReplicationManager.h"
#include "SMultiClientView.h"

namespace UE::MultiUserClient::Replication
{
	void SAllClientsView::Construct(const FArguments&, TSharedRef<IConcertClient> InConcertClient, FMultiUserReplicationManager& InMultiUserReplicationManager)
	{
		AllOnlineClientsModel = MakeUnique<FAllOnlineClientsSelectionModel>(*InMultiUserReplicationManager.GetOnlineClientManager());
		AllOfflineClientsModel = MakeUnique<FAllOfflineClientsSelectionModel>(*InMultiUserReplicationManager.GetOfflineClientManager());
		
		ChildSlot
		[
			SAssignNew(ClientView, SMultiClientView, InConcertClient, InMultiUserReplicationManager, *AllOnlineClientsModel, *AllOfflineClientsModel)
		];
	}
}
