// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Selection/AllOfflineClientsSelectionModel.h"
#include "Selection/AllOnlineClientsSelectionModel.h"

#include "HAL/Platform.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IConcertClient;

namespace UE::MultiUserClient::Replication
{
	class FMultiUserReplicationManager;
	class FOnlineClient;
	class FOnlineClientManager;
	class SMultiClientView;

	/** Leverages SMultiClientView to display all online and offline clients. */
	class SAllClientsView : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SAllClientsView){}
		SLATE_END_ARGS()

		/**
		 * @param InConcertClient The local client instance
		 * @param InMultiUserReplicationManager The caller ensures it outlives the lifetime of the widget.
		 */
		void Construct(const FArguments&, TSharedRef<IConcertClient> InConcertClient, FMultiUserReplicationManager& InMultiUserReplicationManager);

		const TSharedPtr<SMultiClientView>& GetClientView() const { return ClientView; }
		
	private:

		/** Keeps track of all online clients. */
		TUniquePtr<FAllOnlineClientsSelectionModel> AllOnlineClientsModel;
		/** Keeps track of all offline clients. */
		TUniquePtr<FAllOfflineClientsSelectionModel> AllOfflineClientsModel;

		/** Shows the content of all replication clients. */
		TSharedPtr<SMultiClientView> ClientView;
	};
}
