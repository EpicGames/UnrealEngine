// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMultiClientView.h"

#include "Selection/ISelectionModel.h"
#include "MultiStreamModel.h"
#include "SObjectOverlayRow.h"
#include "Misc/ObjectUtils.h"
#include "Replication/ClientReplicationWidgetFactories.h"
#include "Replication/MultiUserReplicationManager.h"
#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Client/Online/OnlineClient.h"
#include "Replication/Client/Online/OnlineClientManager.h"
#include "Replication/Editor/Model/Object/IObjectNameModel.h"
#include "Replication/Editor/Model/PropertySource/SelectPropertyFromUClassModel.h"
#include "Replication/Editor/View/IMultiObjectPropertyAssignmentView.h"
#include "Replication/Editor/View/IMultiReplicationStreamEditor.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"
#include "Replication/Editor/Model/ObjectSource/ActorSelectionSourceModel.h"
#include "Replication/Stream/Discovery/MultiUserStreamExtender.h"
#include "ViewOptions/SMultiViewOptions.h"
#include "Widgets/ActiveSession/Replication/Client/Context/ContextMenuUtils.h"
#include "Widgets/ActiveSession/Replication/Client/Multi/Columns/MultiStreamColumns.h"
#include "Widgets/ActiveSession/Replication/Client/PropertySelection/SPropertySelectionComboButton.h"
#include "Widgets/ActiveSession/Replication/Client/SPresetComboButton.h"
#include "Widgets/ActiveSession/Replication/Client/SReplicationStatus.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMultiClientView"

namespace UE::MultiUserClient::Replication
{
	void SMultiClientView::Construct(
		const FArguments&,
		const TSharedRef<IConcertClient>& InConcertClient,
		FMultiUserReplicationManager& InMultiUserReplicationManager,
		IOnlineClientSelectionModel& InOnlineClientSelectionModel,
		IOfflineClientSelectionModel& InOfflineClientSelectionModel
		)
	{
		ConcertClient = InConcertClient;
		UserSelectedProperties = InMultiUserReplicationManager.GetUserPropertySelector();
		OnlineClientManager = InMultiUserReplicationManager.GetOnlineClientManager();
		OfflineClientManager = InMultiUserReplicationManager.GetOfflineClientManager();
		OnlineClientSelectionModel = &InOnlineClientSelectionModel;
		OfflineClientSelectionModel = &InOfflineClientSelectionModel;
		StreamModel = MakeShared<FMultiStreamModel>(
			InOnlineClientSelectionModel, InOfflineClientSelectionModel, *OnlineClientManager, *OfflineClientManager, ViewOptions
			);
		
		OnlineClientManager->OnRemoteClientsChanged().AddSP(this, &SMultiClientView::RebuildClientSubscriptions);
		OnlineClientSelectionModel->OnSelectionChanged().AddSP(this, &SMultiClientView::RebuildClientSubscriptions);
		UserSelectedProperties->OnPropertySelectionChanged().AddRaw(this, &SMultiClientView::RefreshUI);

		TSharedPtr<SVerticalBox> Content;
		ChildSlot
		[
			SAssignNew(Content, SVerticalBox)

			// Editor
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				CreateEditorContent(ConcertClient.ToSharedRef(), InMultiUserReplicationManager)
			]
		];
		
		SReplicationStatus::AppendReplicationStatus(*Content, OnlineClientManager->GetAuthorityCache(),
			SReplicationStatus::FArguments()
			.ReplicatableClients(this, &SMultiClientView::GetReplicatableClientIds)
			.ForEachObjectInStream(this, &SMultiClientView::EnumerateReplicatedObjectsInStreams)
			);

		// Changing worlds affects what things are displayed in the editor.
		HideObjectsNotInEditorWorld.OnRefreshObjects().AddRaw(this, &SMultiClientView::RefreshUI);
		ViewOptions.OnOptionsChanged().AddRaw(this, &SMultiClientView::RefreshUI);
		RebuildClientSubscriptions();
	}

	SMultiClientView::~SMultiClientView()
	{
		// Some of StreamEditor's columns reference some of our members.
		// Hence, StreamEditor needs to be destroyed before ~SCompoundWidget destroys it.
		// E.g. MultiStreamColumns::AssignedClientsColumn references ViewOptions so make sure ViewOptions is destroyed after the widget.
		ChildSlot.DetachWidget();
		StreamEditor.Reset();
		// These objects depend on ViewOptions. These resets are strictly not needed to be safe but doing this explicitly protects us in case
		// somebody moves declaration order of properties without realising dependency order.
		StreamModel.Reset();
		PropertyAssignmentView.Reset();
		
		OnlineClientManager->OnRemoteClientsChanged().RemoveAll(this);
		UserSelectedProperties->OnPropertySelectionChanged().RemoveAll(this);
		CleanClientSubscriptions();
	}

	TSharedRef<SWidget> SMultiClientView::CreateEditorContent(
		const TSharedRef<IConcertClient>& InConcertClient,
		FMultiUserReplicationManager& InMultiUserReplicationManager
		)
	{
		using namespace UE::ConcertSharedSlate;

		FMuteStateManager& MuteManager = *InMultiUserReplicationManager.GetMuteManager();
		FUserPropertySelector& PropertySelector = *InMultiUserReplicationManager.GetUserPropertySelector();
		
		ObjectHierarchy = ConcertClientSharedSlate::CreateObjectHierarchyForComponentHierarchy();
		const TSharedRef<IObjectNameModel> NameModel = ConcertClientSharedSlate::CreateEditorObjectNameModel();
		
		TAttribute<TSharedPtr<IMultiReplicationStreamEditor>> MultiStreamEditorAttribute =
		   TAttribute<TSharedPtr<IMultiReplicationStreamEditor>>::CreateLambda([this]()
		   {
			   return StreamEditor;
		   });
		FGetAutoAssignTarget GetAutoAssignTargetDelegate = FGetAutoAssignTarget::CreateLambda([this](TConstArrayView<UObject*>)
		{
			const TSharedRef<IEditableReplicationStreamModel>& LocalStream = OnlineClientManager->GetLocalClient().GetClientEditModel();
			return StreamModel->GetEditableStreams().Contains(LocalStream) ? LocalStream.ToSharedPtr() : nullptr;
		});
		
		FCreatePropertyTreeViewParams TreeViewParams
		{
			.PropertyColumns =
			{
				ReplicationColumns::Property::LabelColumn(),
				MultiStreamColumns::AssignPropertyColumn(
					MultiStreamEditorAttribute, *InMultiUserReplicationManager.GetUnifiedClientView(), ViewOptions
					)
			},
			.CreateCategoryRow = CreateDefaultCategoryGenerator(NameModel),
		};
		TreeViewParams.LeftOfPropertySearchBar.Widget = SAssignNew(PropertySelectionButton, SPropertySelectionComboButton, PropertySelector)
			.GetObjectDisplayString_Lambda([NameModel](const TSoftObjectPtr<>& Object){ return NameModel->GetObjectDisplayName(Object); });
		TreeViewParams.NoItemsContent.Widget = CreateNoPropertiesWarning();
		const TSharedRef<IPropertyTreeView> PropertyTreeView = CreateSearchablePropertyTreeView(MoveTemp(TreeViewParams));
		
		const TSharedRef<IPropertySourceProcessor> PropertySourceModel = PropertySelector.GetPropertySourceProcessor();
		PropertyAssignmentView = CreateMultiObjectAssignmentView(
			{ .PropertyTreeView = PropertyTreeView, .ObjectHierarchy = ObjectHierarchy, .PropertySource = PropertySourceModel}
			);
		PropertyAssignmentView->OnObjectGroupsChanged().AddLambda([this]()
		{
			PropertySelectionButton->RefreshSelectableProperties(PropertyAssignmentView->GetDisplayedGroups());
		});
		
		
		FCreateMultiStreamEditorParams Params
		{
			.MultiStreamModel = StreamModel.ToSharedRef(),
			.ConsolidatedObjectModel = ConcertClientSharedSlate::CreateTransactionalStreamModel(),
			.ObjectSource = MakeShared<ConcertClientSharedSlate::FActorSelectionSourceModel>(),
			.PropertySource = PropertySourceModel,
			.GetAutoAssignToStreamDelegate = MoveTemp(GetAutoAssignTargetDelegate),
			.OnPreAddSelectedObjectsDelegate = FSelectObjectsFromComboButton::CreateSP(this, &SMultiClientView::OnPreAddObjectsFromComboButton),
			.OnPostAddSelectedObjectsDelegate = FSelectObjectsFromComboButton::CreateSP(this, &SMultiClientView::OnPostAddObjectsFromComboButton),
		};
		FCreateViewerParams ViewerParams
		{
			.PropertyAssignmentView = PropertyAssignmentView.ToSharedRef(),
			// .ObjectHierarchy Do not assign so we only show the actors
			.NameModel = NameModel, // This makes actors use their labels, and components use the names given in the BP editor
			.OnExtendObjectsContextMenu = FExtendObjectMenu::CreateSP(this, &SMultiClientView::ExtendObjectContextMenu),
			.ObjectColumns =
			{
				MultiStreamColumns::MuteToggleColumn(MuteManager.GetChangeTracker()),
				MultiStreamColumns::AssignedClientsColumn(
					InConcertClient, MultiStreamEditorAttribute, *ObjectHierarchy, *InMultiUserReplicationManager.GetUnifiedClientView(), ViewOptions
					)
			},
			.ShouldDisplayObjectDelegate = FShouldDisplayObject::CreateSP(this, &SMultiClientView::ShouldDisplayObject),
			.MakeObjectRowOverlayWidgetDelegate = FMakeObjectRowOverlayWidget::CreateSP(this, &SMultiClientView::MakeObjectRowOverlayWidget),
			.WrapOutlinerWidgetDelegate = ConcertClientSharedSlate::CreateDropTargetOutlinerWrapper(
				{ ConcertClientSharedSlate::FDragDropReplicatableObject::CreateSP(this, &SMultiClientView::HandleDroppedObjects) }
				)
		};
		ViewerParams.RightOfObjectSearchBar.Widget = CreateRightOfSearchBarContent(InConcertClient, InMultiUserReplicationManager);
		StreamEditor = CreateBaseMultiStreamEditor(MoveTemp(Params), MoveTemp(ViewerParams));
		check(StreamEditor);
		
		return StreamEditor.ToSharedRef();
	}

	TSharedRef<SWidget> SMultiClientView::CreateNoPropertiesWarning() const
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoProperties", "Use Edit button to add replicated properties"))
			];
	}

	TSharedRef<SWidget> SMultiClientView::CreateRightOfSearchBarContent(
		const TSharedRef<IConcertClient>& InConcertClient,
		FMultiUserReplicationManager& InMultiUserReplicationManager
		)
	{
		return SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPresetComboButton, *InConcertClient, *InMultiUserReplicationManager.GetPresetManager())
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SMultiViewOptions, ViewOptions)
			];
	}

	TSet<FGuid> SMultiClientView::GetReplicatableClientIds() const
	{
		TSet<FGuid> ClientIds;
		StreamModel->ForEachDisplayedOnlineClient([&ClientIds](const FOnlineClient* Client)
		{
			ClientIds.Add(Client->GetEndpointId());
			return EBreakBehavior::Continue;
		});
		return ClientIds;
	}

	void SMultiClientView::EnumerateReplicatedObjectsInStreams(TFunctionRef<void(const FSoftObjectPath&)> Consumer) const
	{
		StreamModel->ForEachDisplayedOnlineClient([&Consumer](const FOnlineClient* Client)
		{
			Client->GetClientEditModel()->ForEachReplicatedObject([&Consumer](const FSoftObjectPath& Object)
			{
				Consumer(Object);
				return EBreakBehavior::Continue;
			});
			return EBreakBehavior::Continue;
		});
	}

	void SMultiClientView::RebuildClientSubscriptions()
	{
		CleanClientSubscriptions();

		OnlineClientSelectionModel->ForEachItem([this](FOnlineClient& Client)
		{
			Client.OnModelChanged().AddSP(this, &SMultiClientView::RefreshUI);
			Client.OnHierarchyNeedsRefresh().AddRaw(this, &SMultiClientView::RefreshUI);
			return EBreakBehavior::Continue;
		});

		RefreshUI();
	}

	void SMultiClientView::CleanClientSubscriptions() const
	{
		OnlineClientManager->ForEachClient([this](FOnlineClient& Client)
		{
			Client.OnModelChanged().RemoveAll(this);
			Client.OnHierarchyNeedsRefresh().RemoveAll(this);
			return EBreakBehavior::Continue;
		});
	}

	void SMultiClientView::RefreshUI() const
	{
		StreamEditor->GetEditorBase().Refresh();
	}

	void SMultiClientView::ExtendObjectContextMenu(FMenuBuilder& MenuBuilder, TConstArrayView<TSoftObjectPtr<>> ContextObjects) const
	{
		ContextMenuUtils::AddFrequencyOptionsIfOneContextObject_MultiClient(MenuBuilder, ContextObjects, *OnlineClientManager);

		if (ContextObjects.Num() == 1)
		{
			ContextMenuUtils::AddReassignmentOptions(
				MenuBuilder,
				ContextObjects[0],
				*ConcertClient,
				*OnlineClientManager,
				*ObjectHierarchy,
				OnlineClientManager->GetReassignmentLogic(),
				*StreamEditor
				);
		}
	}

	bool SMultiClientView::ShouldDisplayObject(const FSoftObjectPath& Object) const
	{
		return HideObjectsNotInEditorWorld.ShouldShowObject(Object);
	}

	TSharedRef<SWidget> SMultiClientView::MakeObjectRowOverlayWidget(const ConcertSharedSlate::FReplicatedObjectData& ReplicatedObjectData) const
	{
		return ConcertSyncCore::IsActor(ReplicatedObjectData.GetObjectPath())
			? SNew(SObjectOverlayRow, ReplicatedObjectData.GetObjectPath(), StreamEditor.ToSharedRef())
			: SNullWidget::NullWidget;
	}

	void SMultiClientView::HandleDroppedObjects(TConstArrayView<UObject*> DroppedObjects) const
	{
		EnableObjectExtensionOnAdd();

		ConcertSharedSlate::IEditableReplicationStreamModel& Model = StreamEditor->GetConsolidatedModel();
		TArray<UObject*> ObjectsToAdd; 
		Algo::TransformIf(DroppedObjects, ObjectsToAdd,
			[&Model](const TWeakObjectPtr<>& WeakObject)
			{
				UObject* Object = WeakObject.Get();
				return Object && !Model.ContainsObjects({ Object });
			},
			[](const TWeakObjectPtr<>& WeakObject){ return WeakObject.Get(); }
			);
		Model.AddObjects(ObjectsToAdd);

		DisableObjectExtensionOnAdd();
	}

	void SMultiClientView::EnableObjectExtensionOnAdd() const
	{
		OnlineClientManager->GetLocalClient().GetStreamExtender()
			.SetShouldExtend(true);
	}

	void SMultiClientView::DisableObjectExtensionOnAdd() const
	{
		OnlineClientManager->GetLocalClient().GetStreamExtender()
			.SetShouldExtend(false);
	}
}

#undef LOCTEXT_NAMESPACE