// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Data/ReplicatedObjectData.h"
#include "Replication/Editor/Model/ObjectSource/IObjectSourceModel.h"
#include "Replication/Editor/UnrealEditor/HideObjectsNotInWorldLogic.h"
#include "Selection/SelectionModelFwd.h"
#include "ViewOptions/MultiViewOptions.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IConcertClient;
class FMenuBuilder;

namespace UE::ConcertSharedSlate
{
	class IMultiObjectPropertyAssignmentView;
	class IObjectHierarchyModel;
	class IMultiReplicationStreamEditor;
	class IEditableReplicationStreamModel;
}

namespace UE::MultiUserClient::Replication
{
	class FGlobalAuthorityCache;
	class FMultiStreamModel;
	class FMultiUserReplicationManager;
	class FOfflineClientManager;
	class FOnlineClient;
	class FOnlineClientManager;
	class FUserPropertySelector;
	class SPropertySelectionComboButton;

	/** Displays a selection of clients. */
	class SMultiClientView
		: public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SMultiClientView){}
		SLATE_END_ARGS()

		void Construct(
			const FArguments& InArgs, const TSharedRef<IConcertClient>& InConcertClient,
			FMultiUserReplicationManager& InMultiUserReplicationManager,
			IOnlineClientSelectionModel& InOnlineClientSelectionModel,
			IOfflineClientSelectionModel& InOfflineClientSelectionModel
			);
		virtual ~SMultiClientView() override;

		const TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>& GetStreamEditor() const { return StreamEditor; }

	private:

		TSharedPtr<IConcertClient> ConcertClient;
		/** Keeps track of the properties that the user has selected to iterate on. */
		FUserPropertySelector* UserSelectedProperties = nullptr;
		
		// These are used to know when to refresh the UI.
		FOnlineClientManager* OnlineClientManager = nullptr;
		FOfflineClientManager* OfflineClientManager = nullptr;
		IOnlineClientSelectionModel* OnlineClientSelectionModel = nullptr;
		IOfflineClientSelectionModel* OfflineClientSelectionModel = nullptr;
		
		/**
		 * Controls the content shown in the UI.
		 * Important: Some systems keep a reference to ViewOptions. Evaluate destruction order if you move the member ordering.
		 */
		FMultiViewOptions ViewOptions;
		
		/** Combines the online and offline clients. */
		TSharedPtr<FMultiStreamModel> StreamModel;
		/** Displayed in the UI. */
		TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor> StreamEditor;
		/** Used by widgets in columns. */
		TSharedPtr<ConcertSharedSlate::IObjectHierarchyModel> ObjectHierarchy;

		/**
		 * This combo button is shown to the left of the search bar in the bottom half of the replication UI.
		 * It allows users to specify the properties they want to work on (i.e. these properties should be shown in the property view).
		 */
		TSharedPtr<SPropertySelectionComboButton> PropertySelectionButton;
		/** Displays the properties for the objects displayed in the top view. */
		TSharedPtr<ConcertSharedSlate::IMultiObjectPropertyAssignmentView> PropertyAssignmentView;
		
		/** This logic helps us decide whether an object should be displayed and lets us know that the object list needs to be refreshed (e.g. due to world change). */
		ConcertClientSharedSlate::FHideObjectsNotInWorldLogic HideObjectsNotInEditorWorld;

		/** Creates this widget's editor content */
		TSharedRef<SWidget> CreateEditorContent(const TSharedRef<IConcertClient>& InConcertClient, FMultiUserReplicationManager& InMultiUserReplicationManager);
		TSharedRef<SWidget> CreateNoPropertiesWarning() const;
		TSharedRef<SWidget> CreateRightOfSearchBarContent(const TSharedRef<IConcertClient>& InConcertClient, FMultiUserReplicationManager& InMultiUserReplicationManager);

		// SClientToolbar attributes
		/** @return Gets the clients that may be replicating */
		TSet<FGuid> GetReplicatableClientIds() const;
		/** Calls Consumer for each object path that is in a stream - independent of whether it is being replicated or not. */
		void EnumerateReplicatedObjectsInStreams(TFunctionRef<void(const FSoftObjectPath&)> Consumer) const;
		
		void RebuildClientSubscriptions();
		void CleanClientSubscriptions() const;
		void RefreshUI() const;
		
		/** Adds additional entries to the context menu for the object tree view. */
		void ExtendObjectContextMenu(FMenuBuilder& MenuBuilder, TConstArrayView<TSoftObjectPtr<>> ContextObjects) const;
		/** Decides whether the object should be displayed: do not show it if it's not in the editor world. */
		bool ShouldDisplayObject(const FSoftObjectPath& Object) const;

		/** Creates the widget that overlays actor rows. */
		TSharedRef<SWidget> MakeObjectRowOverlayWidget(const ConcertSharedSlate::FReplicatedObjectData& ReplicatedObjectData) const;
		
		/** When the user adds using the combo button, automatically add discover relevant objects and properties. */
		void OnPreAddObjectsFromComboButton(TArrayView<const ConcertSharedSlate::FSelectableObjectInfo>) const { EnableObjectExtensionOnAdd(); }
		void OnPostAddObjectsFromComboButton(TArrayView<const ConcertSharedSlate::FSelectableObjectInfo>) const { DisableObjectExtensionOnAdd(); }
		
		/** Called when objects are dropped into the view. */
		void HandleDroppedObjects(TConstArrayView<UObject*> DroppedObjects) const;

		/** Enables adding common properties and subobjects when an object is added to replication. */
		void EnableObjectExtensionOnAdd() const;
		/** Stops adding common properties and subobjects when an object is added to replication. */
		void DisableObjectExtensionOnAdd() const;
	};
}
