// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectOverlayRow.h"

#include "Replication/Editor/Model/IEditableMultiReplicationStreamModel.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"
#include "Replication/Editor/View/IMultiReplicationStreamEditor.h"

#include "Algo/AnyOf.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SObjectOverlayRow"

namespace UE::MultiUserClient::Replication
{
	void SObjectOverlayRow::Construct(
		const FArguments& InArgs,
		const FSoftObjectPath& InRootObject,
		const TSharedRef<ConcertSharedSlate::IMultiReplicationStreamEditor>& InStreamEditor
		)
	{
		RootObject = InRootObject;
		StreamEditor = InStreamEditor;
		ChildSlot
		[
			SNew(SButton)
			.ButtonStyle( FAppStyle::Get(), "SimpleButton")
			.IsEnabled(this, &SObjectOverlayRow::IsBinIconEnabled)
			.ToolTipText(this, &SObjectOverlayRow::GetBinIconToolTipText)
			.OnClicked(this, &SObjectOverlayRow::OnPressBinIcon)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Delete"))
			]
		];
	}

	FReply SObjectOverlayRow::OnPressBinIcon() const
	{
		ConcertSharedSlate::IEditableReplicationStreamModel& ConsolidatedModel = StreamEditor->GetConsolidatedModel();
		
		// We want to delete all children, too, i.e. the ones not listed in the outliner, such as components and other subobjects
		TArray<FSoftObjectPath> ObjectAndChildren = { RootObject };
		ConsolidatedModel.ForEachReplicatedObject([this, &ObjectAndChildren](const FSoftObjectPath& ReplicatedObject)
		{
			// ReplicatedObject is a child of a deleted object if the path before it contains one of the deleted objects
			const bool bIsChildOfDeletedObject = ReplicatedObject.ToString().Contains(RootObject.ToString());
			if (bIsChildOfDeletedObject)
			{
				ObjectAndChildren.Add(ReplicatedObject);
			}
			return EBreakBehavior::Continue;
		});
		
		ConsolidatedModel.RemoveObjects(ObjectAndChildren);
		return FReply::Handled();
	}

	FText SObjectOverlayRow::GetBinIconToolTipText() const
	{
		return IsBinIconEnabled()
			? LOCTEXT("ToolTipText.Enabled", "Removes actor and its subobjects")
			: LOCTEXT("ToolTipText.Disabled", "The properties are not editable.\nContent assigned to offline clients can not be edited until the clients rejoin.");
	}

	bool SObjectOverlayRow::IsBinIconEnabled() const
	{
		using namespace ConcertSharedSlate;
		
		const TSet<TSharedRef<IEditableReplicationStreamModel>> Models = StreamEditor->GetMultiStreamModel().GetEditableStreams();
		return Algo::AnyOf(Models, [this](const TSharedRef<IEditableReplicationStreamModel>& EditableModel)
		{
			bool bContainsObject = false;
			EditableModel->ForEachReplicatedObject([this, &bContainsObject](const FSoftObjectPath& ReplicatedObject)
			{
				// ReplicatedObject is a child of a deleted object if the path before it contains one of the deleted objects
				bContainsObject = ReplicatedObject.ToString().Contains(RootObject.ToString());
				return bContainsObject ? EBreakBehavior::Break : EBreakBehavior::Continue;
			});
			return bContainsObject;
		});
	}
}

#undef LOCTEXT_NAMESPACE