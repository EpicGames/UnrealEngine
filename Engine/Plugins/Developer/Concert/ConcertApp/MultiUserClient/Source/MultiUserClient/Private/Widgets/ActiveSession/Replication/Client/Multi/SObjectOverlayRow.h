// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertSharedSlate
{
	class IMultiReplicationStreamEditor;
}

namespace UE::MultiUserClient::Replication
{
	/** Overlaid on actor rows. */
	class SObjectOverlayRow : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SObjectOverlayRow){}
		SLATE_END_ARGS()

		void Construct(
			const FArguments& InArgs,
			const FSoftObjectPath& InRootObject,
			const TSharedRef<ConcertSharedSlate::IMultiReplicationStreamEditor>& InStreamEditor
			);

	private:

		/** The top-level object that this row is being shown for. */
		FSoftObjectPath RootObject;
		/** Used to delete the object. */
		TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor> StreamEditor;
		
		/** Called when the delete icon over an actor is pressed. Clears the entire hierarchy. */
		FReply OnPressBinIcon() const;
		
		FText GetBinIconToolTipText() const;
		bool IsBinIconEnabled() const;
	};
}

