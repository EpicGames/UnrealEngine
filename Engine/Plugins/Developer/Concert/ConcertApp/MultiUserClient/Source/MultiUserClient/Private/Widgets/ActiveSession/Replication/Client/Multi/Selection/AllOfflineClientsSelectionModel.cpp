// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllOfflineClientsSelectionModel.h"

#include "Replication/Client/Offline/OfflineClientManager.h"

namespace UE::MultiUserClient::Replication
{
	FAllOfflineClientsSelectionModel::FAllOfflineClientsSelectionModel(FOfflineClientManager& InClientManager)
		: ClientManager(InClientManager)
	{
		ClientManager.OnClientsChanged().AddRaw(this, &FAllOfflineClientsSelectionModel::OnClientsChanged);
	}

	FAllOfflineClientsSelectionModel::~FAllOfflineClientsSelectionModel()
	{
		ClientManager.OnClientsChanged().RemoveAll(this);
	}

	void FAllOfflineClientsSelectionModel::ForEachItem(TFunctionRef<EBreakBehavior(FOfflineClient&)> ProcessClient) const
	{
		ClientManager.ForEachClient([&ProcessClient](FOfflineClient& Client){ return ProcessClient(Client); });
	}

	void FAllOfflineClientsSelectionModel::OnClientsChanged()
	{
		OnSelectionChangedDelegate.Broadcast();
	}
}
