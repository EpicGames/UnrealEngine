// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISelectionModel.h"
#include "SelectionModelFwd.h"
#include "Replication/Client/Offline/OfflineClient.h"

#include "HAL/Platform.h"

namespace UE::MultiUserClient::Replication
{
	class FOfflineClientManager;
	
	/** Exposes all clients and detects when clients disconnect. */
	class FAllOfflineClientsSelectionModel : public IOfflineClientSelectionModel
	{
	public:
		
		FAllOfflineClientsSelectionModel(FOfflineClientManager& InClientManager UE_LIFETIMEBOUND);
		virtual ~FAllOfflineClientsSelectionModel() override;

		//~ Begin IOfflineClientSelectionModel Interface
		virtual void ForEachItem(TFunctionRef<EBreakBehavior(FOfflineClient&)> ProcessClient) const override;
		virtual FOnSelectionChanged& OnSelectionChanged() override { return OnSelectionChangedDelegate; }
		//~ End IOfflineClientSelectionModel Interface

	private:

		/** Informs us when the list of clients changes */
		FOfflineClientManager& ClientManager;
		
		/** Called when the clients ForEachSelectedClient enumerates has changed. */
		FOnSelectionChanged OnSelectionChangedDelegate;
		
		void OnClientsChanged();
	};
}
