// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllOnlineClientsSelectionModel.h"

#include "Replication/Client/Online/OnlineClientManager.h"

namespace UE::MultiUserClient::Replication
{
	FAllOnlineClientsSelectionModel::FAllOnlineClientsSelectionModel(FOnlineClientManager& InClientManager)
		: ClientManager(InClientManager)
	{
		ClientManager.OnRemoteClientsChanged().AddRaw(this, &FAllOnlineClientsSelectionModel::OnRemoteClientsChanged);
	}

	FAllOnlineClientsSelectionModel::~FAllOnlineClientsSelectionModel()
	{
		ClientManager.OnRemoteClientsChanged().RemoveAll(this);
	}

	void FAllOnlineClientsSelectionModel::ForEachItem(TFunctionRef<EBreakBehavior(FOnlineClient&)> ProcessClient) const
	{
		ClientManager.ForEachClient(ProcessClient);
	}

	void FAllOnlineClientsSelectionModel::OnRemoteClientsChanged()
	{
		OnSelectionChangedDelegate.Broadcast();
	}
}
