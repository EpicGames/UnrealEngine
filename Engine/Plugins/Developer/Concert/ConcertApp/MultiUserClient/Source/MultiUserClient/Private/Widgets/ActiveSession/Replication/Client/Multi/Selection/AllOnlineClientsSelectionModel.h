// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISelectionModel.h"
#include "SelectionModelFwd.h"

#include "HAL/Platform.h"

namespace UE::MultiUserClient::Replication
{
	class FOnlineClientManager;
	
	/** Exposes all clients and detects when clients disconnect. */
	class FAllOnlineClientsSelectionModel : public IOnlineClientSelectionModel
	{
	public:
		
		FAllOnlineClientsSelectionModel(FOnlineClientManager& InClientManager UE_LIFETIMEBOUND);
		virtual ~FAllOnlineClientsSelectionModel() override;

		//~ Begin IOnlineClientSelectionModel Interface
		virtual void ForEachItem(TFunctionRef<EBreakBehavior(FOnlineClient&)> ProcessClient) const override;
		virtual FOnSelectionChanged& OnSelectionChanged() override { return OnSelectionChangedDelegate; }
		//~ End IOnlineClientSelectionModel Interface

	private:

		/** Informs us when the list of clients changes */
		FOnlineClientManager& ClientManager;
		
		/** Called when the clients ForEachSelectedClient enumerates has changed. */
		FOnSelectionChanged OnSelectionChangedDelegate;
		
		void OnRemoteClientsChanged();
	};
}
