// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/Function.h"

enum class EBreakBehavior : uint8;

namespace UE::MultiUserClient::Replication
{
	class FOnlineClient;
	
	/** Abstracts the concept of a client selection. */
	template<typename TItemType>
	class ISelectionModel
	{
	public:

		/** Iterates through every client in the selection. */
		virtual void ForEachItem(TFunctionRef<EBreakBehavior(TItemType&)> ProcessItem) const = 0;

		DECLARE_MULTICAST_DELEGATE(FOnSelectionChanged);
		/** Called when the clients ForEachSelectedClient enumerates has changed. */
		virtual FOnSelectionChanged& OnSelectionChanged() = 0;
		
		virtual ~ISelectionModel() = default;
	};

}
