// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISelectionModel.h"

namespace UE::MultiUserClient::Replication
{
	class FOfflineClient;
	class FOnlineClient;
	struct FClientItem;
	
	using IOfflineClientSelectionModel = ISelectionModel<FOfflineClient>;
	using IOnlineClientSelectionModel = ISelectionModel<FOnlineClient>;
}
