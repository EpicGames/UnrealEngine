// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::MultiUserClient::Replication
{
	/** The view options for the SMultiClientView. */
	class FMultiViewOptions
	{
	public:

		bool ShouldShowOfflineClients() const { return bShowOfflineClients; }
		void SetShouldShowOfflineClients(bool bValue)
		{
			if (bShowOfflineClients != bValue)
			{
				bShowOfflineClients = bValue;
				OnOptionsChangedDelegate.Broadcast();
			}
		}
		void ToggleShouldShowOfflineClients() { SetShouldShowOfflineClients(!ShouldShowOfflineClients()); }

		/** Broadcasts when a view option changes. */
		FSimpleMulticastDelegate& OnOptionsChanged() { return OnOptionsChangedDelegate; }
		
	private:

		/** Whether offline clients should be shown. */
		bool bShowOfflineClients = true;

		/** Broadcasts when a view option changes. */
		FSimpleMulticastDelegate OnOptionsChangedDelegate;
	};
}

