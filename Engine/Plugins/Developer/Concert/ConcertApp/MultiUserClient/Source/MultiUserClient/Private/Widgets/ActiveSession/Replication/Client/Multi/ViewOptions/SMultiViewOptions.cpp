// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMultiViewOptions.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MultiViewOptions.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SMultiViewOptions"

namespace UE::MultiUserClient::Replication
{
	void SMultiViewOptions::Construct(const FArguments& InArgs, FMultiViewOptions& InViewOptions)
	{
		ViewOptions = &InViewOptions;
		ChildSlot
		[
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon") // Use the tool bar item style for this button
			.OnGetMenuContent(this, &SMultiViewOptions::GetViewButtonContent)
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
			]
		];
	}

	TSharedRef<SWidget> SMultiViewOptions::GetViewButtonContent() const
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowOfflineClients.Label", "Show offline clients"),
			LOCTEXT("ShowOfflineClients.ToolTipText", "Controls whether properties for clients rejoining the session are displayed."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]{ ViewOptions->ToggleShouldShowOfflineClients(); }),
				FCanExecuteAction::CreateLambda([]{ return true; }),
				FIsActionChecked::CreateLambda([this]{ return ViewOptions->ShouldShowOfflineClients(); })
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		return MenuBuilder.MakeWidget();
	}
}

#undef LOCTEXT_NAMESPACE
