// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::MultiUserClient::Replication
{
	class FMultiViewOptions;

	/** Displays FMultiViewOptions in a combo button. */
	class SMultiViewOptions : public SCompoundWidget
    {
    public:

		SLATE_BEGIN_ARGS(SMultiViewOptions){}
		SLATE_END_ARGS()

		/**
		 * @param InArgs The widget args
		 * @param InViewOptions Options governing the display settings. The caller ensures it outlives the lifetime of the widget.
		 */
		void Construct(const FArguments& InArgs, FMultiViewOptions& InViewOptions);

	private:

		/** The view options being displayed and mutated by this widget. */
		FMultiViewOptions* ViewOptions = nullptr;

		/** @return The menu content for the combo button */
		TSharedRef<SWidget> GetViewButtonContent() const;
    };
}

