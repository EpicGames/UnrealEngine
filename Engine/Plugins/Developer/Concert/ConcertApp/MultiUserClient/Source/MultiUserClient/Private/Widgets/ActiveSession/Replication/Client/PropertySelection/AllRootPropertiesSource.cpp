// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllRootPropertiesSource.h"

namespace UE::MultiUserClient::Replication
{
	namespace Private
	{
		static TArray<FUserSelectableProperty> PrebuildOptions(const ConcertSharedSlate::FObjectGroup& ObjectGroup, UClass& SharedClass)
		{
			TMap<FConcertPropertyChain, TArray<FConcertPropertyChain>> RootToChildren;
			
			ConcertClientSharedSlate::FReplicatablePropertySource(&SharedClass)
				.EnumerateProperties([&RootToChildren](const ConcertSharedSlate::FPropertyInfo& Info)
				{
					const FConcertPropertyChain& Property = Info.Property;
					if (Info.Property.IsRootProperty())
					{
						RootToChildren.Add(Info.Property);
					}
					else
					{
						RootToChildren[Info.Property.GetRootParent()].Add(Info.Property);
					}
					return EBreakBehavior::Continue;
				});
			
			TArray<FUserSelectableProperty> Result;
			for (TPair<FConcertPropertyChain, TArray<FConcertPropertyChain>>& Pair : RootToChildren)
			{
				Pair.Value.Add(Pair.Key);
				Result.Emplace(ObjectGroup, MoveTemp(Pair.Key), MoveTemp(Pair.Value));
			}
			return Result;
		}
	}
	
	FAllRootPropertiesSource::FAllRootPropertiesSource(ConcertSharedSlate::FBaseDisplayInfo BaseDisplayInfo, const ConcertSharedSlate::FObjectGroup& ObjectGroup, UClass& SharedClass)
		: BaseDisplayInfo(BaseDisplayInfo)
		, Options(Private::PrebuildOptions(ObjectGroup, SharedClass))
	{}

	ConcertSharedSlate::FSourceDisplayInfo FAllRootPropertiesSource::GetDisplayInfo() const
	{
		return { { BaseDisplayInfo }, ConcertSharedSlate::ESourceType::ShowAsToggleButtonList };
	}

	void FAllRootPropertiesSource::EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const FUserSelectableProperty& SelectableOption)> Delegate) const
	{
		for (const FUserSelectableProperty& Option : Options)
		{
			if (Delegate(Option) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}
}
