// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RootPropertySourceModel.h"
#include "IPropertyItemSource.h"
#include "Model/Item/IItemSourceModel.h"
#include "Replication/Editor/Model/PropertySource/ReplicatablePropertySource.h"

#include "Templates/UnrealTemplate.h"

class UClass;

namespace UE::MultiUserClient::Replication
{
	/** Adapts FReplicatablePropertySource. */
	class FAllRootPropertiesSource
		: public IPropertyItemSource
		, public FNoncopyable
	{
	public:

		FAllRootPropertiesSource(
			ConcertSharedSlate::FBaseDisplayInfo BaseDisplayInfo, const ConcertSharedSlate::FObjectGroup& ObjectGroup,
			UClass& SharedClass
			);

		//~ Begin IPropertyItemSource Interface
		virtual ConcertSharedSlate::FSourceDisplayInfo GetDisplayInfo() const override;
		virtual void EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const FUserSelectableProperty& SelectableOption)> Delegate) const override;
		//~ End IPropertyItemSource Interface

	private:

		/** Passed in display info about ObjectGroup. */
		const ConcertSharedSlate::FBaseDisplayInfo BaseDisplayInfo;

		/** The options being iterated. */
		const TArray<FUserSelectableProperty> Options;
	};
}

