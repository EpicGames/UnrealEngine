// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Item/IItemSourceModel.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Editor/View/IPropertyAssignmentView.h"

#include "Containers/Array.h"

namespace UE::MultiUserClient::Replication
{
	struct FUserSelectableProperty
	{
		/** The objects for which this property is selected */
		ConcertSharedSlate::FObjectGroup ObjectGroup;
		/** The property shown to the user as option. */
		FConcertPropertyChain RootProperty;
		/** The properties that become (un)selected when the user (un)selects RootProperty. Contains RootProperty. */
		TArray<FConcertPropertyChain> PropertiesToAdd;

		FUserSelectableProperty(ConcertSharedSlate::FObjectGroup ObjectGroup, FConcertPropertyChain Property, TArray<FConcertPropertyChain> PropertiesToAdd)
			: ObjectGroup(MoveTemp(ObjectGroup))
			, RootProperty(MoveTemp(Property))
			, PropertiesToAdd(MoveTemp(PropertiesToAdd))
		{}
	};
	
	using IPropertyItemSource = ConcertSharedSlate::IItemSourceModel<FUserSelectableProperty>;
}

