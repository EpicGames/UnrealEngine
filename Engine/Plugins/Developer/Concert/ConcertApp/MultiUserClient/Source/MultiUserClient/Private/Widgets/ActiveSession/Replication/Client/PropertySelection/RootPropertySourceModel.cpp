// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootPropertySourceModel.h"

#include "AllRootPropertiesSource.h"
#include "Replication/Editor/View/IPropertyAssignmentView.h"

#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "FRootPropertySourceModel"

namespace UE::MultiUserClient::Replication
{
	FRootPropertySourceModel::FRootPropertySourceModel(FGetObjectDisplayString InGetObjectDisplayStringDelegate)
		: GetObjectDisplayStringDelegate(MoveTemp(InGetObjectDisplayStringDelegate))
	{
		check(GetObjectDisplayStringDelegate.IsBound());
	}

	void FRootPropertySourceModel::RefreshSelectableProperties(TConstArrayView<ConcertSharedSlate::FObjectGroup> DisplayedObjectGroups)
	{
		PerObjectGroup_AllPropertiesSources.Empty(DisplayedObjectGroups.Num());
		
		for (const ConcertSharedSlate::FObjectGroup& Group : DisplayedObjectGroups)
		{
			const UObject* Object = Group.Group.IsEmpty() ? nullptr : Group.Group[0].Get();
			if (!ensure(Object))
			{
				continue;
			}

			UClass* Class = Object->GetClass();
			const FText Label = GetObjectDisplayStringDelegate.Execute(Group.Group[0]);
			const ConcertSharedSlate::FBaseDisplayInfo DisplayInfo
			{
				Label,
				FText::Format(LOCTEXT("ToolTipFmt", "Edit properties for {0}"), Label),
				FSlateIconFinder::FindIconForClass(Class)
			};
			
			PerObjectGroup_AllPropertiesSources.Add(MakeShared<FAllRootPropertiesSource>(DisplayInfo, Group, *Class));
		}
	}
}

#undef LOCTEXT_NAMESPACE