// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyItemSource.h"
#include "Model/Item/SourceSelectionCategory.h"
#include "Replication/Editor/View/IPropertyAssignmentView.h"
#include "SelectionDelegates.h"

#include "Templates/UnrealTemplate.h"

namespace UE::ConcertSharedSlate { struct FObjectGroup; }

namespace UE::MultiUserClient::Replication
{
	/**
	 * This model is used to build the combo button to the left of the search bar in the bottom half of the replication UI.
	 * It allows users to specify the properties they want to work on (i.e. these properties should be shown in the property view).
	 */
	class FRootPropertySourceModel : public FNoncopyable
	{
	public:

		FRootPropertySourceModel(FGetObjectDisplayString InGetObjectDisplayStringDelegate);
		
		/** Refreshes the properties that the user can select given the objects currently displayed in the bottom view. */
		void RefreshSelectableProperties(TConstArrayView<ConcertSharedSlate::FObjectGroup> DisplayedObjectGroups);

		/** Contains one source for each object group. Each source displays all root properties in that class. */
		const TArray<TSharedRef<IPropertyItemSource>>& GetPerObjectGroup_AllPropertiesSources() const { return PerObjectGroup_AllPropertiesSources; }

	private:
		
		/** Determines the display string of an object. */
		const FGetObjectDisplayString GetObjectDisplayStringDelegate;

		/** Contains one source for each object group. Each source displays all root properties in that class. */
		TArray<TSharedRef<IPropertyItemSource>> PerObjectGroup_AllPropertiesSources;
	};
}

