// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertySelectionComboButton.h"

#include "RootPropertySourceModel.h"
#include "Model/Item/SourceModelBuilders.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Misc/PropertySelection/UserPropertySelector.h"

#include "Algo/AllOf.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SPropertySelectionComboButton"

namespace UE::MultiUserClient::Replication
{
	namespace Private
	{
		static bool IsSelected(const FUserPropertySelector& UserSelection, const FUserSelectableProperty& Property)
		{
			return Algo::AllOf(Property.ObjectGroup.Group, [&UserSelection, &Property](const TSoftObjectPtr<>& Object)
			{
				return UserSelection.IsPropertySelected(Object.GetUniqueID(), Property.RootProperty);
			});
		}

		static void ToggleSelection(FUserPropertySelector& UserSelection, const FUserSelectableProperty& Property)
		{
			const bool bIsSelected = IsSelected(UserSelection, Property);
			for (const TSoftObjectPtr<>& SoftObjectPtr : Property.ObjectGroup.Group)
			{
				UObject* Object = SoftObjectPtr.Get();
				if (bIsSelected)
				{
					UserSelection.RemoveUserSelectedProperties(Object, Property.PropertiesToAdd);
				}
				else
				{
					UserSelection.AddUserSelectedProperties(Object, Property.PropertiesToAdd);
				}
			}
			
		}
	}
	
	void SPropertySelectionComboButton::Construct(const FArguments& InArgs, FUserPropertySelector& InPropertySelector)
	{
		PropertySelector = &InPropertySelector;
		PropertySourceModel = MakeUnique<FRootPropertySourceModel>(InArgs._GetObjectDisplayString);
		
		ChildSlot
		[
			SNew(SPositiveActionButton)
			.Text(LOCTEXT("Edit.Label", "Edit")) 
			.ToolTipText(LOCTEXT("Edit.ToolTip", "Select properties you want to work with"))
			.OnGetMenuContent(this, &SPropertySelectionComboButton::MakeMenu)
		];
	}

	void SPropertySelectionComboButton::RefreshSelectableProperties(TConstArrayView<ConcertSharedSlate::FObjectGroup> DisplayedObjectGroups) const
	{
		PropertySourceModel->RefreshSelectableProperties(DisplayedObjectGroups);
	}

	TSharedRef<SWidget> SPropertySelectionComboButton::MakeMenu() const
	{
		FMenuBuilder MenuBuilder(false, nullptr);
		const FModelBuilder::FItemPickerArgs PickerArgs = MakePickerArguments();
		
		MenuBuilder.AddSeparator();
		MenuBuilder.AddWidget(
			SNew(SBox)
			.Padding(3.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Instructions", "Add a property you want to replicate here."))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			],
			FText::GetEmpty()
		);
		
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("Section.AllProperties", "All Properties"));
		for (const TSharedRef<IPropertyItemSource>& ItemSource : PropertySourceModel->GetPerObjectGroup_AllPropertiesSources())
		{
			FModelBuilder::AddOptionToMenu(ItemSource, PickerArgs, MenuBuilder);
		}
		MenuBuilder.EndSection();
		
		return MenuBuilder.MakeWidget();
	}

	ConcertSharedSlate::FSourceModelBuilders<FUserSelectableProperty>::FItemPickerArgs SPropertySelectionComboButton::MakePickerArguments() const
	{
		return FModelBuilder::FItemPickerArgs
		(
			FModelBuilder::FOnItemsSelected::CreateSP(this, &SPropertySelectionComboButton::OnItemsSelected),
			FModelBuilder::FGetItemDisplayString::CreateLambda([](const FUserSelectableProperty& Item) -> FString
			{
				UObject* Object = Item.ObjectGroup.Group.IsEmpty() ? nullptr : Item.ObjectGroup.Group[0].Get();
				if (!Object)
				{
					return TEXT("Unknown object");
				}

				const FProperty* Property = Item.RootProperty.ResolveProperty(*Object->GetClass());
				return Property
					? Property->GetDisplayNameText().ToString()
					: TEXT("Unknown property");
			}),
			FModelBuilder::FGetItemIcon{},
			FModelBuilder::FIsItemSelected::CreateLambda([this](const FUserSelectableProperty& Item)
			{
				return Private::IsSelected(*PropertySelector, Item);
			})
		);
	}

	void SPropertySelectionComboButton::OnItemsSelected(TArray<FUserSelectableProperty> Properties) const
	{
		// Usually there will only be one changed property
		const FText Message = Properties.Num() == 1
			? FText::Format(
				LOCTEXT("Transaction.SingleFmt", "Change property '{0}'"),
				FText::FromString(Properties[0].RootProperty.ToString(FConcertPropertyChain::EToStringMethod::LeafProperty))
				)
			: LOCTEXT("Transaction.Multi", "Change properties");
		const FScopedTransaction Transaction(Message);
		
		for (const FUserSelectableProperty& Property : Properties)
		{
			Private::ToggleSelection(*PropertySelector, Property);
		}
	}
}

#undef LOCTEXT_NAMESPACE