// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RootPropertySourceModel.h"
#include "Model/Item/SourceModelBuilders.h"
#include "SelectionDelegates.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"

namespace UE::ConcertSharedSlate { struct FObjectGroup; }

namespace UE::MultiUserClient::Replication
{
	class FUserPropertySelector;
	struct FUserSelectableProperty;
	
	/**
	 * This combo button is shown to the left of the search bar in the bottom half of the replication UI.
	 * It allows users to specify the properties they want to work on (i.e. these properties should be shown in the property view).
	 */
	class SPropertySelectionComboButton : public SCompoundWidget
	{
		using FModelBuilder = ConcertSharedSlate::FSourceModelBuilders<FUserSelectableProperty>;
	public:

		SLATE_BEGIN_ARGS(SPropertySelectionComboButton) {}
			/** Gets the display string of objects in the drop-down */
			SLATE_EVENT(FGetObjectDisplayString, GetObjectDisplayString)
		SLATE_END_ARGS()

		/**
		 * @param InArgs Widget arguments
		 * @param InPropertySelector User to change the selected properties. The caller ensures it outlives the lifetime of the widget.
		 */
		void Construct(const FArguments& InArgs, FUserPropertySelector& InPropertySelector);

		/** Refreshes the properties that the user can select given the objects currently displayed in the bottom view. */
		void RefreshSelectableProperties(TConstArrayView<ConcertSharedSlate::FObjectGroup> DisplayedObjectGroups) const;

	private:

		/** Manages the user selected properties */
		FUserPropertySelector* PropertySelector = nullptr;
		
		/** Manages the button's content. */
		TUniquePtr<FRootPropertySourceModel> PropertySourceModel;

		/** @return The menu to display when the combo button is clicked */
		TSharedRef<SWidget> MakeMenu() const;
		FModelBuilder::FItemPickerArgs MakePickerArguments() const;
		
		/** Handles the user selecting an option in the combo button. */
		void OnItemsSelected(TArray<FUserSelectableProperty> Properties) const;
	};
}

