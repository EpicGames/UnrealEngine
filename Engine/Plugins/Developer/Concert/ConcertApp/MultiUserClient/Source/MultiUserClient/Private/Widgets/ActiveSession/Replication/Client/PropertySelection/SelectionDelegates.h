// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "UObject/SoftObjectPtr.h"

class FText;

namespace UE::MultiUserClient::Replication
{
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetObjectDisplayString, const TSoftObjectPtr<>&)
}

