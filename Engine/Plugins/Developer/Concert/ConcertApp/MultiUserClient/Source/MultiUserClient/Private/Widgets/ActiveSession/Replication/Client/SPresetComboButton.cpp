// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPresetComboButton.h"

#include "ConcertLogGlobal.h"
#include "Assets/MultiUserReplicationSessionPreset.h"
#include "Replication/Preset/PresetManager.h"

#include "Containers/StringFwd.h"
#include "Containers/Ticker.h"
#include "ContentBrowserModule.h"
#include "Engine/World.h"
#include "IContentBrowserSingleton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/StringBuilder.h"
#include "Replication/Preset/PresetUtils.h"
#include "SSimpleComboButton.h"
#include "Styling/AppStyle.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Client/SClientName.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SPresetComboButton"

namespace UE::MultiUserClient::Replication::Private
{
	static void LogActorsInPreset(const UMultiUserReplicationSessionPreset& Preset)
	{
		TStringBuilder<2048> StringBuilder;
		ForEachSavedActorLabel(Preset,
			[&StringBuilder](const FSoftObjectPath& ActorPath, const FString& Label)
			{
				StringBuilder << TEXT("\tLabel=\"") << *Label << TEXT("\", Path=\"") << *ActorPath.ToString() << TEXT("\"\n");
				return EBreakBehavior::Continue;
			});
		
		UE_LOGF(LogConcert, Warning, "No actors could be mapped to world %ls for preset %ls. Saved actors:\n%ls",
			GWorld ? *GWorld->GetPathName() : TEXT("none"),
			*Preset.GetPathName(),
			*StringBuilder
			);
	}
	
	static void LogPresetErrorsIfNeeded(
		const TWeakObjectPtr<const UMultiUserReplicationSessionPreset>& WeakPreset,
		EReplaceSessionContentErrorCode ErrorCode
		)
	{
		const UMultiUserReplicationSessionPreset* Preset = WeakPreset.Get();
		if (!Preset)
		{
			return;
		}

		switch (ErrorCode)
		{
		case EReplaceSessionContentErrorCode::NoObjectsFound:
			LogActorsInPreset(*Preset);
			break;
		default: break;
		}
	}
	
	static FText MakeTitle(const FReplaceSessionContentResult& Result, const FText& PresetText)
	{
		const FText Format = [&Result]()
		{
			switch (Result.ErrorCode)
			{
			case EReplaceSessionContentErrorCode::Success: return LOCTEXT("ApplyPreset.Title.SuccessFmt", "Applied {0} preset");
			case EReplaceSessionContentErrorCode::NoObjectsFound: return LOCTEXT("ApplyPreset.Title.NoObjectsFoundFmt", "No actors matched for {0} preset");
			default: return LOCTEXT("ApplyPreset.Title.FailFmt", "Failed to apply {0} preset");
			}
		}();
		
		return FText::Format(Format, PresetText);
	}

	static FText MakeSubText(const FReplaceSessionContentResult& Result)
	{
		switch (Result.ErrorCode)
		{
		case EReplaceSessionContentErrorCode::Success: return FText::GetEmpty();
		case EReplaceSessionContentErrorCode::NoObjectsFound: return LOCTEXT("ApplyPreset.SubText.NoObjectsFound", "No actors from the preset were found in the world.\n\nCheck the output log to see the actors saved in the preset.");
		case EReplaceSessionContentErrorCode::NoWorld: return LOCTEXT("ApplyPreset.SubText.NoWorld", "No world instance to remap preset content to.");
		case EReplaceSessionContentErrorCode::Cancelled: return LOCTEXT("ApplyPreset.SubText.Success", "Disconnected from session.");
		case EReplaceSessionContentErrorCode::InProgress: return LOCTEXT("ApplyPreset.SubText.InProgress", "Another operation is already in progress.");
		case EReplaceSessionContentErrorCode::Timeout: return LOCTEXT("ApplyPreset.SubText.Timeout", "Request timed out.");
		case EReplaceSessionContentErrorCode::FeatureDisabled: return LOCTEXT("ApplyPreset.SubText.FeatureDisabled", "This session does not support presets.");
		case EReplaceSessionContentErrorCode::Rejected: return LOCTEXT("ApplyPreset.SubText.Rejected", "Rejected by server.");
		default: checkNoEntry(); return FText::GetEmpty();
		}
	}

	static FText GetSavedClientsText() { return LOCTEXT("Save.IncludedClients.Label", "Saved clients"); }
}

namespace UE::MultiUserClient::Replication
{
	void SPresetComboButton::Construct(const FArguments& InArgs, const IConcertClient& InClient, FPresetManager& InPresetManager)
	{
		Client = &InClient;
		PresetManager = &InPresetManager;
		ChildSlot
		[
			SNew(SSimpleComboButton)
			.Icon(FAppStyle::Get().GetBrush("AssetEditor.SaveAsset"))
			.Text(LOCTEXT("Presets", "Presets"))
			.OnGetMenuContent(this, &SPresetComboButton::CreateMenuContent)
			.HasDownArrow(true)
		];
	}

	TSharedRef<SWidget> SPresetComboButton::CreateMenuContent()
	{
		FMenuBuilder MenuBuilder(false, nullptr);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("Section.Save", "Save preset"));
		BuildSaveMenuContent(MenuBuilder);
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("Section.Import", "Import preset"));
		BuildLoadMenuContent(MenuBuilder);
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void SPresetComboButton::BuildSaveMenuContent(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Save.SavePresetAs.Label", "Save Preset as..."),
			TAttribute<FText>::CreateLambda([this]()
			{
				const ECanSaveResult CanSaveResult = PresetManager->CanSavePreset(BuildSaveOptions());
				switch (CanSaveResult)
				{
				case ECanSaveResult::Yes: return LOCTEXT("Save.SavePresetAs.ToolTip.Yes", "Saves what each client is replicating as a preset.");
				case ECanSaveResult::NoClients: return FText::Format(LOCTEXT("Save.SavePresetAs.ToolTip.NoClients", "Selec the clients you want to save first in '{0}'"), Private::GetSavedClientsText());
				default: return FText::GetEmpty();
				}
				
			}),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAssetAs"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SPresetComboButton::SavePresetAs),
				FCanExecuteAction::CreateLambda([this]{ return PresetManager->CanSavePreset(BuildSaveOptions()) == ECanSaveResult::Yes; })
				)	
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Save.IncludeAllClients.Label", "Save all clients"),
			LOCTEXT("Save.IncludeAllClients.ToolTip", "Whether you want to include all clients in the session into the preset."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ Options.bIncludeAllClients = !Options.bIncludeAllClients; }),
				FCanExecuteAction::CreateLambda([](){ return true; }),
				FIsActionChecked::CreateLambda([this]{ return Options.bIncludeAllClients; })
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);
		
		MenuBuilder.AddSubMenu(
			Private::GetSavedClientsText(),
			LOCTEXT("Save.IncludedClients.ToolTip", "Select the clients you want to save into the preset"),
			FNewMenuDelegate::CreateRaw(this, &SPresetComboButton::BuildExcludedClientSubmenu),
			FUIAction(
				FExecuteAction::CreateLambda([]{}),
				FCanExecuteAction::CreateLambda([this]{ return !Options.bIncludeAllClients; }),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateLambda([this]{ return !Options.bIncludeAllClients; })
				),
			NAME_None,
			EUserInterfaceActionType::None
			);
	}

	void SPresetComboButton::BuildExcludedClientSubmenu(FMenuBuilder& MenuBuilder)
	{
		const TSharedPtr<IConcertClientSession> Session = Client->GetCurrentSession();
		if (!ensure(Session))
		{
			return;
		}

		const auto GetDisplayText = [](const FConcertClientInfo& ClientInfo, bool bDisplayAsLocalClient = false)
		{
			return ConcertSharedSlate::SClientName::GetDisplayText(ClientInfo, bDisplayAsLocalClient);
		};
		
		TArray<FConcertSessionClientInfo> RemoteClients = Session->GetSessionClients();
		RemoteClients.Sort([&GetDisplayText](const FConcertSessionClientInfo& Left, const FConcertSessionClientInfo& Right)
		{
			return GetDisplayText(Left.ClientInfo).ToString() < GetDisplayText(Right.ClientInfo).ToString();
		});

		const auto AddClient = [this, &MenuBuilder, &GetDisplayText](const FConcertClientInfo& ClientInfo, bool bDisplayHasLocalClient = false)
		{
			const FText DisplayText = GetDisplayText(ClientInfo, bDisplayHasLocalClient);
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("Save.Client.LabelFmt", "{0}"), DisplayText),
				FText::Format(LOCTEXT("Save.Client.ToolTipFmt", "Check if you want the {0} saved in the preset."), DisplayText),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, ClientInfo]()
					{
						if (Options.IncludedClients.Contains(ClientInfo))
						{
							Options.IncludedClients.RemoveSingle(ClientInfo);
						}
						else
						{
							Options.IncludedClients.Add(ClientInfo);
						}
					}),
					FCanExecuteAction::CreateLambda([](){ return true; }),
					FIsActionChecked::CreateLambda([this, ClientInfo]{ return Options.IncludedClients.Contains(ClientInfo); })
					),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
				);
		};
		AddClient(Session->GetLocalClientInfo(), true);
		for (const FConcertSessionClientInfo& RemoteClient : RemoteClients)
		{
			AddClient(RemoteClient.ClientInfo);
		}
	}

	void SPresetComboButton::BuildLoadMenuContent(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ImportPreset.ClearOtherClients.Label", "Clear clients not in preset"),
			LOCTEXT("ImportPreset.ClearOtherClients.ToolTip", "Clients that were not in the session when the preset was created will get their content reset, too."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ Options.bResetAllOtherClients = true; }),
				FCanExecuteAction::CreateLambda([](){ return true; }),
				FIsActionChecked::CreateLambda([this]{ return Options.bResetAllOtherClients; })
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ImportPreset.AdditivelyAdd.Label", "Only change clients in preset"),
			LOCTEXT("ImportPreset.AdditivelyAdd.ToolTip", "Clients that were not in the session when the preset was created will not be modified by this preset."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ Options.bResetAllOtherClients = false; }),
				FCanExecuteAction::CreateLambda([](){ return true; }),
				FIsActionChecked::CreateLambda([this]{ return !Options.bResetAllOtherClients; })
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);
			
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.Filter.ClassPaths.Add(UMultiUserReplicationSessionPreset::StaticClass()->GetClassPathName());
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.Filter.bRecursiveClasses = false;
		AssetPickerConfig.OnAssetSelected.BindSP(this, &SPresetComboButton::LoadPreset);
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.bAllowDragging = false;
		
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.HeightOverride(450)
			.WidthOverride(320)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				]
			];
		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}

	void SPresetComboButton::SavePresetAs() const
	{
		PresetManager->ExportToPresetAndSaveAs(
			BuildSaveOptions()
			);
	}

	void SPresetComboButton::LoadPreset(const FAssetData& AssetData) const
	{
		FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
		if (UMultiUserReplicationSessionPreset* Preset = Cast<UMultiUserReplicationSessionPreset>(AssetData.GetAsset()))
		{
			FText PresetText = FText::FromName(Preset->GetFName());
			FNotificationInfo Info(FText::Format(LOCTEXT("ApplyPreset.Title.InProgressFmt", "Applying preset {0}"), PresetText));
			Info.ExpireDuration = 4.f;
			TSharedPtr<SNotificationItem> Notification = NotificationManager.AddNotification(Info);
			Notification->SetCompletionState(SNotificationItem::CS_Pending);

			PresetManager->ReplaceSessionContentWithPreset(*Preset, BuildFlags())
				.Next([Preset, PresetText = MoveTemp(PresetText), Notification = MoveTemp(Notification)]
					(const FReplaceSessionContentResult& Result) mutable
				{
					ExecuteOnGameThread(TEXT("SPresetComboButton"),
						[Preset = TWeakObjectPtr<const UMultiUserReplicationSessionPreset>(Preset),
							PresetText = MoveTemp(PresetText),
							Notification = MoveTemp(Notification),
							Result]
					{
						Private::LogPresetErrorsIfNeeded(Preset, Result.ErrorCode);
						Notification->SetText(Private::MakeTitle(Result, PresetText));
						Notification->SetSubText(Private::MakeSubText(Result));
						Notification->SetCompletionState(Result.IsSuccess() ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
					});
				});
		}
		else
		{
			FNotificationInfo Info(LOCTEXT("FailedToLoad", "Failed to load preset"));
			Info.ExpireDuration = 4.f;
			NotificationManager.AddNotification(Info)
				->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}

	EApplyPresetFlags SPresetComboButton::BuildFlags() const
	{
		EApplyPresetFlags Flags = EApplyPresetFlags::None;

		if (Options.bResetAllOtherClients)
		{
			Flags |= EApplyPresetFlags::ClearUnreferencedClients;
		}

		return Flags;
	}

	FSavePresetOptions SPresetComboButton::BuildSaveOptions() const
	{
		FSavePresetOptions SavePresetOptions;
		
		if (!Options.bIncludeAllClients)
		{
			const auto Filter = [this](const FConcertClientInfo& Info)
			{
				return Options.IncludedClients.Contains(Info) ? EFilterResult::Include : EFilterResult::Exclude;
			};
			SavePresetOptions.ClientFilterDelegate.BindLambda(Filter);
		}

		return SavePresetOptions;
	}
}

#undef LOCTEXT_NAMESPACE