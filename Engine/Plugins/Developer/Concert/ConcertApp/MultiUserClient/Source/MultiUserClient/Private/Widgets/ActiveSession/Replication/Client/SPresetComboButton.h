// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Disconnected/ConcertClientSessionBrowserController.h"

namespace UE::MultiUserClient::Replication
{
	struct FSavePresetOptions;
}

class FMenuBuilder;
class IConcertClient;
namespace UE::MultiUserClient::Replication { enum class EApplyPresetFlags : uint8; }

namespace UE::MultiUserClient::Replication
{
	class FPresetManager;

	/**
	 * A combo button displayed to the right of the object search bar.
	 * Its menu allows the user to save and load replication presets.
	 */
	class SPresetComboButton : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SPresetComboButton) {}
			
		SLATE_END_ARGS()

		/**
		 * @param InArgs Widget arguments
		 * @param InClient Caller ensures that it outlives the lifetime of this widget.
		 * @param InPresetManager Caller ensures that it outlives the lifetime of this widget.
		 */
		void Construct(const FArguments& InArgs, const IConcertClient& InClient, FPresetManager& InPresetManager);

	private:

		/** Used to get clients in the session (for filtering purposes). */
		const IConcertClient* Client = nullptr;
		/** Used to save & load preset. */
		FPresetManager* PresetManager = nullptr;

		struct FPresetOptions
		{
			/** Clients not mentioned by the preset will get their content wiped. */
			bool bResetAllOtherClients = true;

			/** Whether the user want so to capture all clients in the preset. */
			bool bIncludeAllClients = true;
			
			/** The clients that the preset should not be captured for */
			TArray<FConcertClientInfo> IncludedClients;
		} Options;

		/** Creates the Save & Load options for the menu. */
		TSharedRef<SWidget> CreateMenuContent();
		void BuildSaveMenuContent(FMenuBuilder& MenuBuilder);
		void BuildExcludedClientSubmenu(FMenuBuilder& MenuBuilder);
		void BuildLoadMenuContent(FMenuBuilder& MenuBuilder);

		/** Saves the current session content. */
		void SavePresetAs() const;
		/** Handles loading the preset. */
		void LoadPreset(const FAssetData& AssetData) const;

		EApplyPresetFlags BuildFlags() const;
		FSavePresetOptions BuildSaveOptions() const;
	};
}

