// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReplicationStatus.h"

#include "Replication/Misc/GlobalAuthorityCache.h"

#include "Algo/AnyOf.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SReplicationStatus"

namespace UE::MultiUserClient::Replication
{
	namespace Private
	{
		static TOptional<FSoftObjectPath> GetOwningActorOf(const FSoftObjectPath& SoftObjectPath)
		{
			// Example of an actor called floor
			// SoftObjectPath = { AssetPath = {PackageName = "/Game/Maps/SyncBoxLevel", AssetName = "SyncBoxLevel"}, SubPathString = "PersistentLevel.Floor" } }
			const FString& SubPathString = SoftObjectPath.GetSubPathString();

			constexpr int32 PersistentLevelStringLength = 16; // "PersistentLevel." has 16 characters
			const bool bIsWorldObject = SubPathString.Contains(TEXT("PersistentLevel."), ESearchCase::CaseSensitive);
			if (!bIsWorldObject)
			{
				// Not a path to a world object
				return {};
			}

			// Start search after the . behind PersistentLevel
			const int32 StartSearch = PersistentLevelStringLength + 1;
			const int32 IndexOfDotAfterActorName = SubPathString.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromStart, StartSearch);
			if (IndexOfDotAfterActorName == INDEX_NONE)
			{
				// SoftObjectPath points to an actor
				return SoftObjectPath;
			}

			const int32 NumToChopOffRight = SubPathString.Len() - IndexOfDotAfterActorName;
			const FString NewSubstring = SubPathString.LeftChop(NumToChopOffRight);
			const FSoftObjectPath PathToOwningActor(SoftObjectPath.GetAssetPath(), NewSubstring);
			return PathToOwningActor;
		}
	}

	void SReplicationStatus::AppendReplicationStatus(SVerticalBox& VerticalBox, FGlobalAuthorityCache& InAuthorityCache, const FArguments& InArgs)
	{
		VerticalBox.AddSlot()
			.AutoHeight()
			.Padding(4, 3)
			[
				SNew(SSeparator)
			];

		VerticalBox.AddSlot()
			.AutoHeight()
			.Padding(4, 0, 4, 3)
			[
				SNew(SReplicationStatus, InAuthorityCache)
				.ReplicatableClients(InArgs._ReplicatableClients)
				.ForEachObjectInStream(InArgs._ForEachObjectInStream)
			];
	}

	void SReplicationStatus::Construct(const FArguments& InArgs, FGlobalAuthorityCache& InAuthorityCache)
	{
		AuthorityCache = &InAuthorityCache;
		
		ReplicatableClientsAttribute = InArgs._ReplicatableClients;
		check(ReplicatableClientsAttribute.IsBound() || ReplicatableClientsAttribute.IsSet());
		ForEachObjectInStreamDelegate = InArgs._ForEachObjectInStream;
		check(ForEachObjectInStreamDelegate.IsBound());

		AuthorityCache->OnCacheChanged().AddSP(this, &SReplicationStatus::OnAuthorityCacheChanged);
		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ActorsText, STextBlock)
				.Text(LOCTEXT("Replicating", "Replicating "))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ObjectsText, STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("BoldFont"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ActorsText, STextBlock)
				.Text(LOCTEXT("For", " for "))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ActorsText, STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("BoldFont"))
			]
		];
		
		RefreshStatusText();
	}

	SReplicationStatus::~SReplicationStatus()
	{
		AuthorityCache->OnCacheChanged().RemoveAll(this);
	}

	void SReplicationStatus::RefreshStatusText()
	{
		const TSet<FGuid> ReplicatingClients = ReplicatableClientsAttribute.Get();

		TSet<FSoftObjectPath> ReplicatedActors;
		TSet<FSoftObjectPath> ReplicatedObjects;
		ForEachObjectInStreamDelegate.Execute([this, &ReplicatingClients, &ReplicatedActors, &ReplicatedObjects](const FSoftObjectPath& Path)
		{
			const bool bIsReplicated = Algo::AnyOf(AuthorityCache->GetClientsWithAuthorityOverObject(Path), [&ReplicatingClients](const FGuid& ClientId)
			{
				return ReplicatingClients.Contains(ClientId);
			});
			if (bIsReplicated)
			{
				const TOptional<FSoftObjectPath> PathToOwningActor = Private::GetOwningActorOf(Path);
				if (PathToOwningActor)
				{
					ReplicatedActors.Add(*PathToOwningActor);
				}
				
				ReplicatedObjects.Add(Path);
			}
			return EBreakBehavior::Continue;
		});

		ObjectsText->SetText(FText::Format(LOCTEXT("ObjectsTextFmt", "{0} {0}|plural(one=Object,other=Objects)"),
			ReplicatedObjects.Num()
			));
		ActorsText->SetText(FText::Format(LOCTEXT("ActorsTextFmt", "{0} {0}|plural(one=Actor,other=Actors)"),
			ReplicatedActors.Num()
			));
	}
}

#undef LOCTEXT_NAMESPACE