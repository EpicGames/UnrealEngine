// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class STextBlock;
class SVerticalBox;

namespace UE::ConcertSharedSlate { class IReplicationStreamModel; }

namespace UE::MultiUserClient::Replication
{
	class FGlobalAuthorityCache;

	DECLARE_DELEGATE_OneParam(FForEachObjectInStream, TFunctionRef<void(const FSoftObjectPath&)> Consumer);
	
	/**
	 * Displays a text "Replicating x Objects for y Actors".
	 *
	 * This view sums up all objects being replicated and shows the distinct actors being replicated.
	 * Example: You are replicating AActor::ActorGuid and USceneComponent::RelativeLocation on an actor called Floor.
	 * Result: "Replicating 2 Objects for 1 Actor".
	 */
	class SReplicationStatus : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SReplicationStatus)
		{}
			/** The clients that may be replicating, for which we show the stats. */
			SLATE_ATTRIBUTE(TSet<FGuid>, ReplicatableClients)

			/** Delegate which enumerates every object registered in a stream - independent of whether it is being replicated or not. */
			SLATE_EVENT(FForEachObjectInStream, ForEachObjectInStream)
		SLATE_END_ARGS()

		/** Adds a separator and SReplicationStatus to the bottom of the VerticalBox- */
		static void AppendReplicationStatus(SVerticalBox& VerticalBox, FGlobalAuthorityCache& InAuthorityCache, const FArguments& InArgs);

		void Construct(const FArguments& InArgs, FGlobalAuthorityCache& InAuthorityCache);
		virtual ~SReplicationStatus() override;

		/** Updates the status text after an external update has occured. */
		void RefreshStatusText();

	private:

		/** Used to get authority state of objects and informs us when authority changes. */
		FGlobalAuthorityCache* AuthorityCache = nullptr;

		/** The clients that may be replicating, for which we show the stats. */
		TAttribute<TSet<FGuid>> ReplicatableClientsAttribute;
		/** Delegate which enumerates every object registered in a stream - independent of whether it is being replicated or not. */
		FForEachObjectInStream ForEachObjectInStreamDelegate;

		/** Updated when authority changes. Displays subobjects in bold. */
		TSharedPtr<STextBlock> ObjectsText;
		/** Updated when authority changes. Displays actors in bold. */
		TSharedPtr<STextBlock> ActorsText;

		void OnAuthorityCacheChanged(const FGuid&) { RefreshStatusText(); }
	};
}

