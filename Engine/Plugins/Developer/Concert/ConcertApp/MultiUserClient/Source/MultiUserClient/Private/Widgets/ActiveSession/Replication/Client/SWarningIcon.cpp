// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWarningIcon.h"

#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"

namespace UE::MultiUserClient::Replication
{
	void SWarningIcon::Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor"))
				.DesiredSizeOverride(FVector2D(16,16))
			]
		];
	}
}

