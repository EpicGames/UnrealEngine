// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::MultiUserClient::Replication
{
	/** Displays a warning triangle */
	class SWarningIcon : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SWarningIcon)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
	};
}


