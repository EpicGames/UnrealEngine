// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::MultiUserClient::Replication
{
	/** Describes a type of view that is being displayed in SReplicationJoinedView and can get selected by the combo button. */
	enum class EClientViewType 
	{
		// When changing this enum, make sure to update the slot order of SReplicationJoinedView's widget switcher
		AllClients = 0,
		LocalClient = 1,
		RemoteClient = 2
	};
}