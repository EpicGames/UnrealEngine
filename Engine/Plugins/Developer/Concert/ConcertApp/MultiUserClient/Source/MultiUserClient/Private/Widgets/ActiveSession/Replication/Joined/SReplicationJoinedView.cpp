// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReplicationJoinedView.h"

#include "IConcertSyncClient.h"
#include "Replication/MultiUserReplicationManager.h"
#include "Widgets/ActiveSession/Replication/Client/Multi/SAllClientsView.h"

namespace UE::MultiUserClient::Replication
{
	void SReplicationJoinedView::Construct(
		const FArguments& InArgs,
		const TSharedRef<FMultiUserReplicationManager>& InReplicationManager,
		const TSharedRef<IConcertSyncClient>& InClient
		)
	{
		ChildSlot
		[
			SAssignNew(AllClientsView, SAllClientsView, InClient->GetConcertClient(), *InReplicationManager)
		];
	}
}