// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IConcertSyncClient;
class SNotificationItem;
class SWidgetSwitcher;

namespace UE::ConcertSharedSlate { class IReplicationStreamEditor; }

namespace UE::MultiUserClient::Replication
{
	class FMultiUserReplicationManager;
	class SAllClientsView;

	/** This widget is displayed by SReplicationRootWidget when the client has joined replication. */
	class SReplicationJoinedView : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SReplicationJoinedView)
		{}
		SLATE_END_ARGS()

		void Construct(
			const FArguments& InArgs,
			const TSharedRef<FMultiUserReplicationManager>& InReplicationManager,
			const TSharedRef<IConcertSyncClient>& InClient
			);

		const TSharedPtr<SAllClientsView>& GetAllClientsView() const { return AllClientsView; }

	private:

		/** The local client this widget is created for. */
		TSharedPtr<IConcertSyncClient> Client;
		/** Acts as the model of this view */
		TSharedPtr<FMultiUserReplicationManager> ReplicationManager;

		/** Shows the content of all replication clients. */
		TSharedPtr<SAllClientsView> AllClientsView;
	};
}