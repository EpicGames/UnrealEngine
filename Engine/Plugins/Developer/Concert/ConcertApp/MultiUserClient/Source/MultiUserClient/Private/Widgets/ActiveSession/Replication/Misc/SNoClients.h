// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SNoClients"

namespace UE::MultiUserClient::Replication
{
	/** Special widget that indicates no clients in widgets that list clients, like ownership reassignment combo box. */
	class SNoClients : public SCompoundWidget
	{
	public:

		static FText GetDisplayText() { return LOCTEXT("None", " - "); }
		
		SLATE_BEGIN_ARGS(SNoClients)
		{}
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs)
		{
			ChildSlot
			[
				SNew(STextBlock)
				.Text(GetDisplayText())
			];
		}
	};
}

#undef LOCTEXT_NAMESPACE