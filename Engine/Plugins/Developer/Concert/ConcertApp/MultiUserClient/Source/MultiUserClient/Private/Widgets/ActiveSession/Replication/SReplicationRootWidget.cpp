// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReplicationRootWidget.h"

#include "Widgets/ActiveSession/Replication/Joined/SReplicationJoinedView.h"
#include "Replication/MultiUserReplicationManager.h"

#include "Styling/AppStyle.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SReplicationRootWidget"

namespace UE::MultiUserClient::Replication
{
	void SReplicationRootWidget::Construct(
		const FArguments& InArgs,
		TSharedRef<FMultiUserReplicationManager> InReplicationManager,
		TSharedRef<IConcertSyncClient> InClient
		)
	{
		Client = MoveTemp(InClient);
		ReplicationManager = MoveTemp(InReplicationManager);
		
		// Show different widget based on whether we've joined replication.
		// We'll join replication automatically on session start but this may be rejected (in practice rejection should not happen though)
		ReplicationManager->OnReplicationConnectionStateChanged().AddSP(this, &SReplicationRootWidget::OnReplicationConnectionStateChanged);
		OnReplicationConnectionStateChanged(ReplicationManager->GetConnectionState());
	}

	void SReplicationRootWidget::OnReplicationConnectionStateChanged(EMultiUserReplicationConnectionState NewState)
	{
		switch (NewState)
		{
		case EMultiUserReplicationConnectionState::Connecting:
			ShowWidget_Connecting();
			break;
		case EMultiUserReplicationConnectionState::Connected:
			ShowWidget_Connected();
			break;
		case EMultiUserReplicationConnectionState::Disconnected:
			ShowWidget_Disconnected();
			break;
		default: checkNoEntry();
		}
	}

	void SReplicationRootWidget::ShowWidget_Connecting()
	{
		ConnectedView.Reset();
		
		ChildSlot
		[
			SNew(SBox)
			.Padding(0.f, 0.f, 0.f, 10.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SHorizontalBox)
				
				+SHorizontalBox::Slot()
				[
					SNew(SThrobber)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SuccessfullyConnected", "Joining replication session"))
					.Justification(ETextJustify::Center)
				]
			]
		];
	}

	void SReplicationRootWidget::ShowWidget_Connected()
	{
		ChildSlot
		[
			SAssignNew(ConnectedView, SReplicationJoinedView, ReplicationManager.ToSharedRef(), Client.ToSharedRef())
		];
	}

	void SReplicationRootWidget::ShowWidget_Disconnected()
	{
		ConnectedView.Reset();
		
		ChildSlot
		[
			SNew(SBox)
			.Padding(0.f, 0.f, 0.f, 10.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FailedToConnect", "Failed to join replication session."))
					.Justification(ETextJustify::Center)
				]

				+SVerticalBox::Slot()
				.Padding(0, 4, 0, 0)
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SButton)
					.OnClicked_Lambda([this]()
					{
						ReplicationManager->JoinReplicationSession();
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Text(LOCTEXT("EnableLoggingVisibility.Button.Text", "Join replication session"))
					]
				]
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE