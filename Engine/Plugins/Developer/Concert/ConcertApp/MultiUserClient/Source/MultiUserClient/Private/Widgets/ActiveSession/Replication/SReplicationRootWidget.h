// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SSplitter.h"

class IConcertSyncClient;

namespace UE::MultiUserClient::Replication
{
	class FMultiUserReplicationManager;
	class SReplicationJoinedView;
	
	enum class EMultiUserReplicationConnectionState : uint8;

	/** Root widget for replication in Multi-User session. */
	class SReplicationRootWidget : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SReplicationRootWidget)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FMultiUserReplicationManager> InReplicationManager, TSharedRef<IConcertSyncClient> InClient);

		const TWeakPtr<SReplicationJoinedView>& GetConnectedView() const { return ConnectedView; }

	private:

		/** The client this widget was created for. */
		TSharedPtr<IConcertSyncClient> Client;
		/** Manages the business logic which we represent. */
		TSharedPtr<FMultiUserReplicationManager> ReplicationManager;

		/** Shown when the replication handshake has been completed. Invalid otherwise. */
		TWeakPtr<SReplicationJoinedView> ConnectedView;
		
		/** Called when joining replication session state changes. */
		void OnReplicationConnectionStateChanged(EMultiUserReplicationConnectionState);

		void ShowWidget_Connecting();
		void ShowWidget_Connected();
		void ShowWidget_Disconnected();
	};
}

