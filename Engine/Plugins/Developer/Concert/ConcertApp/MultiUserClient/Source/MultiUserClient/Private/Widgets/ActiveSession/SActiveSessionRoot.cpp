// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActiveSessionRoot.h"

#include "Overview/SActiveSessionOverviewTab.h"
#include "Replication/SReplicationRootWidget.h"
#include "SActiveSessionToolbar.h"
#include "STabArea.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SActiveSessionRoot"

namespace UE::MultiUserClient
{
	const FName SActiveSessionRoot::SessionOverviewTabId(TEXT("OverviewTabId"));
	const FName SActiveSessionRoot::ReplicationTabId(TEXT("ReplicationTabId"));
	
	void SActiveSessionRoot::Construct(
		const FArguments& InArgs,
		TSharedPtr<IConcertSyncClient> InConcertSyncClient,
		TSharedRef<Replication::FMultiUserReplicationManager> InReplicationManager
		)
	{
		ChildSlot
		[
			SNew(SVerticalBox)

			// Toolbar
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.0f)
				[
					SNew(SActiveSessionToolbar, InConcertSyncClient)
					.TabArea()
					[
						CreateTabArea()
					]
				]
			]

			// Tabs
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(TabSwitcher, SWidgetSwitcher)

				+SWidgetSwitcher::Slot()
				[
					SAssignNew(OverviewContent, SActiveSessionOverviewTab, InConcertSyncClient)
				]
				+SWidgetSwitcher::Slot()
				[
					SAssignNew(ReplicationContent, Replication::SReplicationRootWidget, InReplicationManager, InConcertSyncClient.ToSharedRef())
				]
			]
		];
	}

	void SActiveSessionRoot::OpenTab(EMultiUserTab Tab)
	{
		const int32 Index = static_cast<int32>(Tab);
		TabArea->SetButtonActivated(Index);
		TabSwitcher->SetActiveWidgetIndex(Index);
	}

	TSharedRef<SWidget> SActiveSessionRoot::CreateTabArea()
	{
		const auto CreateTabEntry = [this](int32 Index, const ANSICHAR* ImageBrush, FText Label, FText ToolTipText)
		{
			FTabEntry Entry;

			Entry.ButtonContent.Widget = SNew(SHorizontalBox)
				.ToolTipText(ToolTipText)
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FConcertFrontendStyle::Get()->GetBrush(ImageBrush))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2., 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Text(Label)
				];
			Entry.OnTabSelected.BindLambda([this, Index]()
			{
				TabSwitcher->SetActiveWidgetIndex(Index);
			});

			return Entry;
		};

		TArray<FTabEntry> Tabs
		{
			CreateTabEntry(0, "Concert.ActiveSession.Icon", LOCTEXT("OverviewTab.DisplayName", "Overview"), LOCTEXT("SessionOverviewTab.Tooltip", "Displays active session clients and activity.")),
			CreateTabEntry(1, "Concert.MultiUser", LOCTEXT("ReplicationTab.Label", "Replication"), LOCTEXT("ReplicationTab.Tooltip", "Manage real-time object replication"))
		};
		
		return SAssignNew(TabArea, STabArea)
			.Tabs(Tabs)
			.ActiveTabIndex(0);
	}
}

#undef LOCTEXT_NAMESPACE