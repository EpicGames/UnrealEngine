// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FSpawnTabArgs;
class FTabManager;
class FWorkspaceItem;
class IConcertSyncClient;
class SActiveSessionOverviewTab;
class SDockTab;
class SWidgetSwitcher;
class SWindow;
namespace UE::MultiUserClient::Replication
{
	class FMultiUserReplicationManager;
	class SReplicationRootWidget;
}

namespace UE::MultiUserClient
{
	class STabArea;

	enum class EMultiUserTab : uint8
	{
		Overview,
		Replication
	};
	
	/**
	 * Displayed when the client is connected to an active session.
	 * Manages the child content in inline tabs.
	 */
	class SActiveSessionRoot : public SCompoundWidget
	{
	public:
		
		static const FName SessionOverviewTabId;
		static const FName ReplicationTabId;

		SLATE_BEGIN_ARGS(SActiveSessionRoot)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IConcertSyncClient> InConcertSyncClient, TSharedRef<Replication::FMultiUserReplicationManager> InReplicationManager);

		const TSharedPtr<SActiveSessionOverviewTab>& GetOverviewContent() const { return OverviewContent; }
		const TSharedPtr<Replication::SReplicationRootWidget>& GetReplicationContent() const { return ReplicationContent; }

		/** Opens the specified inline tab. */
		void OpenTab(EMultiUserTab Tab);
		
	private:

		/** This switches "tabs" when a button in the "tab" area is changed. */
		TSharedPtr<SWidgetSwitcher> TabSwitcher;
		/** Holds the buttons for switching inline tabs. */
		TSharedPtr<STabArea> TabArea;

		/** Shows general stats, like transaction history and other clients. */
		TSharedPtr<SActiveSessionOverviewTab> OverviewContent;
		/** Shows content specific to replication. */
		TSharedPtr<Replication::SReplicationRootWidget> ReplicationContent;

		TSharedRef<SWidget> CreateTabArea();
	};
}


