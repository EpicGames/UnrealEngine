// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActiveSessionToolbar.h"

#include "ConcertClientFrontendUtils.h"
#include "IConcertSyncClient.h"
#include "IMultiUserClientModule.h"

#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

#define LOCTEXT_NAMESPACE "SActiveSessionToolbar"

namespace UE::MultiUserClient
{
	namespace Private
	{
		static TArray< TSharedPtr<SActiveSessionToolbar::FSendReceiveComboItem>> ConstructSendReceiveComboList()
		{
			TArray<TSharedPtr<SActiveSessionToolbar::FSendReceiveComboItem>> ComboList;
			ComboList.Add(MakeShared<SActiveSessionToolbar::FSendReceiveComboItem>(
					LOCTEXT("DefaultSendReceiveState", "Default"),
					LOCTEXT("DefaultSendReceiveStateTooltip", "Full send/receive mode for multi-user events."),
					EConcertSendReceiveState::Default)
				);
			ComboList.Add(MakeShared<SActiveSessionToolbar::FSendReceiveComboItem>(
					LOCTEXT("SendState", "Send only"),
					LOCTEXT("SendStateTooltip", "Transactions received from clients will be suspended; however local transactions will be sent."),
					EConcertSendReceiveState::SendOnly)
				);
			ComboList.Add(MakeShared<SActiveSessionToolbar::FSendReceiveComboItem>(
					LOCTEXT("ReceiveState", "Receive only"),
					LOCTEXT("ReceiveStateTooltip", "Local changes will be queued for transmission but not sent. Updates from clients will be received."),
					EConcertSendReceiveState::ReceiveOnly)
				);
			return ComboList;
		}
	}
	
	void SActiveSessionToolbar::Construct(const FArguments& InArgs, TSharedPtr<IConcertSyncClient> InConcertSyncClient)
	{
		WeakSessionPtr = InConcertSyncClient->GetConcertClient()->GetCurrentSession();
		SendReceiveComboList = Private::ConstructSendReceiveComboList();

		TSharedPtr<SHorizontalBox> Toolbar;
		ChildSlot
		[
			SAssignNew(Toolbar, SHorizontalBox)

			// Status Icon
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0f, 1.0f, 0.0f, 1.0f))
			[
				SNew(STextBlock)
				.Font(this, &SActiveSessionToolbar::GetConnectionIconFontInfo)
				.ColorAndOpacity(this, &SActiveSessionToolbar::GetConnectionIconColor)
				.Text(FEditorFontGlyphs::Circle)
			]

			// Status Message
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 1.0f))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				.ColorAndOpacity(FLinearColor(0.75f, 0.75f, 0.75f))
				.Padding(FMargin(0.0f, 4.0f, 6.0f, 4.0f))
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("BoldFont"))
					.Text(this, &SActiveSessionToolbar::GetConnectionStatusText)
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 3)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]

			// Tab area
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				InArgs._TabArea.Widget
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 3)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]

			// Move all the other content to the right if the panel is big
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SSpacer)
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(1.0f))
			[
				SNew(SBox)
				.MinDesiredWidth(130.0f)
				[
					SAssignNew(SendReceiveComboBox, SComboBox<TSharedPtr<FSendReceiveComboItem> >)
					.OptionsSource(&SendReceiveComboList)
					.InitiallySelectedItem(SendReceiveComboList[GetInitialSendReceiveComboIndex()])
					.ContentPadding(FMargin(4.0f,1.0f))
					.OnGenerateWidget(this, &SActiveSessionToolbar::GenerateSendReceiveComboItem)
					.OnSelectionChanged(this, &SActiveSessionToolbar::HandleSendReceiveChanged)
					[
						SNew(STextBlock)
						.Text(this, &SActiveSessionToolbar::GetRequestedSendReceiveComboText)
					]
				]
			]
			// The "Settings" icons.
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.OnClicked_Lambda([](){ FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer("Project", "Plugins", "Concert"); return FReply::Handled(); })
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Settings"))
				]
			]
		];
		
		// Append the buttons to the status bar
		{
			TArray<FConcertActionDefinition> ButtonDefs;

			// Leave Session
			FConcertActionDefinition& LeaveSessionDef = ButtonDefs.AddDefaulted_GetRef();
			LeaveSessionDef.Type = EConcertActionType::Normal;
			LeaveSessionDef.IsVisible = MakeAttributeSP(this, &SActiveSessionToolbar::IsStatusBarLeaveSessionVisible);
			LeaveSessionDef.Text = FEditorFontGlyphs::Sign_Out;
			LeaveSessionDef.ToolTipText = LOCTEXT("LeaveCurrentSessionToolTip", "Leave the current session");
			LeaveSessionDef.OnExecute.BindLambda([this]() { OnClickLeaveSession(); });
			LeaveSessionDef.IconStyle = TEXT("Concert.LeaveSession");

			ConcertClientFrontendUtils::AppendButtons(Toolbar.ToSharedRef(), ButtonDefs);
		}
	}

	FSlateFontInfo SActiveSessionToolbar::GetConnectionIconFontInfo() const
	{
		FSlateFontInfo ConnectionIconFontInfo = FAppStyle::Get().GetFontStyle(ConcertClientFrontendUtils::ButtonIconSyle);
		ConnectionIconFontInfo.OutlineSettings.OutlineSize = 1;
		ConnectionIconFontInfo.OutlineSettings.OutlineColor = GetConnectionIconStyle().Pressed.TintColor.GetSpecifiedColor();

		return ConnectionIconFontInfo;
	}

	FSlateColor SActiveSessionToolbar::GetConnectionIconColor() const
	{
		return GetConnectionIconStyle().Normal.TintColor;
	}

	const FButtonStyle& SActiveSessionToolbar::GetConnectionIconStyle() const
	{
		EConcertActionType ButtonStyle = EConcertActionType::Danger;

		TSharedPtr<IConcertClientSession> ClientSession = WeakSessionPtr.Pin();
		if (ClientSession.IsValid())
		{
			if (ClientSession->GetConnectionStatus() == EConcertConnectionStatus::Connected)
			{
				const bool bIsDefault = ClientSession->GetSendReceiveState() == EConcertSendReceiveState::Default;
				if (bIsDefault)
				{
					ButtonStyle = EConcertActionType::Success;
				}
				else
				{
					ButtonStyle = EConcertActionType::Warning;
				}
			}
		}

		return FAppStyle::Get().GetWidgetStyle<FButtonStyle>(ConcertClientFrontendUtils::ButtonStyleNames[static_cast<int32>(ButtonStyle)]);
	}

	FText SActiveSessionToolbar::GetConnectionStatusText() const
	{
		FText StatusText = LOCTEXT("StatusDisconnected", "Disconnected");
		TSharedPtr<IConcertClientSession> ClientSessionPtr = WeakSessionPtr.Pin();
		if (ClientSessionPtr.IsValid() && ClientSessionPtr->GetConnectionStatus() == EConcertConnectionStatus::Connected)
		{
			const FText SessionDisplayName = FText::FromString(ClientSessionPtr->GetSessionInfo().SessionName);

			switch (ClientSessionPtr->GetSendReceiveState())
			{
			case EConcertSendReceiveState::SendOnly:
				StatusText = FText::Format(LOCTEXT("StatusSendOnlyFmt", "Send Only: {0}"), SessionDisplayName);
				break;
			case EConcertSendReceiveState::ReceiveOnly:
				StatusText = FText::Format(LOCTEXT("StatusReceiveOnlyFmt", "Receive Only: {0}"), SessionDisplayName);
				break;
			case EConcertSendReceiveState::Default:
				StatusText = FText::Format(LOCTEXT("StatusConnectedFmt", "Connected: {0}"), SessionDisplayName);
				break;
			};
		}

		return StatusText;
	}

	int32 SActiveSessionToolbar::GetInitialSendReceiveComboIndex() const
	{
		if (!WeakSessionPtr.IsValid())
		{
			return 0;
		}

		TSharedPtr<IConcertClientSession> Session = WeakSessionPtr.Pin();
		switch(Session->GetSendReceiveState())
		{
		case EConcertSendReceiveState::Default:
			return 0;
		case EConcertSendReceiveState::SendOnly:
			return 1;
		case EConcertSendReceiveState::ReceiveOnly:
			return 2;
		}

		return 0;
	}

	TSharedRef<SWidget> SActiveSessionToolbar::GenerateSendReceiveComboItem(TSharedPtr<FSendReceiveComboItem> InItem) const
	{
		return SNew(STextBlock)
			.Text(InItem->Name)
			.ToolTipText(InItem->ToolTip);
	}

	void SActiveSessionToolbar::HandleSendReceiveChanged(TSharedPtr<FSendReceiveComboItem> Item, ESelectInfo::Type SelectInfo) const
	{
		check(Item.IsValid());
		if (TSharedPtr<IConcertClientSession> Session = WeakSessionPtr.Pin())
		{
			Session->SetSendReceiveState(Item->State);
		}
	}

	FText SActiveSessionToolbar::GetRequestedSendReceiveComboText() const
	{
		return SendReceiveComboBox->GetSelectedItem().IsValid()
			? SendReceiveComboBox->GetSelectedItem()->Name
			: LOCTEXT("ActiveSessionDefaultSendReceive", "Default");
	}

	bool SActiveSessionToolbar::IsStatusBarLeaveSessionVisible() const
	{
		const TSharedPtr<IConcertClientSession> ClientSession = WeakSessionPtr.Pin();
		return ClientSession && ClientSession->GetConnectionStatus() == EConcertConnectionStatus::Connected;
	}

	FReply SActiveSessionToolbar::OnClickLeaveSession() const
	{
		IMultiUserClientModule::Get().DisconnectSession();
		return FReply::Handled();
	}
}

#undef LOCTEXT_NAMESPACE