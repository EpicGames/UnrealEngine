// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessages.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class IConcertClientSession;
class IConcertSyncClient;
struct FButtonStyle;

namespace UE::MultiUserClient
{
	/**
	 * Displayed on the top of the window when in a session.
	 *
	 * Enables the client to leave the session, open the same level as another client or teleport to another
	 * client presence.
	 */
	class SActiveSessionToolbar : public SCompoundWidget
	{
	public:
		
		/** Struct to store the current send / receive state. */
		struct FSendReceiveComboItem
		{
			FSendReceiveComboItem(FText InName, FText InToolTip, EConcertSendReceiveState InState) :
				Name(MoveTemp(InName)), ToolTip(MoveTemp(InToolTip)), State(InState) {};

			FText Name;
			FText ToolTip;
			EConcertSendReceiveState State;
		};

		SLATE_BEGIN_ARGS(SActiveSessionToolbar)
		{}
			/** Contains the "tab" buttons. */
			SLATE_NAMED_SLOT(FArguments, TabArea)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IConcertSyncClient> InConcertSyncClient);

	private:
		
		/** Holds a concert client session. */
		TWeakPtr<IConcertClientSession> WeakSessionPtr;
		
		/** Sets whether this client sends or receives. */
		TSharedPtr<SComboBox<TSharedPtr<FSendReceiveComboItem>>> SendReceiveComboBox;
		
		/** Available states for the SendReceiveComboBox */
		TArray<TSharedPtr<FSendReceiveComboItem>> SendReceiveComboList;

		// Status icon and text
		FSlateFontInfo GetConnectionIconFontInfo() const;
		FSlateColor GetConnectionIconColor() const;
		const FButtonStyle& GetConnectionIconStyle() const;
		FText GetConnectionStatusText() const;

		// Default, Send Only & Receive Only combo box
		int32 GetInitialSendReceiveComboIndex() const;
		TSharedRef<SWidget> GenerateSendReceiveComboItem(TSharedPtr<FSendReceiveComboItem> InItem) const;
		void HandleSendReceiveChanged(TSharedPtr<FSendReceiveComboItem> Item, ESelectInfo::Type SelectInfo) const;
		FText GetRequestedSendReceiveComboText() const;
		
		// Handling for leave session button 
		bool IsStatusBarLeaveSessionVisible() const;
		FReply OnClickLeaveSession() const;

	};
}

