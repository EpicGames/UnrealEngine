// Copyright Epic Games, Inc. All Rights Reserved.

#include "STabArea.h"

#include "STabButton.h"

#include "Widgets/SBoxPanel.h"

namespace UE::MultiUserClient
{
	void STabArea::Construct(const FArguments& InArgs)
	{
		const TSharedRef<SHorizontalBox> Content = SNew(SHorizontalBox);
		for (const FTabEntry& Entry : InArgs._Tabs)
		{
			const int32 ButtonIndex = TabButtons.Num();
			
			TSharedPtr<STabButton> Button;
			Content->AddSlot()
				.Padding(InArgs._Padding)
				[
					SAssignNew(Button, STabButton)
					.OnActivated_Lambda([this, OnTabSelected = Entry.OnTabSelected, ButtonIndex]()
					{
						OnButtonActivated(ButtonIndex);
						OnTabSelected.ExecuteIfBound();
					})
					[
						Entry.ButtonContent.Widget
					]
				];

			TabButtons.Emplace(Button.ToSharedRef());
		}

		if (ensure(TabButtons.IsValidIndex(InArgs._ActiveTabIndex)))
		{
			TabButtons[InArgs._ActiveTabIndex]->Activate();
		}
		
		ChildSlot
		[
			Content
		];
	}

	void STabArea::SetButtonActivated(int32 ButtonIndex)
	{
		if (ensure(TabButtons.IsValidIndex(ButtonIndex)))
		{
			TabButtons[ButtonIndex]->Activate();
			OnButtonActivated(ButtonIndex);
		}
	}

	void STabArea::OnButtonActivated(int32 ButtonIndex)
	{
		for (int32 i = 0; i < TabButtons.Num(); ++i)
		{
			if (ButtonIndex != i)
			{
				TabButtons[i]->Deactivate();
			}
		}
	}
}
