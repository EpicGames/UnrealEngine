// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::MultiUserClient
{
	class STabButton;
}

namespace UE::MultiUserClient
{
	struct FTabEntry
	{
		TAlwaysValidWidget ButtonContent;
		FSimpleDelegate OnTabSelected;
	};
	
	/** Manages multiple STabButtons, making sure exactly 1 is active at a time. */
	class STabArea : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(STabArea)
			: _Padding(3.f)
			{}
			/** The tabs to create */
			SLATE_ARGUMENT(TArray<FTabEntry>, Tabs)
			/** The tab that should be active by default. */
			SLATE_ARGUMENT(int32, ActiveTabIndex)
			
			/** Padding between buttons */
			SLATE_ARGUMENT(FMargin, Padding)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/** Informs the owning view has manually changed to the content displayed by ButtonIndex. Make the button appear selected. */
		void SetButtonActivated(int32 ButtonIndex);

	private:

		/** The buttons in this area. */
		TArray<TSharedRef<STabButton>> TabButtons;

		/** Called when the button at TabButtons[ButtonIndex] is activated. */
		void OnButtonActivated(int32 ButtonIndex);
	};
}

