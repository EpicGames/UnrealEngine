// Copyright Epic Games, Inc. All Rights Reserved.

#include "STabButton.h"

#include "ConcertFrontendStyle.h"

#include "Widgets/Input/SCheckBox.h"

namespace UE::MultiUserClient
{
	void STabButton::Construct(const FArguments& InArgs)
	{
		OnActivatedDelegate = InArgs._OnActivated;
		
		ChildSlot
		[
			SNew(SCheckBox)
			.Style(FConcertFrontendStyle::Get(), TEXT("Concert.TabButton"))
			.IsChecked(this, &STabButton::IsChecked)
            .OnCheckStateChanged(this, &STabButton::OnButtonClicked)
			[
				InArgs._ButtonContent.Widget
			]
		];
	}

	ECheckBoxState STabButton::IsChecked() const
	{
		return bIsActive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void STabButton::OnButtonClicked(ECheckBoxState NewState)
	{
		if (!bIsActive)
		{
			bIsActive = true;
			OnActivatedDelegate.ExecuteIfBound();
		}
	}
}
