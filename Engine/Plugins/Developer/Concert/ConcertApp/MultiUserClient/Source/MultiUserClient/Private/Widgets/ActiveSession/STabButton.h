// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

enum class ECheckBoxState : uint8;

namespace UE::MultiUserClient
{
	/** Used to switch "tabs" in the active session UI. It is a blue button when active and gray when inactive. */
	class STabButton : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(STabButton){}
			/** Called when this button is clicked and becomes active. Not trigger if it was already active. */
			SLATE_EVENT(FSimpleDelegate, OnActivated)
			
			SLATE_DEFAULT_SLOT(FArguments, ButtonContent)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/** Makes the button appear active. */
		void Activate() { bIsActive = true; }
		/** Makes the button appear inactive. */
		void Deactivate() { bIsActive = false; }
		
	private:

		/** Whether the button is active (the tab content is supposed to be shown). */
		bool bIsActive = false;

		/** Called when this button is clicked and becomes active. Not trigger if it was already active. */
		FSimpleDelegate OnActivatedDelegate;
		
		ECheckBoxState IsChecked() const;
		void OnButtonClicked(ECheckBoxState NewState);
	};
}

