// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientSessionHistoryController.h"

#include "ConcertSyncSessionTypes.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"

#include "Session/History/SSessionHistory.h"

FClientSessionHistoryController::FClientSessionHistoryController(TSharedRef<IConcertSyncClient> Client, FName PackageFilter)
	: FAbstractSessionHistoryController(SSessionHistory::FArguments().PackageFilter(PackageFilter))
	, Client(MoveTemp(Client))
{
	SubscribeToWorkspaceEvents();
}

FClientSessionHistoryController::~FClientSessionHistoryController()
{
	if (const TSharedPtr<IConcertSyncClient> ClientPtr = Client.Pin())
	{
		ClientPtr->OnWorkspaceStartup().RemoveAll(this);
		ClientPtr->OnWorkspaceShutdown().RemoveAll(this);
	}
	
	if (const TSharedPtr<IConcertClientWorkspace> WorkspacePtr = Workspace.Pin())
	{
		WorkspacePtr->OnActivityAddedOrUpdated().RemoveAll(this);
		WorkspacePtr->OnWorkspaceSynchronized().RemoveAll(this);
	}
}

void FClientSessionHistoryController::GetActivities(int64 MaximumNumberOfActivities, TMap<FGuid, FConcertClientInfo>& EndpointClientInfoMap, TArray<FConcertSessionActivity>& FetchedActivities) const
{
	if (const TSharedPtr<IConcertClientWorkspace> WorkspacePtr = Workspace.Pin())
	{
		const int64 LastActivityId = WorkspacePtr->GetLastActivityId();
		const int64 FirstActivityIdToFetch = FMath::Max<int64>(1, LastActivityId - MaximumNumberOfActivities);
		WorkspacePtr->GetActivities(FirstActivityIdToFetch, MaximumNumberOfActivities, EndpointClientInfoMap, FetchedActivities);
	}
}

bool FClientSessionHistoryController::GetPackageEvent(const FConcertSessionActivity& Activity, FConcertSyncPackageEventMetaData& OutPackageEvent) const
{
	if (TSharedPtr<IConcertClientWorkspace> WorkspacePtr = Workspace.Pin())
	{
		// Don't request the package data, the widget only display the meta-data.
		return WorkspacePtr->FindPackageEvent(Activity.Activity.EventId, OutPackageEvent);
	}

	return false; // The data is not available.
}

TFuture<TOptional<FConcertSyncTransactionEvent>> FClientSessionHistoryController::GetTransactionEvent(const FConcertSessionActivity& Activity) const
{
	if (TSharedPtr<IConcertClientWorkspace> WorkspacePtr = Workspace.Pin())
	{
		// Ask to get the full transaction to display which properties changed.
		return WorkspacePtr->FindOrRequestTransactionEvent(Activity.Activity.EventId, /*bMetaDataOnly*/false);
	}

	return MakeFulfilledPromise<TOptional<FConcertSyncTransactionEvent>>().GetFuture();
}

void FClientSessionHistoryController::SubscribeToWorkspaceEvents()
{
	TSharedPtr<IConcertSyncClient> ClientPtr = Client.Pin();
	checkf(ClientPtr, TEXT("We're supposed to be in the constructor where the Client should still be valid!"));
	
	// Must use AddRaw instead of AddSP because we're called from constructor (before TSharedFromThis is initialised)
	ClientPtr->OnWorkspaceStartup().AddRaw(this, &FClientSessionHistoryController::HandleWorkspaceStartup);
	ClientPtr->OnWorkspaceShutdown().AddRaw(this, &FClientSessionHistoryController::HandleWorkspaceShutdown);

	if (TSharedPtr<IConcertClientWorkspace> WorkspacePtr = ClientPtr->GetWorkspace())
	{
		Workspace = WorkspacePtr;
		RegisterWorkspaceHandler();
		ReloadActivities();
	}
}

void FClientSessionHistoryController::RegisterWorkspaceHandler()
{
	TSharedPtr<IConcertClientWorkspace> WorkspacePtr = Workspace.Pin();
	if (WorkspacePtr.IsValid())
	{
		// Must use AddRaw instead of AddSP because we may be called from constructor (before TSharedFromThis is initialised)
		WorkspacePtr->OnActivityAddedOrUpdated().AddRaw(this, &FClientSessionHistoryController::HandleActivityAddedOrUpdated);
		WorkspacePtr->OnWorkspaceSynchronized().AddRaw(this, &FClientSessionHistoryController::ReloadActivities);
	}
}
void FClientSessionHistoryController::HandleWorkspaceStartup(const TSharedPtr<IConcertClientWorkspace>& NewWorkspace)
{
	Workspace = NewWorkspace;
	RegisterWorkspaceHandler();
}

void FClientSessionHistoryController::HandleWorkspaceShutdown(const TSharedPtr<IConcertClientWorkspace>& WorkspaceShuttingDown)
{
	if (WorkspaceShuttingDown == Workspace)
	{
		Workspace.Reset();
		ReloadActivities();
	}
}

void FClientSessionHistoryController::HandleActivityAddedOrUpdated(const FConcertClientInfo& InClientInfo, const FConcertSyncActivity& InActivity, const FStructOnScope& InActivitySummary)
{
	GetSessionHistory()->HandleActivityAddedOrUpdated(InClientInfo, InActivity, InActivitySummary);
}
