// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Session/History/AbstractSessionHistoryController.h"

struct FConcertSyncActivity;
struct FConcertClientInfo;
class IConcertClientWorkspace;
class IConcertSyncClient;
class FStructOnScope;

/** Manages SSessionHistory for IConcertClients */
class FClientSessionHistoryController : public FAbstractSessionHistoryController
{
public:

	FClientSessionHistoryController(TSharedRef<IConcertSyncClient> Client, FName PackageFilter = EName::None);
	virtual ~FClientSessionHistoryController() override;

protected:

	//~ Begin FAbstractSessionHistoryController Interface
	virtual void GetActivities(int64 MaximumNumberOfActivities, TMap<FGuid, FConcertClientInfo>& EndpointClientInfoMap, TArray<FConcertSessionActivity>& FetchedActivities) const override;
	virtual bool GetPackageEvent(const FConcertSessionActivity& Activity, FConcertSyncPackageEventMetaData& OutPackageEvent) const override;
	virtual TFuture<TOptional<FConcertSyncTransactionEvent>> GetTransactionEvent( const FConcertSessionActivity& Activity) const override;
	//~ End FAbstractSessionHistoryController Interface
	
private:

	/** So we can unsubscribe from events when destroyed */
	TWeakPtr<IConcertSyncClient> Client;
	
	/** Holds a weak pointer to the current workspace. */ 
	TWeakPtr<IConcertClientWorkspace> Workspace;
	
	void SubscribeToWorkspaceEvents();
	
	/** Registers callbacks with the current workspace. */
	void RegisterWorkspaceHandler();

	// Workspace events
	void HandleWorkspaceStartup(const TSharedPtr<IConcertClientWorkspace>& NewWorkspace);
	void HandleWorkspaceShutdown(const TSharedPtr<IConcertClientWorkspace>& WorkspaceShuttingDown);
	void HandleActivityAddedOrUpdated(const FConcertClientInfo& InClientInfo, const FConcertSyncActivity& InActivity, const FStructOnScope& InActivitySummary);
};
