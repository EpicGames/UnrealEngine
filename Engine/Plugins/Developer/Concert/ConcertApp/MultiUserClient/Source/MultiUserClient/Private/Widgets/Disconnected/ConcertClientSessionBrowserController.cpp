// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientSessionBrowserController.h"

#include "ConcertActivityStream.h"
#include "ConcertLogGlobal.h"
#include "IMultiUserClientModule.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "SConcertSessionRecovery.h"

#include "Algo/Transform.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/AsyncTaskNotification.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "FConcertClientSessionBrowserController"

FConcertClientSessionBrowserController::FConcertClientSessionBrowserController(IConcertClientPtr InConcertClient)
{
	check(InConcertClient.IsValid()); // Don't expect this class to be instantiated if the concert client is not available.
	check(InConcertClient->IsConfigured()); // Expected to be done by higher level code.
	ConcertClient = InConcertClient;

	// When others servers are running, add them to the list if you want to test the UI displayed if no servers/no sessions exists.
	//IgnoredServers.Add(TEXT("wksyul10355")); // TODO: COMMENT BEFORE SUBMIT

	// Start server discovery to find the available Concert servers.
	ConcertClient->StartDiscovery();

	// Populate the session cache.
	TickServersAndSessionsDiscovery();
}

FConcertClientSessionBrowserController::~FConcertClientSessionBrowserController()
{
	if (ConcertClient.IsValid())
	{
		ConcertClient->StopDiscovery();
	}
}

const TArray<FConcertSessionClientInfo>& FConcertClientSessionBrowserController::GetClients(const FGuid& AdminEndpoint, const FGuid& SessionId) const
{
	// If a session clients are monitored and the list is cached
	if (ClientMonitoredSession.IsSet() && ClientMonitoredSession->Session->ServerInfo.AdminEndpointId == AdminEndpoint && ClientMonitoredSession->Session->SessionInfo.SessionId == SessionId)
	{
		return ClientMonitoredSession->Session->Clients; // Returns the list retrieved by the last TickClientsDiscovery().
	}

	// Returns an empty list for now. We expect the caller to call TickClientsDiscovery() periodically to maintain the session client list.
	static TArray<FConcertSessionClientInfo> EmptyClientList;
	return EmptyClientList;
}

void FConcertClientSessionBrowserController::JoinSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId)
{
	// On success: The client joins the session and SConcertBrowser::HandleSessionConnectionChanged() will transit the UI to the SActiveSession.
	// On failure: An async notification banner will be displayer to the user.
	ConcertClient->JoinSession(ServerAdminEndpointId, SessionId);
}

TOptional<FConcertSessionInfo> FConcertClientSessionBrowserController::GetActiveSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const
{
	const TSharedPtr<FClientActiveSessionInfo>* SessionInfo = ActiveSessions.FindByPredicate([&AdminEndpoint, &SessionId](const TSharedPtr<FActiveSessionInfo>& MatchCandidate)
	{
		return MatchCandidate->ServerInfo.AdminEndpointId == AdminEndpoint && MatchCandidate->SessionInfo.SessionId == SessionId;
	});

	return SessionInfo != nullptr ? (*SessionInfo)->SessionInfo : TOptional<FConcertSessionInfo>{};
}

TOptional<FConcertSessionInfo> FConcertClientSessionBrowserController::GetArchivedSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const
{
	const TSharedPtr<FClientArchivedSessionInfo>* SessionInfo = ArchivedSessions.FindByPredicate([&AdminEndpoint, &SessionId](const TSharedPtr<FArchivedSessionInfo>& MatchCandidate)
	{
		return MatchCandidate->ServerInfo.AdminEndpointId == AdminEndpoint && MatchCandidate->SessionInfo.SessionId == SessionId;
	});

	return SessionInfo != nullptr ? (*SessionInfo)->SessionInfo : TOptional<FConcertSessionInfo>{};
}

void FConcertClientSessionBrowserController::CreateSession(const FGuid& ServerAdminEndpointId, const FString& SessionName, const FString& ProjectName)
{
	FAsyncRequest& CreateRequest = CreateSessionRequests.AddDefaulted_GetRef();
	TWeakPtr<uint8, ESPMode::ThreadSafe> CreateRequestExecutionToken = CreateRequest.ResetExecutionToken();

	// On success, the client automatically joins the new session and SConcertBrowser::HandleSessionConnectionChanged() will transit the UI to the SActiveSession.
	// On failure: An async notification banner will be displayer to the user.
	FConcertCreateSessionArgs CreateSessionArgs;
	CreateSessionArgs.SessionName = SessionName;
	// Project Name is not used an override for the client session create.
	ConcertClient->CreateSession(ServerAdminEndpointId, CreateSessionArgs).Next([this, CreateRequestExecutionToken, ServerAdminEndpointId, SessionName](EConcertResponseCode ResponseCode)
	{
		if (TSharedPtr<uint8, ESPMode::ThreadSafe> ExecutionToken = CreateRequestExecutionToken.Pin())
		{
			if (ResponseCode == EConcertResponseCode::Success)
			{
				// Expect to find those session at some point in the future.
				ExpectedSessionsToDiscover.Add(FPendingSessionDiscovery{ FDateTime::UtcNow(), ServerAdminEndpointId, SessionName });
			}

			// Stop tracking the request.
			CreateSessionRequests.RemoveAll([&ExecutionToken](const FAsyncRequest& DiscardCandidate) { return DiscardCandidate.FutureExecutionToken == ExecutionToken; });
		}
	});
}

void FConcertClientSessionBrowserController::ArchiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& ArchiveName, const FConcertSessionFilter& SessionFilter)
{
	// On success, an archived is created and TickServersAndSessionsDiscovery() will eventually discover it.
	// On failure: An async notification banner will be displayer to the user.
	FConcertArchiveSessionArgs ArchiveSessionArgs;
	ArchiveSessionArgs.SessionId = SessionId;
	ArchiveSessionArgs.ArchiveNameOverride = ArchiveName;
	ArchiveSessionArgs.SessionFilter = SessionFilter;
	ConcertClient->ArchiveSession(ServerAdminEndpointId, ArchiveSessionArgs);
}

void FConcertClientSessionBrowserController::RestoreSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& RestoredName, const FConcertSessionFilter& SessionFilter)
{
	FString ArchivedSessionName;
	if (const TOptional<FConcertSessionInfo> SessionInfo = GetArchivedSessionInfo(ServerAdminEndpointId, SessionId))
	{
		ArchivedSessionName = SessionInfo->SessionName;
	}

	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(FText::Format(LOCTEXT("RestoreSessionDialogTitle", "Restoring {0}"), FText::AsCultureInvariant(ArchivedSessionName)))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(1200, 800))
		.IsTopmostWindow(false) // Consider making it always on top?
		.SupportsMaximize(true)
		.SupportsMinimize(false);

	// Ask the stream to pull the activity details (for transaction/package) for inspection.
	constexpr bool bRequestActivityDetails = true;

	// Create a stream of activities (streaming from the most recent to the oldest).
	TSharedPtr<FConcertActivityStream> ActivityStream = MakeShared<FConcertActivityStream>(ConcertClient, ServerAdminEndpointId, SessionId, bRequestActivityDetails);

	// The UI uses this function to read and consume the activity stream.
	auto ReadActivitiesFn = [ActivityStream](TArray<TSharedPtr<FConcertSessionActivity>>& InOutActivities, int32& OutFetchCount, FText& OutErrorMsg)
	{
		return ActivityStream->Read(InOutActivities, OutFetchCount, OutErrorMsg);
	};

	// The UI uses this function to map an activity ID from the stream to a client info.
	auto GetActivityClientInfoFn = [ActivityStream](FGuid EndpointID)
	{
		const FConcertClientInfo* Result = ActivityStream->GetActivityClientInfo(EndpointID);
		return Result ? TOptional<FConcertClientInfo>{ *Result } : TOptional<FConcertClientInfo>{};
	};

	// Invoked if the client selects a point in time to recover.
	TWeakPtr<IConcertClient, ESPMode::ThreadSafe> WeakClient = ConcertClient;
	auto OnAcceptRestoreFn = [WeakClient, ServerAdminEndpointId, SessionId, RestoredName, SessionFilter](TSharedPtr<FConcertSessionActivity> SelectedRecoveryActivity)
	{
		FConcertCopySessionArgs RestoreSessionArgs;
		RestoreSessionArgs.bAutoConnect = true;
		RestoreSessionArgs.SessionId = SessionId;
		RestoreSessionArgs.SessionName = RestoredName;
		RestoreSessionArgs.SessionFilter = SessionFilter;
		RestoreSessionArgs.SessionFilter.bOnlyLiveData = false;

		// Set which item was selected to recover through.
		if (SelectedRecoveryActivity)
		{
			RestoreSessionArgs.SessionFilter.ActivityIdUpperBound = SelectedRecoveryActivity->Activity.ActivityId;
		}
		// else -> Restore the entire session as it.

		bool bDismissRecoveryWindow = true; // Dismiss the window showing the session activities

		if (TSharedPtr<IConcertClient, ESPMode::ThreadSafe> ConcertClientPin = WeakClient.Pin())
		{
			// Prompt the user to persist and leave the session.
			bool bDisconnected = IMultiUserClientModule::Get().DisconnectSession(/*bAlwaysAskConfirmation*/true);
			if (bDisconnected)
			{
				// On success, a new session is created from the archive, the client automatically disconnects from the current session (if any), joins the restored one and SConcertBrowser::HandleSessionConnectionChanged() will transit the UI to the SActiveSession.
				// On failure: An async notification banner will be displayer to the user.
				ConcertClientPin->RestoreSession(ServerAdminEndpointId, RestoreSessionArgs);
			}
			else // The user declined disconnection.
			{
				bDismissRecoveryWindow = false; // Keep the window open, let the client handle it (close/cancel or just restore later).
			}
		}
		else
		{
			FAsyncTaskNotificationConfig NotificationConfig;
			NotificationConfig.bIsHeadless = false;
			NotificationConfig.bKeepOpenOnFailure = true;
			NotificationConfig.LogCategory = &LogConcert;

			FAsyncTaskNotification Notification(NotificationConfig);
			Notification.SetComplete(LOCTEXT("RecoveryError", "Failed to recover the session"), LOCTEXT("ClientUnavailable", "Concert client unavailable"), /*Success*/ false);
		}
		return bDismissRecoveryWindow;
	};

	TSharedRef<SConcertSessionRecovery> RestoreWidget = SNew(SConcertSessionRecovery)
		.ParentWindow(NewWindow)
		.IntroductionText(LOCTEXT("RecoverSessionIntroductionText", "Select the point in time at which the session should be restored"))
		.OnFetchActivities_Lambda(ReadActivitiesFn)
		.OnMapActivityToClient_Lambda(GetActivityClientInfoFn)
		.OnRestore(OnAcceptRestoreFn)
		.WithClientAvatarColorColumn(true)
		.WithClientNameColumn(true)
		.WithOperationColumn(true)
		.WithPackageColumn(false) // Even tough the column is not present, the tooltips and summary contains the affected package.
		.DetailsAreaVisibility(bRequestActivityDetails ? EVisibility::Visible : EVisibility::Collapsed) // The activity stream was configured to pull the activity details.
		.IsConnectionActivityFilteringEnabled(true)
		.IsLockActivityFilteringEnabled(true);

	NewWindow->SetContent(RestoreWidget);
	FSlateApplication::Get().AddWindow(NewWindow, true);
}

void FConcertClientSessionBrowserController::RenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName)
{
	// Find the currently cached session info.
	const TSharedPtr<FClientActiveSessionInfo>* SessionInfo = ActiveSessions.FindByPredicate([&ServerAdminEndpointId, &SessionId](const TSharedPtr<FActiveSessionInfo>& MatchCandidate)
	{
		return MatchCandidate->ServerInfo.AdminEndpointId == ServerAdminEndpointId && MatchCandidate->SessionInfo.SessionId == SessionId;
	});

	check(SessionInfo); // If the UI is displaying it, the UI backend should have it.
	ConcertClient->RenameSession(ServerAdminEndpointId, SessionId, NewName).Next([ActiveSessionInfo = *SessionInfo](EConcertResponseCode Response)
	{
		if (Response != EConcertResponseCode::Success)
		{
			ActiveSessionInfo->bSessionNameDirty = true; // Renamed failed, the UI may be displaying the wrong name. Force the UI to update against the latest cached values.
		}
		// else -> Succeeded -> TickServersAndSessionsDiscovery() will pick up the new name.
	});
}

void FConcertClientSessionBrowserController::RenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName)
{
	// Find the currently cached session info.
	const TSharedPtr<FClientArchivedSessionInfo>* SessionInfo = ArchivedSessions.FindByPredicate([&ServerAdminEndpointId, &SessionId](const TSharedPtr<FArchivedSessionInfo>& MatchCandidate)
	{
		return MatchCandidate->ServerInfo.AdminEndpointId == ServerAdminEndpointId && MatchCandidate->SessionInfo.SessionId == SessionId;
	});

	check(SessionInfo); // If the UI is displaying it, the UI backend should have it.
	ConcertClient->RenameSession(ServerAdminEndpointId, SessionId, NewName).Next([SessionInfo = *SessionInfo](EConcertResponseCode Response)
	{
		if (Response != EConcertResponseCode::Success)
		{
			SessionInfo->bSessionNameDirty = true; // Renamed failed, the UI may be displaying the wrong name. Force the UI to update against the latest cached values.
		}
		// else -> Succeeded -> TickServersAndSessionsDiscovery() will pick up the new name.
	});
}

bool FConcertClientSessionBrowserController::CanRenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const
{
	return CanDeleteActiveSession(ServerAdminEndpointId, SessionId); // Rename requires the same permission than delete.
}

bool FConcertClientSessionBrowserController::CanRenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const
{
	return CanDeleteArchivedSession(ServerAdminEndpointId, SessionId); // Rename requires the same permission than delete.
}

void FConcertClientSessionBrowserController::DeleteSessions(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& SessionIds)
{
	// The difference between the two functions is how error notifications are displayed...
	if (SessionIds.Num() == 1)
	{
		// ... shows a specific notification
		ConcertClient->DeleteSession(ServerAdminEndpointId, SessionIds[0]);
	}
	else
	{
		TSet<FGuid> SessionIdSet;
		Algo::Transform(SessionIds, SessionIdSet, [](const FGuid& SessionId) { return SessionId; });
		// ... shows compressed notification(s)
		ConcertClient->BatchDeleteSessions(ServerAdminEndpointId, { SessionIdSet, EBatchSessionDeletionFlags::SkipForbiddenSessions });
	}
}

bool FConcertClientSessionBrowserController::CanDeleteActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const
{
	const TSharedPtr<FClientActiveSessionInfo>* SessionInfo = ActiveSessions.FindByPredicate([&ServerAdminEndpointId, &SessionId](const TSharedPtr<FActiveSessionInfo>& MatchCandidate)
	{
		return MatchCandidate->ServerInfo.AdminEndpointId == ServerAdminEndpointId && MatchCandidate->SessionInfo.SessionId == SessionId;
	});

	// Can delete the session only if the concert client is the owner.
	return SessionInfo == nullptr ? false : ConcertClient->IsOwnerOf((*SessionInfo)->SessionInfo);
}

bool FConcertClientSessionBrowserController::CanDeleteArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const
{
	const TSharedPtr<FClientArchivedSessionInfo>* SessionInfo = ArchivedSessions.FindByPredicate([&ServerAdminEndpointId, &SessionId](const TSharedPtr<FArchivedSessionInfo>& MatchCandidate)
	{
		return MatchCandidate->ServerInfo.AdminEndpointId == ServerAdminEndpointId && MatchCandidate->SessionInfo.SessionId == SessionId;
	});

	// Can delete the session only if the concert client is the owner.
	return SessionInfo == nullptr ? false : ConcertClient->IsOwnerOf((*SessionInfo)->SessionInfo);
}

TPair<uint32, uint32> FConcertClientSessionBrowserController::TickServersAndSessionsDiscovery()
{
	// Fire new async queries to poll the sessions on all known servers.
	UpdateSessionsAsync();

	// Returns the versions corresponding to the last async responses received. The version are incremented when response to the asynchronous queries are received.
	return MakeTuple(ServerListVersion, SessionListVersion);
}

uint32 FConcertClientSessionBrowserController::TickClientsDiscovery(const FGuid& ServerAdminEndpointId, const FGuid& SessionId)
{
	// Fire new async queries to poll clients for the selected server/session pair.
	UpdateClientsAsync(ServerAdminEndpointId, SessionId);

	// Returns the versions corresponding to the last async responses received. The version is incremented when response to the asynchronous query is received.
	return ClientListVersion;
}

bool FConcertClientSessionBrowserController::GetAndClearDiscoveryUpdateFlag()
{
	bool bOldCacheChanged = bCacheUpdated; // This flag is raised every time an async response updates the cached data.
	bCacheUpdated = false;
	return bOldCacheChanged;
}

void FConcertClientSessionBrowserController::UpdateSessionsAsync()
{
	// Get the list of known servers.
	TArray<FConcertServerInfo> OnlineServers = ConcertClient->GetKnownServers();
	if (IgnoredServers.Num())
	{
		OnlineServers.RemoveAll([this](const FConcertServerInfo& ServerInfo) { return IgnoredServers.Contains(ServerInfo.ServerName); });
	}

	bool bServerListVersionUpdated = false;

	// Detects which server(s) went offline since the last update.
	for (const FConcertServerInfo& ServerInfo : Servers)
	{
		// If a server, previously tracked, is not online anymore, remove all its sessions.
		if (!OnlineServers.ContainsByPredicate([this, &ServerInfo](const FConcertServerInfo& Visited) { return ServerInfo.InstanceInfo.InstanceId == Visited.InstanceInfo.InstanceId; }))
		{
			// Remove all active sessions corresponding to this server.
			ActiveSessions.RemoveAll([this, &ServerInfo](const TSharedPtr<FActiveSessionInfo>& ActiveSessionInfo)
			{
				if (ServerInfo.InstanceInfo.InstanceId == ActiveSessionInfo->ServerInfo.InstanceInfo.InstanceId)
				{
					OnActiveSessionDiscarded(*ActiveSessionInfo);
					return true;
				}
				return false;
			});

			// Remove all archived sessions corresponding to this server.
			ArchivedSessions.RemoveAll([this, &ServerInfo](const TSharedPtr<FArchivedSessionInfo>& ArchivedSessionInfo)
			{
				if (ServerInfo.InstanceInfo.InstanceId == ArchivedSessionInfo->ServerInfo.InstanceInfo.InstanceId)
				{
					OnArchivedSessionDiscarded(*ArchivedSessionInfo);
					return true;
				}
				return false;
			});

			// Disarm any active async request future for this server. Removing it from the map effectively disarm the future as the shared pointer used to as token gets released.
			ActiveSessionRequests.Remove(ServerInfo.InstanceInfo.InstanceId);
			ArchivedSessionRequests.Remove(ServerInfo.InstanceInfo.InstanceId);

			// The server went offline, update the server list version.
			ServerListVersion++;
			bServerListVersionUpdated = true;
		}
	}

	// For all online servers.
	for (const FConcertServerInfo& ServerInfo : OnlineServers)
	{
		// Check if this is a new server.
		if (!bServerListVersionUpdated && !Servers.ContainsByPredicate([&ServerInfo](const FConcertServerInfo& Visited) { return ServerInfo.InstanceInfo.InstanceId == Visited.InstanceInfo.InstanceId; }))
		{
			ServerListVersion++;
			bServerListVersionUpdated = true; // No need to look up further, only one new server is required to update the version.
		}

		// Poll the sessions for this online server.
		UpdateActiveSessionsAsync(ServerInfo);
		UpdateArchivedSessionsAsync(ServerInfo);
	}

	// Keep the list of online servers.
	Servers = MoveTemp(OnlineServers);
}

void FConcertClientSessionBrowserController::UpdateActiveSessionsAsync(const FConcertServerInfo& ServerInfo)
{
	// Check if a request is already pulling active session from this server.
	FAsyncRequest& ListActiveSessionAsyncRequest = ActiveSessionRequests.FindOrAdd(ServerInfo.InstanceInfo.InstanceId);
	if (ListActiveSessionAsyncRequest.IsOngoing())
	{
		// Don't stack another async request on top of the ongoing one, TickServersAndSessionsDiscovery() will eventually catch up any possibly missed info here.
		return;
	}

	// Arm the future, enabling its continuation to execute (or not).
	TWeakPtr<uint8, ESPMode::ThreadSafe> ListActiveSessionExecutionToken = ListActiveSessionAsyncRequest.ResetExecutionToken();

	// Keep track of the request time.
	FDateTime ListRequestTimestamp = FDateTime::UtcNow();

	// Retrieve all live sessions currently known by this server.
	ListActiveSessionAsyncRequest.Future = ConcertClient->GetLiveSessions(ServerInfo.AdminEndpointId)
		.Next([this, ServerInfo, ListActiveSessionExecutionToken, ListRequestTimestamp](const FConcertAdmin_GetSessionsResponse& Response)
		{
			// If the future is disarmed.
			if (!ListActiveSessionExecutionToken.IsValid())
			{
				// Don't go further, the future execution was canceled, maybe because this object was deleted, the server removed from the list or it wasn't safe to get this future executing.
				return;
			}

			// If the server responded.
			if (Response.ResponseCode == EConcertResponseCode::Success)
			{
				bInitialActiveSessionQueryResponded = true;

				// Remove from the cache any session that were deleted from 'Server' since the last update. Find which one were renamed.
				ActiveSessions.RemoveAll([this, &ServerInfo, &Response](const TSharedPtr<FClientActiveSessionInfo>& DiscardCandidate)
				{
					// If the candidate is owned by another server.
					if (DiscardCandidate->ServerInfo.InstanceInfo.InstanceId != ServerInfo.InstanceInfo.InstanceId)
					{
						return false; // Keep that session, it's owned by another server.
					}
					// If the candidate is still active.
					else if (const FConcertSessionInfo* SessionFromServer = Response.Sessions.FindByPredicate([DiscardCandidate](const FConcertSessionInfo& MatchCandidate) { return DiscardCandidate->SessionInfo.SessionId == MatchCandidate.SessionId; }))
					{
						// Don't discard the session, it is still active, but check if it was renamed.
						if (SessionFromServer->SessionName != DiscardCandidate->SessionInfo.SessionName)
						{
							DiscardCandidate->SessionInfo.SessionName = SessionFromServer->SessionName; // Update the session name.
							OnActiveSessionRenamed(*DiscardCandidate, SessionFromServer->SessionName);
						}
						else if (DiscardCandidate->bSessionNameDirty) // Renaming this particular session failed?
						{
							OnActiveSessionListDirty(); // UI optimistically updated a session name from a rename, but it failed on the server. The controller caches the latest official server list unmodified. Force the client to resync its list against the cache.
							DiscardCandidate->bSessionNameDirty = false;
						}
						return false; // Don't remove the session, it's still active.
					}

					// The session is not active anymore on the server, it was discarded.
					OnActiveSessionDiscarded(*DiscardCandidate);
					return true;
				});

				// Add to the cache any session that was added to 'Server' since the last update.
				for (const FConcertSessionInfo& SessionInfo : Response.Sessions)
				{
					// Try to find the session in the list of active session.
					if (!ActiveSessions.ContainsByPredicate([&SessionInfo](const TSharedPtr<FClientActiveSessionInfo>& MatchCandidate)
						{ return SessionInfo.ServerInstanceId == MatchCandidate->ServerInfo.InstanceInfo.InstanceId && SessionInfo.SessionId == MatchCandidate->SessionInfo.SessionId; }))
					{
						// This is a newly discovered session, add it to the list.
						ActiveSessions.Add(MakeShared<FClientActiveSessionInfo>(ServerInfo, SessionInfo));
						OnActiveSessionDiscovered(*ActiveSessions.Last());
					}

					// Remove sessions that were created and discovered or those that should have shown up by now, but have not.
					ExpectedSessionsToDiscover.RemoveAll([&SessionInfo, &ListRequestTimestamp](const FPendingSessionDiscovery& DiscardCandidate)
					{
						bool bDiscovered = DiscardCandidate.ServerEndpoint == SessionInfo.ServerEndpointId && DiscardCandidate.SessionName == SessionInfo.SessionName; // Session was discovered as expected.
						bool bDeleted = ListRequestTimestamp > DiscardCandidate.CreateTimestamp; // The session was created successfully, a list request posted, but the session did not turn up. The session was deleted before it could be listed.
						return bDiscovered || bDeleted;
					});
				}
			}
			// else -> The concert request failed, possibly because the server went offline. Wait until next TickServersAndSessionsDiscovery() to sync the server again and discover if it went offline.
		});
}

void FConcertClientSessionBrowserController::UpdateArchivedSessionsAsync(const FConcertServerInfo& ServerInfo)
{
	// Check if a request is already pulling archived sessions from this server.
	FAsyncRequest& ListArchivedSessionAsyncRequest = ArchivedSessionRequests.FindOrAdd(ServerInfo.InstanceInfo.InstanceId);
	if (ListArchivedSessionAsyncRequest.IsOngoing())
	{
		// Don't stack another async request on top of the ongoing one, TickServersAndSessionsDiscovery() will eventually catch up any possibly missed info here.
		return;
	}

	// Arm the future, enabling its continuation to execute.
	TWeakPtr<uint8, ESPMode::ThreadSafe> ListArchivedSessionExecutionToken = ListArchivedSessionAsyncRequest.ResetExecutionToken();

	// Retrieve the archived sessions.
	ListArchivedSessionAsyncRequest.Future = ConcertClient->GetArchivedSessions(ServerInfo.AdminEndpointId)
		.Next([this, ServerInfo, ListArchivedSessionExecutionToken](const FConcertAdmin_GetSessionsResponse& Response)
		{
			// If the future is disarmed.
			if (!ListArchivedSessionExecutionToken.IsValid())
			{
				// Don't go further, the future execution was canceled, maybe because this object was deleted, the server removed from the list or it wasn't safe to get this future executing.
				return;
			}

			// If the server responded.
			if (Response.ResponseCode == EConcertResponseCode::Success)
			{
				bInitialArchivedSessionQueryResponded = true;

				// Remove from the cache archives that were deleted from 'Server' since the last update. Find which one were renamed.
				ArchivedSessions.RemoveAll([this, &ServerInfo, &Response](const TSharedPtr<FClientArchivedSessionInfo>& DiscardCandidate)
				{
					// If the discard candidate is stored on another server.
					if (DiscardCandidate->ServerInfo.InstanceInfo.InstanceId != ServerInfo.InstanceInfo.InstanceId)
					{
						return false; // Keep that archive, it's stored on another server.
					}
					// If the archive is still stored on the server.
					else if (const FConcertSessionInfo* SessionFromServer = Response.Sessions.FindByPredicate([DiscardCandidate](const FConcertSessionInfo& MatchCandidate) { return DiscardCandidate->SessionInfo.SessionId == MatchCandidate.SessionId; }))
					{
						// Don't discard the session, it is still there. Check if it was renamed.
						if (SessionFromServer->SessionName != DiscardCandidate->SessionInfo.SessionName)
						{
							DiscardCandidate->SessionInfo.SessionName = SessionFromServer->SessionName; // Update the session name.
							OnArchivedSessionRenamed(*DiscardCandidate, SessionFromServer->SessionName);
						}
						else if (DiscardCandidate->bSessionNameDirty) // Renaming this particular session failed?
						{
							OnArchivedSessionListDirty(); // UI optimistically updated a session name from a rename, but it failed on the server. The controller caches the latest official server list unmodified. Force the client to resync its list against the cache.
							DiscardCandidate->bSessionNameDirty = false;
						}

						return false; // Don't remove the archive, it's still stored on the server.
					}

					OnArchivedSessionDiscarded(*DiscardCandidate);
					return true; // The session is not archived anymore on 'Server' anymore, remove it from the list.
				});

				// Add to the cache archives that was stored on 'Server' since the last update.
				for (const FConcertSessionInfo& SessionInfo : Response.Sessions)
				{
					// Try to find the archived in the list.
					if (!ArchivedSessions.ContainsByPredicate([&SessionInfo, &ServerInfo](const TSharedPtr<FArchivedSessionInfo>& MatchCandidate)
						{ return ServerInfo.InstanceInfo.InstanceId == MatchCandidate->ServerInfo.InstanceInfo.InstanceId && SessionInfo.SessionId == MatchCandidate->SessionInfo.SessionId; }))
					{
						// This is a newly discovered archive, add it to the list.
						ArchivedSessions.Add(MakeShared<FClientArchivedSessionInfo>(ServerInfo, SessionInfo));
						OnArchivedSessionDiscovered(*ArchivedSessions.Last());
					}
				}
			}
			// else -> Request failed, will discovered what happened on next TickServersAndSessionsDiscovery().
		});
}

void FConcertClientSessionBrowserController::UpdateClientsAsync(const FGuid& ServerAdminEndpointId, const FGuid& SessionId)
{
	// The clients request is for a different session than the one cached by the last client update (The UI only shows 1 at the time).
	if (ClientMonitoredSession.IsSet() && (ClientMonitoredSession->Session->ServerInfo.AdminEndpointId != ServerAdminEndpointId || ClientMonitoredSession->Session->SessionInfo.SessionId != SessionId))
	{
		// Reset the cache.
		ClientMonitoredSession->Session->Clients.Reset();
		ClientMonitoredSession->ListClientRequest.Cancel();
		ClientMonitoredSession.Reset();
		ClientListVersion = 0;
	}

	// Find the session requested by the user.
	if (!ClientMonitoredSession.IsSet())
	{
		TSharedPtr<FClientActiveSessionInfo>* MatchEntry = ActiveSessions.FindByPredicate([&ServerAdminEndpointId, &SessionId](const TSharedPtr<FClientActiveSessionInfo>& MatchCandidate)
		{
			return MatchCandidate->ServerInfo.AdminEndpointId == ServerAdminEndpointId && MatchCandidate->SessionInfo.SessionId == SessionId;
		});

		if (MatchEntry != nullptr)
		{
			ClientMonitoredSession = FMonitoredSessionInfo { MatchEntry->ToSharedRef() };
		}
		else
		{
			ClientMonitoredSession.Reset();
		}
	}

	if (!ClientMonitoredSession.IsSet() || ClientMonitoredSession->ListClientRequest.IsOngoing())
	{
		// Don't stack another async request on top of the on-going one, TickClientsDiscovery() will eventually catch up any possibly missed info here.
		return;
	}

	// Arm the future, enabling its continuation to execute when called.
	TWeakPtr<uint8, ESPMode::ThreadSafe> ListClientExecutionToken = ClientMonitoredSession->ListClientRequest.ResetExecutionToken();

	// Retrieve (asynchronously) the clients corresponding to the selected server/session.
	ClientMonitoredSession->ListClientRequest.Future = ConcertClient->GetSessionClients(ServerAdminEndpointId, SessionId)
		.Next([this, ListClientExecutionToken](const FConcertAdmin_GetSessionClientsResponse& Response)
		{
			// If the future execution was canceled.
			if (!ListClientExecutionToken.IsValid())
			{
				// Don't go further, the future execution was canceled.
				return;
			}

			auto SortClientPredicate = [](const FConcertSessionClientInfo& Lhs, const FConcertSessionClientInfo& Rhs) { return Lhs.ClientEndpointId < Rhs.ClientEndpointId; };

			// If the request succeeded.
			if (Response.ResponseCode == EConcertResponseCode::Success)
			{
				if (ClientMonitoredSession->Session->Clients.Num() != Response.SessionClients.Num())
				{
					ClientMonitoredSession->Session->Clients = Response.SessionClients;
					ClientMonitoredSession->Session->Clients.Sort(SortClientPredicate);
					OnActiveSessionClientsUpdated(*ClientMonitoredSession->Session);
					return;
				}
				else if (Response.SessionClients.Num() == 0) // All existing clients disconnected.
				{
					ClientMonitoredSession->Session->Clients.Reset();
					OnActiveSessionClientsUpdated(*ClientMonitoredSession->Session);
				}
				else // Compare the old and the new list, both sorted by client endpoint id.
				{
					TArray<FConcertSessionClientInfo> SortedClients = Response.SessionClients;
					SortedClients.Sort(SortClientPredicate);

					int Index = 0;
					for (const FConcertSessionClientInfo& Client : SortedClients)
					{
						if (ClientMonitoredSession->Session->Clients[Index].ClientEndpointId != Client.ClientEndpointId || // Not the same client?
						    ClientMonitoredSession->Session->Clients[Index].ClientInfo != Client.ClientInfo)               // Client info was updated?
						{
							// The two lists are not identical, don't bother finding the 'delta', refresh all clients.
							ClientMonitoredSession->Session->Clients = SortedClients;
							OnActiveSessionClientsUpdated(*ClientMonitoredSession->Session);
							break;
						}
						++Index;
					}
				}
			}
		});
}

#undef LOCTEXT_NAMESPACE