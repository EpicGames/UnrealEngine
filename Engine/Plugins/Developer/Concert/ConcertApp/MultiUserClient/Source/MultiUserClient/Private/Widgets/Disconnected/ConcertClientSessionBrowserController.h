// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertClientModule.h"
#include "Session/Browser/IConcertSessionBrowserController.h"

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

/**
 * Runs and cache network queries for the UI. In the model-view-controller pattern, this class acts like the controller. Its purpose
 * is to keep the UI code as decoupled as possible from the API used to query it. It encapsulate the asynchronous code and provide a
 * simpler API to the UI.
 */
class FConcertClientSessionBrowserController : public IConcertSessionBrowserController
{
public:
	
	/** Keeps the state of an active async request and provides a tool to cancel its future continuation execution. */
	struct FAsyncRequest
	{
		/** Returns true if there is a registered async request future and if it hasn't executed yet. */
		bool IsOngoing() const { return Future.IsValid() && !Future.IsReady(); }

		/** Reset the execution token, canceling previous execution (if any) and setting up the token for a new request. */
		TWeakPtr<uint8, ESPMode::ThreadSafe> ResetExecutionToken() { FutureExecutionToken = MakeShared<uint8, ESPMode::ThreadSafe>(); return FutureExecutionToken; }

		/** Cancel the execution of request async continuation. */
		void Cancel() { FutureExecutionToken.Reset(); }

		/** The future provided by an asynchronous request. */
		TFuture<void> Future;

		/** Determines whether or not the async request continuation code should execute. Reset to disarm execution of an async future continuation. */
		TSharedPtr<uint8, ESPMode::ThreadSafe> FutureExecutionToken;
	};
	
	FConcertClientSessionBrowserController(IConcertClientPtr InConcertClient);
	virtual ~FConcertClientSessionBrowserController() override;

	/**
	 * Fires new requests to retrieve all known server and for each server, their active and archived sessions. The responses are
	 * received asynchronously and may not be available right now. When a response is received, if the corresponding list cached
	 * is updated, the list version is incremented.
	 *
	 * @return A (serverListVersion, sessionsListVersion) pair, corresponding the the versions currently cached by this object.
	 */
	TPair<uint32, uint32> TickServersAndSessionsDiscovery();

	/**
	 * Fires a new request to retrieve the clients for the selected session. The result is cached and can be retrived by GetClients().
	 * The class caches clients for a single active session determined by this function. If the specified session changes, the cache
	 * of the previous session is cleared.
	 *
	 * @return The version of the client list currently cached by the object.
	 */
	uint32 TickClientsDiscovery(const FGuid& ServerAdminEndpointId, const FGuid& SessionId);

	/** Returns true if the controller received async responses and updated its cache since the last time the function was called, then clear the flag. */
	bool GetAndClearDiscoveryUpdateFlag();
	
	/** Returns the latest list of clients corresponding to the session known to this controller. Ensure to call TickClientsDiscovery() periodically. */
	const TArray<FConcertSessionClientInfo>& GetClients(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const;
	
	void JoinSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId);
	
	// The 2 functions below are used to prevent fast UI transition with the 'no session' panel flashing when the session list is empty.
	bool HasReceivedInitialSessionList() const { return bInitialActiveSessionQueryResponded && bInitialArchivedSessionQueryResponded; }
	bool IsCreatingSession() const { return CreateSessionRequests.Num() > 0 || ExpectedSessionsToDiscover.Num() > 0; }
	
	IConcertClientPtr GetConcertClient() const { return ConcertClient; }
	
	//~ Begin IConcertSessionBrowserController Interface
	virtual TArray<FConcertServerInfo> GetServers() const override { return Servers; }

	virtual TArray<FActiveSessionInfo> GetActiveSessions() const override
	{
		TArray<FActiveSessionInfo> Result;
		Algo::Transform(ActiveSessions, Result, [](TSharedPtr<FClientActiveSessionInfo> Info) { return *Info; });
		return Result;
	}

	virtual TArray<FArchivedSessionInfo> GetArchivedSessions() const override
	{
		TArray<FArchivedSessionInfo> Result;
		Algo::Transform(ArchivedSessions, Result, [](TSharedPtr<FClientArchivedSessionInfo> Info) { return *Info; });
		return Result;
	}

	virtual TOptional<FConcertSessionInfo> GetActiveSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const override;
	virtual TOptional<FConcertSessionInfo> GetArchivedSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const override;

	virtual void CreateSession(const FGuid& ServerAdminEndpointId, const FString& SessionName, const FString& ProjectName) override;
	virtual void ArchiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& ArchiveName, const FConcertSessionFilter& SessionFilter) override;
	virtual void RestoreSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& RestoredName, const FConcertSessionFilter& SessionFilter) override;
	virtual void RenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) override;
	virtual void RenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) override;
	virtual bool CanRenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override;
	virtual bool CanRenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override;
	virtual void DeleteSessions(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& SessionIds) override;
	virtual bool CanDeleteActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override;
	virtual bool CanDeleteArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override;
	virtual bool CanEverCreateSessions() const override { return true; }
	//~ End IConcertSessionBrowserController Interface

private:
	
	/** Hold information about a session created by this client, not yet 'discovered' by a 'list session' query, but expected to be soon. */
	struct FPendingSessionDiscovery
	{
		FDateTime CreateTimestamp;
		FGuid ServerEndpoint;
		FString SessionName;
	};

	struct FMonitoredSessionInfo
	{
		TSharedRef<FActiveSessionInfo> Session;
		FAsyncRequest ListClientRequest;

		FMonitoredSessionInfo(TSharedRef<FActiveSessionInfo> Session)
			: Session(Session)
		{}
	};

	struct FClientActiveSessionInfo : public FActiveSessionInfo
	{
		/** Raised when the UI and the cache values may be out of sync if a rename failed (UI assumed it succeeded) */
		bool bSessionNameDirty = false;

		FClientActiveSessionInfo(FConcertServerInfo ServerInfo, FConcertSessionInfo SessionInfo)
			: FActiveSessionInfo(MoveTemp(ServerInfo), MoveTemp(SessionInfo))
		{}
	};

	struct FClientArchivedSessionInfo : public FArchivedSessionInfo
	{
		/** Raised when the UI and the cache values may be out of sync if a rename failed (UI assumed it succeeded) */
		bool bSessionNameDirty = false;

		FClientArchivedSessionInfo(FConcertServerInfo ServerInfo, FConcertSessionInfo SessionInfo)
			: FArchivedSessionInfo(MoveTemp(ServerInfo), MoveTemp(SessionInfo))
		{}
	};

	void UpdateSessionsAsync();
	void UpdateActiveSessionsAsync(const FConcertServerInfo& ServerInfo);
	void UpdateArchivedSessionsAsync(const FConcertServerInfo& ServerInfo);
	void UpdateClientsAsync(const FGuid& ServerAdminEndpointId, const FGuid& SessionId);

	void OnActiveSessionDiscovered(const FActiveSessionInfo&) { ++SessionListVersion; bCacheUpdated = true; }
	void OnActiveSessionDiscarded(const FActiveSessionInfo&) { ++SessionListVersion; bCacheUpdated = true; }
	void OnArchivedSessionDiscovered(const FArchivedSessionInfo&) { ++SessionListVersion; bCacheUpdated = true; }
	void OnArchivedSessionDiscarded(const FArchivedSessionInfo&) { ++SessionListVersion; bCacheUpdated = true; }
	void OnActiveSessionClientsUpdated(const FActiveSessionInfo&) { ++ClientListVersion; bCacheUpdated = true; }
	void OnActiveSessionRenamed(const FActiveSessionInfo&, const FString& NewName) { ++SessionListVersion; bCacheUpdated = true; }
	void OnArchivedSessionRenamed(const FArchivedSessionInfo&, const FString& NewName) { ++SessionListVersion; bCacheUpdated = true; }
	void OnActiveSessionListDirty() { ++SessionListVersion; bCacheUpdated = true; } // This will force the UI to refresh its list.
	void OnArchivedSessionListDirty() { ++SessionListVersion; bCacheUpdated = true; } // This will force the UI to refresh its list.

	// Holds a concert client instance.
	IConcertClientPtr ConcertClient;

	// The list of active/archived async requests (requesting the list of session) per server. There is only one per server as we prevent stacking more than one at the time.
	TMap<FGuid, FAsyncRequest> ActiveSessionRequests;
	TMap<FGuid, FAsyncRequest> ArchivedSessionRequests;

	// The cached lists.
	TArray<FConcertServerInfo> Servers;
	TArray<TSharedPtr<FClientActiveSessionInfo>> ActiveSessions;
	TArray<TSharedPtr<FClientArchivedSessionInfo>> ArchivedSessions;

	// The session for which the clients are monitored. UI only monitor client of 1 session at the time.
	TOptional<FMonitoredSessionInfo> ClientMonitoredSession;

	// Holds the version of data cached by the controller. The version is updated when an async response is received and implies a change in the cached values.
	uint32 ServerListVersion = 0;
	uint32 SessionListVersion = 0;
	uint32 ClientListVersion = 0;
	bool bCacheUpdated = false;
	bool bInitialActiveSessionQueryResponded = false;
	bool bInitialArchivedSessionQueryResponded = false;

	TArray<FAsyncRequest> CreateSessionRequests;
	TArray<FPendingSessionDiscovery> ExpectedSessionsToDiscover;
	TSet<FString> IgnoredServers; // List of ignored servers (Useful for testing/debugging)
};
