// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertClientSessionBrowser.h"

#include "ConcertClientSettings.h"
#include "ConcertFrontendStyle.h"
#include "ConcertFrontendUtils.h"
#include "ConcertMessageData.h"
#include "ConcertSettings.h"
#include "ConcertVersion.h"
#include "IMultiUserClientModule.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "SConcertDiscovery.h"
#include "SConcertNoAvailability.h"
#include "Session/Browser/ConcertBrowserUtils.h"
#include "Session/Browser/Items/ConcertSessionTreeItem.h"
#include "Session/Browser/SConcertSessionBrowser.h"
#include "Widgets/Disconnected/ConcertClientSessionBrowserController.h"
#include "Widgets/Client/SClientName.h"
#include "Widgets/Client/ClientInfoHelpers.h"

#include "Algo/Transform.h"
#include "Algo/TiedTupleOutput.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

#define LOCTEXT_NAMESPACE "SConcertClientSessionBrowser"

void SConcertClientSessionBrowser::Construct(const FArguments& InArgs, IConcertClientPtr InConcertClient, TSharedPtr<FText> InSearchText)
{
	if (!InConcertClient.IsValid() || IsEngineExitRequested())
	{
		return; // Don't build the UI if ConcertClient is not available.
	}

	Controller = MakeShared<FConcertClientSessionBrowserController>(InConcertClient);

	// Displayed if no server is available.
	ServerDiscoveryPanel = SNew(SConcertDiscovery)
		.Text(LOCTEXT("LookingForServer", "Looking for Multi-User Servers..."))
		.Visibility_Lambda([this]() { return Controller->GetServers().Num() == 0 ? EVisibility::Visible : EVisibility::Hidden; })
		.IsButtonEnabled(this, &SConcertClientSessionBrowser::IsLaunchServerButtonEnabled)
		.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
		.ButtonIcon(FConcertFrontendStyle::Get()->GetBrush("Concert.NewServer.Small"))
		.ButtonText(LOCTEXT("LaunchLocalServer", "Launch a Server"))
		.ButtonToolTip(LOCTEXT("LaunchServerTooltip", "Launch a Multi-User server on your computer unless one is already running.\nThe editor UDP messaging settings will be passed to the launching server.\nThe server will use the server port found in Multi-User client settings if non 0 or use the editor udp messaging port number + 1, if non 0."))
		.OnButtonClicked(this, &SConcertClientSessionBrowser::OnLaunchServerButtonClicked);

	// Controls the text displayed in the 'No sessions' panel.
	auto GetNoSessionText = [this]()
	{
		if (!Controller->HasReceivedInitialSessionList())
		{
			return LOCTEXT("LookingForSession", "Looking for Multi-User Sessions...");
		}

		return Controller->GetActiveSessions().Num() == 0 && Controller->GetArchivedSessions().Num() == 0 ?
			LOCTEXT("NoSessionAvailable", "No Sessions Available") :
			LOCTEXT("AllSessionsFilteredOut", "No Sessions Match the Filters\nChange Your Filter to View Sessions");
	};

	// Displayed when discovering session or if no session is available.
	SessionDiscoveryPanel = SNew(SConcertDiscovery)
		.Text_Lambda(GetNoSessionText)
		.Visibility_Lambda([this]() { return Controller->GetServers().Num() > 0 && !SessionBrowser->HasAnySessions() && !Controller->IsCreatingSession() ? EVisibility::Visible : EVisibility::Hidden; })
		.ThrobberVisibility_Lambda([this]() { return !Controller->HasReceivedInitialSessionList() ? EVisibility::Visible : EVisibility::Collapsed; })
		.ButtonVisibility_Lambda([this]() { return Controller->HasReceivedInitialSessionList() && Controller->GetActiveSessions().Num() == 0 && Controller->GetArchivedSessions().Num() == 0 ? EVisibility::Visible : EVisibility::Collapsed; })
		.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
		.ButtonIcon(FConcertFrontendStyle::Get()->GetBrush("Concert.NewSession.Small"))
		.ButtonText(LOCTEXT("CreateSession", "Create Session"))
		.ButtonToolTip(LOCTEXT("CreateSessionTooltip", "Create a new session"))
		.OnButtonClicked(FOnClicked::CreateSP(this, &SConcertClientSessionBrowser::OnNewButtonClicked));

	// Displayed when the selected session client view is empty (no client to display).
	NoClientPanel = SNew(SConcertNoAvailability)
		.Text(LOCTEXT("NoClientAvailable", "No Connected Clients"))
		.Visibility_Lambda([this]() { return Clients.Num() == 0 ? EVisibility::Visible : EVisibility::Hidden; });

	// Displayed as details when no session is selected. (No session selected or the selected session doesn't have any)
	NoSessionSelectedPanel = SNew(SConcertNoAvailability)
		.Text(LOCTEXT("NoSessionSelected", "Select a Session to View Details"));

	// Displayed as details when the selected session has not specific details to display.
	NoSessionDetailsPanel = SNew(SConcertNoAvailability)
		.Text(LOCTEXT("NoSessionDetails", "The Selected Session Has No Details"));

	// List used in details panel to display clients connected to an active session.
	ClientsView = SNew(SListView<TSharedPtr<FConcertSessionClientInfo>>)
		.ListItemsSource(&Clients)
		.OnGenerateRow(this, &SConcertClientSessionBrowser::OnGenerateClientRowWidget)
		.SelectionMode(ESelectionMode::Single)
		.AllowOverscroll(EAllowOverscroll::No);

	ChildSlot
	[
		MakeBrowserContent(InSearchText, InConcertClient.ToSharedRef())
	];

	// Create a timer to periodically poll the server for sessions and session clients at a lower frequency than the normal tick.
	RegisterActiveTimer(1.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SConcertClientSessionBrowser::TickDiscovery));

	const bool bForceTaskStart = true;
	ScheduleDiscoveryTaskIfPossible(bForceTaskStart);
}

TSharedRef<SWidget> SConcertClientSessionBrowser::MakeBrowserContent(TSharedPtr<FText> InSearchText, const TSharedRef<IConcertClient>& Client)
{
	return SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			// Splitter upper part displaying the available sessions/(server).
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			.MinimumSlotHeight(80.0f) // Prevent widgets from overlapping.
			+SSplitter::Slot()
			.Value(0.6)
			[
				SAssignNew(SessionBrowser, SConcertSessionBrowser, Controller.ToSharedRef(), InSearchText)
					.ExtendSessionTable(this, &SConcertClientSessionBrowser::MakeOverlayedTableView)
					.ExtendControllButtons(this, &SConcertClientSessionBrowser::ExtendControlButtons)
					.ExtendSessionContextMenu(this, &SConcertClientSessionBrowser::ExtendSessionContextMenu)
					.RightOfControlButtons()
					[
						MakeUserAndSettings(InSearchText, Client)
					]
					.OnSessionClicked(this, &SConcertClientSessionBrowser::OnSessionSelectionChanged)
					.OnLiveSessionDoubleClicked(this, &SConcertClientSessionBrowser::OnSessionDoubleClicked)
					.PostRequestedDeleteSession_Lambda([this](auto) { UpdateDiscovery(); /* Don't wait up to 1s, kick discovery right now */ })
					.AskUserToDeleteSessions(this, &SConcertClientSessionBrowser::ConfirmDeleteSessionWithDialog)
			]

			// Session details.
			+SSplitter::Slot()
			.Value(0.4)
			[
				SAssignNew(SessionDetailsView, SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					NoSessionSelectedPanel.ToSharedRef()
				]
			]
		];
}

void SConcertClientSessionBrowser::ExtendControlButtons(FExtender& Extender)
{
	// Before separator
	Extender.AddToolBarExtension(
		SConcertSessionBrowser::ControlButtonExtensionHooks::BeforeSeparator,
		EExtensionHook::First,
		nullptr,
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& Builder)
		{
			Builder.AddWidget(
				SNew(SOverlay)
				// Launch server.
				+SOverlay::Slot()
				[
					ConcertBrowserUtils::MakeIconButton(
						FConcertFrontendStyle::Get()->GetBrush("Concert.NewServer"),
						LOCTEXT("LaunchServerTooltip", "Launch a Multi-User server on your computer unless one is already running.\nThe editor UDP messaging settings will be passed to the launching server.\nThe server will use the server port found in Multi-User client settings if non 0 or use the editor udp messaging port number + 1, if non 0."),
						TAttribute<bool>(this, &SConcertClientSessionBrowser::IsLaunchServerButtonEnabled),
						FOnClicked::CreateSP(this, &SConcertClientSessionBrowser::OnLaunchServerButtonClicked),
						TAttribute<EVisibility>::Create([this]() { return IsLaunchServerButtonEnabled() ? EVisibility::Visible : EVisibility::Collapsed; }))
				]
				// Stop server.
				+SOverlay::Slot()
				[
					ConcertBrowserUtils::MakeIconButton(
						FConcertFrontendStyle::Get()->GetBrush("Concert.CloseServer"),
						LOCTEXT("ShutdownServerTooltip", "Shutdown the Multi-User server running on this computer."),
						true, // Always enabled, but might be collapsed.
						FOnClicked::CreateSP(this, &SConcertClientSessionBrowser::OnShutdownServerButtonClicked),
						TAttribute<EVisibility>::Create([this]() { return IsLaunchServerButtonEnabled() ? EVisibility::Collapsed : EVisibility::Visible; }))
				]
				);
		}));

	// After separator
	Extender.AddToolBarExtension(
		SConcertSessionBrowser::ControlButtonExtensionHooks::AfterSeparator,
		EExtensionHook::First,
		nullptr,
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& Builder)
		{
			TAttribute<FText> AutoJoinTooltip = TAttribute<FText>::Create([this]()
			{
				if (Controller->GetConcertClient()->CanAutoConnect()) // Default session and server are configured?
				{
					return FText::Format(LOCTEXT("JoinDefaultSessionTooltip", "Join the default session '{0}' on '{1}'"),
						FText::FromString(Controller->GetConcertClient()->GetConfiguration()->DefaultSessionName),
						FText::FromString(Controller->GetConcertClient()->GetConfiguration()->DefaultServerURL));
				}
				else
				{
					return LOCTEXT("JoinDefaultSessionConfiguredTooltip", "Join the default session configured in the Multi-Users settings");
				}
			});

			TAttribute<FText> CancelAutoJoinTooltip = TAttribute<FText>::Create([this]()
			{
				return FText::Format(LOCTEXT("CancelJoinDefaultSessionTooltip", "Cancel joining the default session '{0}' on '{1}'"),
					FText::FromString(Controller->GetConcertClient()->GetConfiguration()->DefaultSessionName),
					FText::FromString(Controller->GetConcertClient()->GetConfiguration()->DefaultServerURL));
			});
			
			Builder.AddWidget(
				SNew(SOverlay)
				// Auto-join
				+SOverlay::Slot()
				[
					ConcertBrowserUtils::MakeIconButton(
						FConcertFrontendStyle::Get()->GetBrush("Concert.JoinDefaultSession"),
						AutoJoinTooltip,
						TAttribute<bool>(this, &SConcertClientSessionBrowser::IsAutoJoinButtonEnabled),
						FOnClicked::CreateSP(this, &SConcertClientSessionBrowser::OnAutoJoinButtonClicked),
						// Default button shown if both auto-join/cancel are disabled.
						TAttribute<EVisibility>::Create([this]() { return !IsCancelAutoJoinButtonEnabled() ? EVisibility::Visible : EVisibility::Collapsed; })
						) 
				
				]
				// Cancel auto join.
				+SOverlay::Slot()
				[
					ConcertBrowserUtils::MakeIconButton(
						FConcertFrontendStyle::Get()->GetBrush("Concert.CancelAutoJoin"),
						CancelAutoJoinTooltip,
						TAttribute<bool>(this, &SConcertClientSessionBrowser::IsCancelAutoJoinButtonEnabled),
						FOnClicked::CreateSP(this, &SConcertClientSessionBrowser::OnCancelAutoJoinButtonClicked),
						TAttribute<EVisibility>::Create([this]() { return IsCancelAutoJoinButtonEnabled() ? EVisibility::Visible : EVisibility::Collapsed; })
						)
				]
			);

			// Join
			Builder.AddWidget(
				ConcertBrowserUtils::MakeIconButton(FConcertFrontendStyle::Get()->GetBrush("Concert.JoinSession"), LOCTEXT("JoinButtonTooltip", "Join the selected session"),
					TAttribute<bool>(this, &SConcertClientSessionBrowser::IsJoinButtonEnabled),
					FOnClicked::CreateSP(this, &SConcertClientSessionBrowser::OnJoinButtonClicked),
					TAttribute<EVisibility>::Create([this]() { return !SessionBrowser->IsRestoreButtonEnabled() ? EVisibility::Visible : EVisibility::Collapsed; })) // Default button shown if both join/restore are disabled.
			);
		}));
}

void SConcertClientSessionBrowser::ExtendSessionContextMenu(const TSharedPtr<FConcertSessionTreeItem>& Item, FExtender& Extender)
{
	if (Item->Type == FConcertSessionTreeItem::EType::ActiveSession)
	{
		Extender.AddMenuExtension(
			SConcertSessionBrowser::SessionContextMenuExtensionHooks::ManageSession,
			EExtensionHook::First,
			nullptr,
			FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& Builder)
			{
				Builder.AddMenuEntry(
					LOCTEXT("CtxMenuJoin", "Join"),
					LOCTEXT("CtxMenuJoin_Tooltip", "Join the Session"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this](){ OnJoinButtonClicked(); }),
						FCanExecuteAction::CreateLambda([SelectedCount = SessionBrowser->GetSelectedItems().Num()] { return SelectedCount == 1; }),
						FIsActionChecked::CreateLambda([this] { return false; })),
					NAME_None,
					EUserInterfaceActionType::Button
					);
			}));
	}
	
}

TSharedRef<SWidget> SConcertClientSessionBrowser::MakeUserAndSettings(TSharedPtr<FText> InSearchText, const TSharedRef<IConcertClient>& Client)
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(UE::ConcertSharedSlate::SClientName)
			.ClientInfo(UE::ConcertClientSharedSlate::MakeLocalClientInfoAttribute(Client))
			.HighlightText_Lambda([InSearchText](){ return *InSearchText.Get(); })
		]

		// The "Settings" icons.
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.OnClicked_Lambda([](){FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer("Project", "Plugins", "Concert"); return FReply::Handled(); })
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Settings"))
			]
		];
}

TSharedRef<SWidget> SConcertClientSessionBrowser::MakeOverlayedTableView(const TSharedRef<SWidget>& SessionTable)
{
	return SNew(SOverlay)
		+SOverlay::Slot()
		[
			SessionTable
		]
		+SOverlay::Slot()
		.Padding(0, 20, 0, 0) // To ensure the panel is below the row header.
		[
			SessionDiscoveryPanel.ToSharedRef()
		]
		+SOverlay::Slot()
		.Padding(0, 20, 0, 0) // To ensure the panel is below the row header.
		[
			ServerDiscoveryPanel.ToSharedRef()
		];
}

void SConcertClientSessionBrowser::ScheduleDiscoveryTaskIfPossible(bool bForceTaskStart)
{
	if (bForceTaskStart || DiscoveryTask.IsCompleted())
	{
		bLocalServerRunning = bForceTaskStart ? false : DiscoveryTask.GetResult();
		DiscoveryTask = UE::Tasks::Launch( UE_SOURCE_LOCATION, []
		{
			return IMultiUserClientModule::Get().IsConcertServerRunning();
		});
	}
}

EActiveTimerReturnType SConcertClientSessionBrowser::TickDiscovery(double InCurrentTime, float InDeltaTime)
{
	ScheduleDiscoveryTaskIfPossible(false);
	UpdateDiscovery();
	return EActiveTimerReturnType::Continue;
}

void SConcertClientSessionBrowser::UpdateDiscovery()
{
	// Check if the controller has updated data since the last 'tick' and fire new asynchronous requests for future 'tick'.
	TPair<uint32, uint32> ServerSessionListVersions = Controller->TickServersAndSessionsDiscovery();
	ServerListVersion = ServerSessionListVersions.Key;

	if (ServerSessionListVersions.Value != DisplayedSessionListVersion) // Need to refresh the list?
	{
		SessionBrowser->RefreshSessionList();
		DisplayedSessionListVersion = ServerSessionListVersions.Value;
	}

	// If an active session is selected.
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedSession = SessionBrowser->GetSelectedItems();
	if (SelectedSession.Num() && SelectedSession[0]->Type == FConcertSessionTreeItem::EType::ActiveSession)
	{
		// Ensure to poll its clients.
		uint32 CachedClientListVersion = Controller->TickClientsDiscovery(SelectedSession[0]->ServerAdminEndpointId, SelectedSession[0]->SessionId);
		if (CachedClientListVersion != DisplayedClientListVersion) // Need to refresh the list?
		{
			RefreshClientList(Controller->GetClients(SelectedSession[0]->ServerAdminEndpointId, SelectedSession[0]->SessionId));
			DisplayedClientListVersion = CachedClientListVersion;
		}
	}
}

void SConcertClientSessionBrowser::RefreshClientList(const TArray<FConcertSessionClientInfo>& LastestClientList)
{
	// Remember which client is selected.
	TArray<TSharedPtr<FConcertSessionClientInfo>> SelectedItems = ClientsView->GetSelectedItems();

	// Copy the list of clients.
	TArray<TSharedPtr<FConcertSessionClientInfo>> LatestClientPtrs;
	Algo::Transform(LastestClientList, LatestClientPtrs, [](const FConcertSessionClientInfo& Client) { return MakeShared<FConcertSessionClientInfo>(Client); });

	// Merge the current list with the new list, removing client that disappeared and adding client that appeared.
	ConcertFrontendUtils::SyncArraysByPredicate(Clients, MoveTemp(LatestClientPtrs), [](const TSharedPtr<FConcertSessionClientInfo>& ClientToFind)
	{
		return [ClientToFind](const TSharedPtr<FConcertSessionClientInfo>& PotentialClientMatch)
		{
			return PotentialClientMatch->ClientEndpointId == ClientToFind->ClientEndpointId && PotentialClientMatch->ClientInfo == ClientToFind->ClientInfo;
		};
	});

	// Sort the list item alphabetically.
	Clients.StableSort([](const TSharedPtr<FConcertSessionClientInfo>& Lhs, const TSharedPtr<FConcertSessionClientInfo>& Rhs)
	{
		return Lhs->ClientInfo.DisplayName < Rhs->ClientInfo.DisplayName;
	});

	// Preserve previously selected item (if any).
	if (SelectedItems.Num())
	{
		ClientsView->SetSelection(SelectedItems[0]);
	}
	ClientsView->RequestListRefresh();
}

TSharedRef<SWidget> SConcertClientSessionBrowser::MakeSessionDetails(TSharedPtr<FConcertSessionTreeItem> Item)
{
	const FConcertSessionInfo* SessionInfo = nullptr;

	if (Item.IsValid())
	{
		if (Item->Type == FConcertSessionTreeItem::EType::ActiveSession || Item->Type == FConcertSessionTreeItem::EType::SaveSession)
		{
			return MakeActiveSessionDetails(Item);
		}
		else if (Item->Type == FConcertSessionTreeItem::EType::ArchivedSession)
		{
			return MakeArchivedSessionDetails(Item);
		}
	}

	return NoSessionSelectedPanel.ToSharedRef();
}

TSharedRef<SWidget> SConcertClientSessionBrowser::MakeActiveSessionDetails(TSharedPtr<FConcertSessionTreeItem> Item)
{
	const TOptional<FConcertSessionInfo> SessionInfo = Controller->GetActiveSessionInfo(Item->ServerAdminEndpointId, Item->SessionId);
	if (!SessionInfo)
	{
		return NoSessionSelectedPanel.ToSharedRef();
	}

	TSharedPtr<SGridPanel> Grid;

	// State variables captured and shared by the different lambda functions below.
	TSharedPtr<bool> bDetailsAreaExpanded = MakeShared<bool>(false);
	TSharedPtr<bool> bClientsAreaExpanded = MakeShared<bool>(true);

	auto DetailsAreaSizeRule = [this, bDetailsAreaExpanded]()
	{
		return *bDetailsAreaExpanded ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
	};

	auto OnDetailsAreaExpansionChanged = [bDetailsAreaExpanded](bool bExpanded)
	{
		*bDetailsAreaExpanded = bExpanded;
	};

	auto ClientsAreaSizeRule = [this, bClientsAreaExpanded]()
	{
		return *bClientsAreaExpanded ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
	};

	auto OnClientsAreaExpansionChanged = [bClientsAreaExpanded](bool bExpanded)
	{
		*bClientsAreaExpanded = bExpanded;
	};

	TSharedRef<SSplitter> Widget = SNew(SSplitter)
		.Orientation(Orient_Vertical)

		// Details.
		+SSplitter::Slot()
		.SizeRule(TAttribute<SSplitter::ESizeRule>::Create(DetailsAreaSizeRule))
		.Value(0.6)
		[
			SAssignNew(DetailsArea, SExpandableArea)
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			.BorderImage_Lambda([this]() { return ConcertFrontendUtils::GetExpandableAreaBorderImage(*DetailsArea); })
			.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.BodyBorderBackgroundColor(FLinearColor::White)
			.OnAreaExpansionChanged(FOnBooleanValueChanged::CreateLambda(OnDetailsAreaExpansionChanged))
			.InitiallyCollapsed(!(*bDetailsAreaExpanded))
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("Details", "Details"), FText::FromString(Item->SessionName)))
				.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
				.ShadowOffset(FVector2D(1.0f, 1.0f))
			]
			.BodyContent()
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				+SScrollBox::Slot()
				[
					SNew(SBox)
					.Padding(FMargin(0, 2, 0, 2))
					[
						SAssignNew(Grid, SGridPanel)
					]
				]
			]
		]

		// Clients
		+SSplitter::Slot()
		.SizeRule(TAttribute<SSplitter::ESizeRule>::Create(ClientsAreaSizeRule))
		.Value(0.4)
		[
			SAssignNew(ClientsArea, SExpandableArea)
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			.BorderImage_Lambda([this]() { return ConcertFrontendUtils::GetExpandableAreaBorderImage(*ClientsArea); })
			.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.BodyBorderBackgroundColor(FLinearColor::White)
			.OnAreaExpansionChanged(FOnBooleanValueChanged::CreateLambda(OnClientsAreaExpansionChanged))
			.InitiallyCollapsed(!(*bClientsAreaExpanded))
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Clients", "Clients"))
				.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
				.ShadowOffset(FVector2D(1.0f, 1.0f))
			]
			.BodyContent()
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					ClientsView.ToSharedRef()
				]
				+SOverlay::Slot()
				[
					NoClientPanel.ToSharedRef()
				]
			]
		];

	// Fill the grid.
	PopulateSessionInfoGrid(*Grid, *SessionInfo);

	// Populate the client list.
	RefreshClientList(Controller->GetClients(Item->ServerAdminEndpointId, Item->SessionId));

	return Widget;
}

TSharedRef<SWidget> SConcertClientSessionBrowser::MakeArchivedSessionDetails(TSharedPtr<FConcertSessionTreeItem> Item)
{
	const TOptional<FConcertSessionInfo> SessionInfo = Controller->GetArchivedSessionInfo(Item->ServerAdminEndpointId, Item->SessionId);
	if (!SessionInfo)
	{
		return NoSessionSelectedPanel.ToSharedRef();
	}

	TSharedPtr<SGridPanel> Grid;

	TSharedRef<SExpandableArea> Widget = SAssignNew(DetailsArea, SExpandableArea)
	.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
	.BorderImage_Lambda([this]() { return ConcertFrontendUtils::GetExpandableAreaBorderImage(*DetailsArea); })
	.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
	.BodyBorderBackgroundColor(FLinearColor::White)
	.InitiallyCollapsed(true)
	.HeaderContent()
	[
		SNew(STextBlock)
		.Text(FText::Format(LOCTEXT("Details", "Details"), FText::FromString(Item->SessionName)))
		.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
		.ShadowOffset(FVector2D(1.0f, 1.0f))
	]
	.BodyContent()
	[
		SNew(SScrollBox)
		.Orientation(Orient_Vertical)
		+SScrollBox::Slot()
		[
			SNew(SBox)
			.Padding(FMargin(0, 2, 0, 2))
			[
				SAssignNew(Grid, SGridPanel)
			]
		]
	];

	// Fill the grid.
	PopulateSessionInfoGrid(*Grid, *SessionInfo);

	return Widget;
}

void SConcertClientSessionBrowser::PopulateSessionInfoGrid(SGridPanel& Grid, const FConcertSessionInfo& SessionInfo)
{
	// Local function to populate the details grid.
	auto AddDetailRow = [](SGridPanel& Grid, int32 Row, const FText& Label, const FText& Value)
	{
		const float RowPadding = (Row == 0) ? 0.0f : 4.0f; // Space between line.
		const float ColPadding = 4.0f; // Space between columns. (Minimum)

		Grid.AddSlot(0, Row)
		.Padding(0.0f, RowPadding, ColPadding, 0.0f)
		[
			SNew(STextBlock).Text(Label)
		];

		Grid.AddSlot(1, Row)
		.Padding(0.0f, RowPadding, 0.0f, 0.0f)
		[
			SNew(STextBlock).Text(Value)
		];
	};

	int32 Row = 0;
	AddDetailRow(Grid, Row++, LOCTEXT("SessionId", "Session ID:"), FText::FromString(SessionInfo.SessionId.ToString()));
	AddDetailRow(Grid, Row++, LOCTEXT("SessionName", "Session Name:"), FText::FromString(SessionInfo.SessionName));
	AddDetailRow(Grid, Row++, LOCTEXT("Owner", "Owner:"), FText::FromString(SessionInfo.OwnerUserName));
	AddDetailRow(Grid, Row++, LOCTEXT("Project", "Project:"), FText::FromString(SessionInfo.Settings.ProjectName));
	if (SessionInfo.VersionInfos.Num() > 0)
	{
		const FConcertSessionVersionInfo& VersionInfo = SessionInfo.VersionInfos.Last();
		AddDetailRow(Grid, Row++, LOCTEXT("EngineVersion", "Engine Version:"), VersionInfo.AsText());
	}
	AddDetailRow(Grid, Row++, LOCTEXT("ServerEndPointId", "Server Endpoint ID:"), FText::FromString(SessionInfo.ServerEndpointId.ToString()));
}

TSharedRef<ITableRow> SConcertClientSessionBrowser::OnGenerateClientRowWidget(TSharedPtr<FConcertSessionClientInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FConcertSessionClientInfo>>, OwnerTable)
	[
		SNew(SHorizontalBox)
		.ToolTipText(Item->ToDisplayString())

		// The user "Avatar color" displayed as a small square colored by the user avatar color.
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
			.ColorAndOpacity(Item->ClientInfo.AvatarColor)
			.Image(FAppStyle::GetBrush("Icons.FilledCircle"))
		]

		// The user "Display Name".
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(4.0, 2.0))
		[
			SNew(STextBlock)
			.Text(FText::FromString(Item->ClientInfo.DisplayName))
		]
	];
}

void SConcertClientSessionBrowser::OnSessionSelectionChanged(const TSharedPtr<FConcertSessionTreeItem>& SelectedSession)
{
	// Clear the list of clients (if any)
	Clients.Reset();

	// Update the details panel.
	SessionDetailsView->SetContent(MakeSessionDetails(SelectedSession));
}

void SConcertClientSessionBrowser::OnSessionDoubleClicked(const TSharedPtr<FConcertSessionTreeItem>& SelectedSession)
{
	switch (SelectedSession->Type)
	{
	case FConcertSessionTreeItem::EType::ActiveSession: 
		RequestJoinSession(SelectedSession);
		break;
	case FConcertSessionTreeItem::EType::ArchivedSession:
		SessionBrowser->InsertRestoreSessionAsEditableRow(SelectedSession);
		break;
	default: 
		checkNoEntry();
	}
}

bool SConcertClientSessionBrowser::ConfirmDeleteSessionWithDialog(const TArray<TSharedPtr<FConcertSessionTreeItem>>& SessionItems) const
{
	TSet<FString> SessionNames;
	TSet<FString> UniqueServers;

	Algo::Transform(SessionItems, Algo::TieTupleAdd(SessionNames, UniqueServers), [](const TSharedPtr<FConcertSessionTreeItem>& Item)
	{
		return MakeTuple(Item->SessionName, Item->ServerName);
	});
	
	const FText ConfirmationMessage = FText::Format(
		LOCTEXT("DeleteSessionConfirmationMessage", "Do you really want to delete {0} {0}|plural(one=session,other=sessions) from {1} {1}|plural(one=server,other=servers)?\n\nThe {0}|plural(one=session,other=sessions) {2} will be deleted from the {1}|plural(one=server,other=servers) {3}."),
		SessionItems.Num(),
		UniqueServers.Num(),
		FText::FromString(FString::JoinBy(SessionNames, TEXT(", "), [](const FString& Name) { return FString("\n\t- ") + Name; }) + FString("\n")),
		FText::FromString(FString::Join(UniqueServers, TEXT(", ")))
		);
	const FText ConfirmationTitle = LOCTEXT("DeleteSessionConfirmationTitle", "Delete Session Confirmation");
	return FMessageDialog::Open(EAppMsgType::YesNo, ConfirmationMessage, ConfirmationTitle) == EAppReturnType::Yes;
}

bool SConcertClientSessionBrowser::IsLaunchServerButtonEnabled() const
{
	return !bLocalServerRunning;
}

bool SConcertClientSessionBrowser::IsJoinButtonEnabled() const
{
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = SessionBrowser->GetSelectedItems();
	return SelectedItems.Num() == 1 && SelectedItems[0]->Type == FConcertSessionTreeItem::EType::ActiveSession;
}

bool SConcertClientSessionBrowser::IsAutoJoinButtonEnabled() const
{
	return Controller->GetConcertClient()->CanAutoConnect() && !Controller->GetConcertClient()->IsAutoConnecting();
}

bool SConcertClientSessionBrowser::IsCancelAutoJoinButtonEnabled() const
{
	return Controller->GetConcertClient()->IsAutoConnecting();
}

FReply SConcertClientSessionBrowser::OnNewButtonClicked()
{
	SessionBrowser->InsertNewSessionEditableRow();
	return FReply::Handled();
}

FReply SConcertClientSessionBrowser::OnLaunchServerButtonClicked()
{
	IMultiUserClientModule::Get().LaunchConcertServer();
	ScheduleDiscoveryTaskIfPossible(false);
	return FReply::Handled();
}

FReply SConcertClientSessionBrowser::OnShutdownServerButtonClicked()
{
	if (bLocalServerRunning)
	{
		IMultiUserClientModule::Get().ShutdownConcertServer();
	}
	return FReply::Handled();
}

FReply SConcertClientSessionBrowser::OnAutoJoinButtonClicked()
{
	IConcertClientPtr ConcertClient = Controller->GetConcertClient();

	// Start the 'auto connect' routine. It will try until it succeeded or gets canceled. Creating or Joining a session automatically cancels it.
	ConcertClient->StartAutoConnect();
	return FReply::Handled();
}

FReply SConcertClientSessionBrowser::OnCancelAutoJoinButtonClicked()
{
	Controller->GetConcertClient()->StopAutoConnect();
	return FReply::Handled();
}

FReply SConcertClientSessionBrowser::OnJoinButtonClicked()
{
	TArray<TSharedPtr<FConcertSessionTreeItem>> SelectedItems = SessionBrowser->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		RequestJoinSession(SelectedItems[0]);
	}
	return FReply::Handled();
}

void SConcertClientSessionBrowser::RequestJoinSession(const TSharedPtr<FConcertSessionTreeItem>& LiveItem)
{
	Controller->JoinSession(LiveItem->ServerAdminEndpointId, LiveItem->SessionId);
}

void SConcertClientSessionBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// If the model received updates.
	if (Controller->GetAndClearDiscoveryUpdateFlag())
	{
		// Don't wait next TickDiscovery() running at lower frequency, update immediately.
		UpdateDiscovery();
	}

	// Ensure the 'default server' filter is updated when the configuration of the default server changes.
	if (DefaultServerURL != Controller->GetConcertClient()->GetConfiguration()->DefaultServerURL)
	{
		DefaultServerURL = Controller->GetConcertClient()->GetConfiguration()->DefaultServerURL;
		bRefreshSessionFilter = true;
	}

	// Should refresh the session filter?
	if (bRefreshSessionFilter)
	{
		SessionBrowser->RefreshSessionList();
		bRefreshSessionFilter = false;
	}
}

#undef LOCTEXT_NAMESPACE
