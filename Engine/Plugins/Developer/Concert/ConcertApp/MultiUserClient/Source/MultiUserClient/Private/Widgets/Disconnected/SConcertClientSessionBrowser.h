// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertClientModule.h"
#include "Tasks/Task.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FConcertClientSessionBrowserController;
class FConcertSessionTreeItem;
class FExtender;
class ITableRow;
class SBorder;
class SConcertSessionBrowser;
class SExpandableArea;
class SGridPanel;
template<typename T>
class SListView;
class STableViewBase;

/**
 * Enables the user to browse/search/filter/sort active and archived sessions, create new session,
 * archive active sessions, restore archived sessions, join a session and open the settings dialog.
 */
class SConcertClientSessionBrowser : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SConcertClientSessionBrowser) { }
	SLATE_END_ARGS();

	/**
	* Constructs the Browser.
	* @param InArgs The Slate argument list.
	* @param InConcertClient The concert client used to list, join, delete the sessions.
	* @param[in,out] InSearchText The text to set in the search box and to remember (as output). Cannot be null.
	*/
	void Construct(const FArguments& InArgs, IConcertClientPtr InConcertClient, TSharedPtr<FText> InSearchText);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	
	// Gives access to the concert data (servers, sessions, clients, etc).
	TSharedPtr<FConcertClientSessionBrowserController> Controller;

	TSharedPtr<SConcertSessionBrowser> SessionBrowser;

	// Filtering.
	bool bRefreshSessionFilter = true;
	FString DefaultServerURL;

	// Selected Session Details.
	TSharedPtr<SBorder> SessionDetailsView;
	TSharedPtr<SExpandableArea> DetailsArea;
	TArray<TSharedPtr<FConcertSessionClientInfo>> Clients;
	TSharedPtr<SExpandableArea> ClientsArea;
	TSharedPtr<SListView<TSharedPtr<FConcertSessionClientInfo>>> ClientsView;

	// Used to compare the version used by UI versus the version cached in the controller.
	uint32 DisplayedSessionListVersion = 0;
	uint32 DisplayedClientListVersion = 0;
	uint32 ServerListVersion = 0;
	bool bLocalServerRunning = false;

	TSharedPtr<SWidget> ServerDiscoveryPanel; // Displayed until a server is found.
	TSharedPtr<SWidget> SessionDiscoveryPanel; // Displayed until a session is found.
	TSharedPtr<SWidget> NoSessionSelectedPanel; // Displays 'select a session to view details' message in the details section.
	TSharedPtr<SWidget> NoSessionDetailsPanel; // Displays 'no details available' message in the details section.
	TSharedPtr<SWidget> NoClientPanel; // Displays the 'no client connected' message in Clients expendable area.
	
	// Layout the 'session|details' split view.
	TSharedRef<SWidget> MakeBrowserContent(TSharedPtr<FText> InSearchText, const TSharedRef<IConcertClient>& Client);
	void ExtendControlButtons(FExtender& Extender);
	void ExtendSessionContextMenu(const TSharedPtr<FConcertSessionTreeItem>& Item, FExtender& Extender);
	TSharedRef<SWidget> MakeUserAndSettings(TSharedPtr<FText> InSearchText, const TSharedRef<IConcertClient>& Client);
	TSharedRef<SWidget> MakeOverlayedTableView(const TSharedRef<SWidget>& TableView);
	
	// Layouts the session detail panel.
	TSharedRef<SWidget> MakeSessionDetails(TSharedPtr<FConcertSessionTreeItem> Item);
	TSharedRef<SWidget> MakeActiveSessionDetails(TSharedPtr<FConcertSessionTreeItem> Item);
	TSharedRef<SWidget> MakeArchivedSessionDetails(TSharedPtr<FConcertSessionTreeItem> Item);
	TSharedRef<ITableRow> OnGenerateClientRowWidget(TSharedPtr<FConcertSessionClientInfo> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void PopulateSessionInfoGrid(SGridPanel& Grid, const FConcertSessionInfo& SessionInfo);

	// The buttons in some overlays
	bool IsLaunchServerButtonEnabled() const;
	bool IsJoinButtonEnabled() const;
	bool IsAutoJoinButtonEnabled() const;
	bool IsCancelAutoJoinButtonEnabled() const;
	FReply OnNewButtonClicked();
	FReply OnLaunchServerButtonClicked();
	FReply OnShutdownServerButtonClicked();
	FReply OnAutoJoinButtonClicked();
	FReply OnCancelAutoJoinButtonClicked();

	// SConcertSessionBrowser extensions
	FReply OnJoinButtonClicked();
	void RequestJoinSession(const TSharedPtr<FConcertSessionTreeItem>& ActiveItem);

	// Manipulates the sessions view (the array and the UI).
	void OnSessionSelectionChanged(const TSharedPtr<FConcertSessionTreeItem>& SessionItem);
	void OnSessionDoubleClicked(const TSharedPtr<FConcertSessionTreeItem>& SessionItem);
	bool ConfirmDeleteSessionWithDialog(const TArray<TSharedPtr<FConcertSessionTreeItem>>& SessionItems) const;

	// Update server/session/clients lists.
	void ScheduleDiscoveryTaskIfPossible(bool bForceTaskStart);
	EActiveTimerReturnType TickDiscovery(double InCurrentTime, float InDeltaTime);
	void UpdateDiscovery();
	void RefreshClientList(const TArray<FConcertSessionClientInfo>& LastestClientList);
	UE::Tasks::TTask<bool> DiscoveryTask;
};
