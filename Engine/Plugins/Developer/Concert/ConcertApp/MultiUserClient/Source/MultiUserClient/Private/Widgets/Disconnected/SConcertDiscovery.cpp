// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertDiscovery.h"

#include "Layout/Clipping.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SConcertDiscovery::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SCircularThrobber)
				.Visibility(InArgs._ThrobberVisibility)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock).Text(InArgs._Text).Justification(ETextJustify::Center)
			]

			+SVerticalBox::Slot()
			.Padding(0, 4, 0, 0)
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(InArgs._ButtonStyle)
				.Visibility(InArgs._ButtonVisibility)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.IsEnabled(InArgs._IsButtonEnabled)
				.OnClicked(InArgs._OnButtonClicked)
				.ToolTipText(InArgs._ButtonToolTip)
				.ContentPadding(FMargin(8, 4))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0, 0, 3, 0)
					[
						SNew(SImage).Image(InArgs._ButtonIcon)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					[
						SNew(STextBlock).Text(InArgs._ButtonText)
					]
				]
			]
		]
	];
}
