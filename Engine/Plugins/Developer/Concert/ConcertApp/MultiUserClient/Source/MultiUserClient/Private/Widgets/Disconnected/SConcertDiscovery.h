// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/** Widget displayed when discovering multi-user server(s) or session(s).*/
class SConcertDiscovery : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SConcertDiscovery)
		: _Text()
		, _ThrobberVisibility(EVisibility::Visible)
		, _ButtonVisibility(EVisibility::Visible)
		, _IsButtonEnabled(true)
		, _ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton"))
		, _ButtonIcon()
		, _ButtonText()
		, _ButtonToolTip()
		, _OnButtonClicked()
	{}
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_ATTRIBUTE(EVisibility, ThrobberVisibility)
		SLATE_ATTRIBUTE(EVisibility, ButtonVisibility)
		SLATE_ATTRIBUTE(bool, IsButtonEnabled)
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
		SLATE_ATTRIBUTE(const FSlateBrush*, ButtonIcon)
		SLATE_ATTRIBUTE(FText, ButtonText)
		SLATE_ATTRIBUTE(FText, ButtonToolTip)
		SLATE_EVENT( FOnClicked, OnButtonClicked)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);
};
