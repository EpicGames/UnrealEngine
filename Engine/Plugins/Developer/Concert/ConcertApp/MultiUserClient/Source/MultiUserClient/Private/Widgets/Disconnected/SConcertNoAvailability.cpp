// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertNoAvailability.h"

#include "Layout/Visibility.h"
#include "SConcertDiscovery.h"

void SConcertNoAvailability::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		// Reuse this panel, but only show the message.
		SNew(SConcertDiscovery) 
			.Text(InArgs._Text)
			.ThrobberVisibility(EVisibility::Collapsed)
			.ButtonVisibility(EVisibility::Collapsed)
	];
}
