// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/** Displayed when something is not available. */
class SConcertNoAvailability : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SConcertNoAvailability) : _Text() {}
		SLATE_ATTRIBUTE(FText, Text)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
};
