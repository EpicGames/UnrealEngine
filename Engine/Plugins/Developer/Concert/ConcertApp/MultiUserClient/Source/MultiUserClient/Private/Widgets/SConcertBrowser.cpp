// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertBrowser.h"

#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "MultiUserClientUtils.h"
#include "ActiveSession/SActiveSessionRoot.h"
#include "Widgets/Disconnected/SConcertClientSessionBrowser.h"
#include "Widgets/Disconnected/SConcertNoAvailability.h"

#include "EntryPoint/EntryPointWidgetFactory.h"
#include "Internationalization/Regex.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "SConcertBrowser"

void SConcertBrowser::Construct(
	const FArguments& InArgs,
	TSharedRef<IConcertSyncClient> InSyncClient,
	TSharedRef<UE::MultiUserClient::Replication::FMultiUserReplicationManager> InReplicationManager,
	TSharedPtr<UE::FileSandboxUI::IExternalSandboxActiveViewModel> InExternalSandboxViewModel
	)
{
	if (!MultiUserClientUtils::HasServerCompatibleCommunicationPluginEnabled())
	{
		// Output a log.
		MultiUserClientUtils::LogNoCompatibleCommunicationPluginEnabled();

		// Show a message in the browser.
		ChildSlot.AttachWidget(SNew(SConcertNoAvailability)
			.Text(MultiUserClientUtils::GetNoCompatibleCommunicationPluginEnabledText()));

		return; // Installing a plug-in implies an editor restart, don't bother initializing the rest.
	}

	WeakConcertSyncClient = InSyncClient;
	WeakReplicationManager = InReplicationManager;
	ExternalSandboxViewModel = InExternalSandboxViewModel;
	
	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = WeakConcertSyncClient.Pin())
	{
		SearchedText = MakeShared<FText>(); // Will keep in memory the session browser search text between join/leave UI transitions.

		IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		check(ConcertClient->IsConfigured());
		
		ConcertClient->OnSessionConnectionChanged().AddSP(this, &SConcertBrowser::HandleSessionConnectionChanged);

		// Attach the panel corresponding the current state.
		AttachChildWidget(ConcertClient->GetSessionConnectionStatus());
	}
}

void SConcertBrowser::HandleSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus ConnectionStatus)
{
	AttachChildWidget(ConnectionStatus);
}

void SConcertBrowser::AttachChildWidget(EConcertConnectionStatus ConnectionStatus)
{
	if (const TSharedPtr<IConcertSyncClient> ConcertSyncClient = WeakConcertSyncClient.Pin()
		; ensure(ConcertSyncClient))
	{
		ActiveSessionWidget.Reset();
		SessionBrowser.Reset();
		
		if (ConnectionStatus == EConcertConnectionStatus::Connected)
		{
			AttachActiveWidget(ConcertSyncClient.ToSharedRef());
		}
		else if (ConnectionStatus == EConcertConnectionStatus::Disconnected)
		{
			AttachBrowserWidget(ConcertSyncClient.ToSharedRef());
		}
	}
}

void SConcertBrowser::AttachActiveWidget(const TSharedRef<IConcertSyncClient>& ConcertSyncClient)
{
	if (const TSharedPtr<UE::MultiUserClient::Replication::FMultiUserReplicationManager> ReplicationManager = WeakReplicationManager.Pin()
		; ensure(ReplicationManager))
	{
		ChildSlot.AttachWidget(
			SAssignNew(ActiveSessionWidget, UE::MultiUserClient::SActiveSessionRoot,
				ConcertSyncClient,
				ReplicationManager.ToSharedRef()
			)
		);
	}
}

void SConcertBrowser::AttachBrowserWidget(const TSharedRef<IConcertSyncClient>& InConcertSyncClient)
{
	const TSharedRef<SConcertClientSessionBrowser> Browser = SAssignNew(SessionBrowser, SConcertClientSessionBrowser,
		InConcertSyncClient->GetConcertClient(),
		SearchedText
		);
	
	const TSharedPtr<UE::FileSandboxUI::IExternalSandboxActiveViewModel> ViewModelPin = ExternalSandboxViewModel.Pin();
	if (!ViewModelPin)
	{
		ChildSlot.AttachWidget(Browser);
		return;
	}
	
	const TSharedRef<SWidget> Overlay = SNew(SOverlay)
		+SOverlay::Slot()
		.ZOrder(0)
		[
			Browser
		]
		+SOverlay::Slot()
		.ZOrder(0)
		[
			UE::FileSandboxUI::MakeExternalSandboxActiveOverlay(ViewModelPin.ToSharedRef())
		];
	
	ChildSlot.AttachWidget(Overlay);
}

#undef LOCTEXT_NAMESPACE
