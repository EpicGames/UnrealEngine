// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "IConcertModule.h"  // Change to use Fwd or Ptr.h?
#include "ConcertMessages.h"

class IConcertClientSession;
class IConcertSyncClient;
class SConcertClientSessionBrowser;
namespace UE::FileSandboxUI { class IExternalSandboxActiveViewModel; }
namespace UE::MultiUserClient { class SActiveSessionRoot; }
namespace UE::MultiUserClient::Replication { class FMultiUserReplicationManager; }

/**
 * Displays the multi-users windows enabling the user to browse active and archived sessions,
 * create new session, archive active sessions, restore archived sessions, join a session and
 * open the settings dialog. Once the user joins a session, the browser displays the SActiveSession
 * widget showing the user status, the session clients and the session history (activity feed).
 */
class SConcertBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertBrowser) { }
	SLATE_END_ARGS();

	/**
	* Constructs the Browser.
	*
	* @param InArgs The Slate argument list.
	* @param InSyncClient The sync client.
	* @param InReplicationManager Used to create replication UI when in an active session
	*/
	void Construct(const FArguments& InArgs,
		TSharedRef<IConcertSyncClient> InSyncClient,
		TSharedRef<UE::MultiUserClient::Replication::FMultiUserReplicationManager> InReplicationManager,
		TSharedPtr<UE::FileSandboxUI::IExternalSandboxActiveViewModel> InExternalSandboxViewModel
		);

	const TWeakPtr<UE::MultiUserClient::SActiveSessionRoot>& GetActiveSessionWidget() const { return ActiveSessionWidget; }
	const TWeakPtr<SConcertClientSessionBrowser>& GetSessionBrowser() const { return SessionBrowser; }

private:

	/** Keeps the sync client interface. */
	TWeakPtr<IConcertSyncClient> WeakConcertSyncClient;
	/** Interacts with the replication system on behalf of Multi-User. */
	TWeakPtr<UE::MultiUserClient::Replication::FMultiUserReplicationManager> WeakReplicationManager;
	/** This view model is used to overlay a widget onto the browser when the engine is already sandboxed. */
	TWeakPtr<UE::FileSandboxUI::IExternalSandboxActiveViewModel> ExternalSandboxViewModel;

	/** Only valid when in session. */
	TWeakPtr<UE::MultiUserClient::SActiveSessionRoot> ActiveSessionWidget;
	/** Only valid when not in session. */
	TWeakPtr<SConcertClientSessionBrowser> SessionBrowser;

	/** Keeps the session browser searched text in memory to reapply it when a user leaves a session and goes back to the session browser. */
	TSharedPtr<FText> SearchedText;
	
	/** Invoked when the session connection state is changed. */
	void HandleSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus ConnectionStatus);

	/** Attaches the child widgets according to the connection status. */
	void AttachChildWidget(EConcertConnectionStatus ConnectionStatus);
	void AttachActiveWidget(const TSharedRef<IConcertSyncClient>& InConcertSyncClient);
	void AttachBrowserWidget(const TSharedRef<IConcertSyncClient>& InConcertSyncClient);
};
