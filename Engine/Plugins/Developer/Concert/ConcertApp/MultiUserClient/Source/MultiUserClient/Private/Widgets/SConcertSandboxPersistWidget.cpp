// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertSandboxPersistWidget.h"

#include "Styling/AppStyle.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Algo/Transform.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"

#define LOCTEXT_NAMESPACE "ConcertFrontend.PersistPanel"

// PersistItem Column name
namespace SConcertSandboxPersistWidgetDefs
{
	const FName ColumnID_CheckBoxLabel(TEXT("Checkbox"));
	const FName ColumnID_IconLabel(TEXT("Icon"));
	const FName ColumnID_FileLabel(TEXT("File"));

	const float CheckBoxColumnWidth = 23.0f;
	const float IconColumnWidth = 21.0f;
};

/** Persist Widget Row */
class SConcertSandboxPersistListRow : public SMultiColumnTableRow<TSharedPtr<FConcertPersistItem>>
{
public:
	SLATE_BEGIN_ARGS(SConcertSandboxPersistListRow) {}
		/** The list item for this row */
		SLATE_ARGUMENT(TSharedPtr<FConcertPersistItem>, Item)
	SLATE_END_ARGS()

	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FConcertPersistItem>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnID) override
	{
		const FMargin RowPadding(3, 0, 0, 0);
		TSharedPtr<SWidget> ItemContentWidget;
		if (ColumnID == SConcertSandboxPersistWidgetDefs::ColumnID_CheckBoxLabel)
		{
			ItemContentWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(RowPadding)
				[
					SNew(SCheckBox)
					.IsChecked(Item.Get(), &FConcertPersistItem::GetCheckBoxState)
				.OnCheckStateChanged(Item.Get(), &FConcertPersistItem::SetCheckBoxState)
				];
		}
		else if (ColumnID == SConcertSandboxPersistWidgetDefs::ColumnID_IconLabel)
		{
			ItemContentWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(Item->GetIcon().GetIcon())
					.ToolTipText(Item->GetIconTooltip())
				];
		}
		else if (ColumnID == SConcertSandboxPersistWidgetDefs::ColumnID_FileLabel)
		{
			ItemContentWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(RowPadding)
				[
					SNew(STextBlock)
					.Text(Item->GetDisplayName())
				];
		}

		return ItemContentWidget.ToSharedRef();
	}

private:
	/** The item associated with this row of data */
	TSharedPtr<FConcertPersistItem> Item;
};

void SConcertSandboxPersistWidget::Construct(const FArguments& InArgs)
{
	bDialogConfirmed = false;
	ParentWindow = InArgs._ParentWindow;
	ListViewItems = InArgs._Items;

	SortByColumn = SConcertSandboxPersistWidgetDefs::ColumnID_FileLabel;
	SortMode = EColumnSortMode::Ascending;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5)
			[
				SAssignNew(SubmitDescriptionExpandable, SExpandableArea)
				.InitiallyCollapsed(true)
				.HeaderContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChangeListDesc", "Changelist Description"))
				]
				.BodyContent()
				[
					SNew(SBox)
					.MinDesiredHeight(120)
					.WidthOverride(520)
					[
						SAssignNew(SubmitDescriptionTextCtrl, SMultiLineEditableTextBox )
						.SelectAllTextWhenFocused(true)
						.AutoWrapText(true)
						.IsReadOnly(this, &SConcertSandboxPersistWidget::IsSubmitDescriptionReadOnly)
					]
				]
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(5, 0))
			[
				SNew(SBorder)
				[
					SAssignNew(ListView, SListView<TSharedPtr<FConcertPersistItem>>)
					.ListItemsSource(&ListViewItems)
					.OnGenerateRow(this, &SConcertSandboxPersistWidget::OnGenerateRowForList)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+ SHeaderRow::Column(SConcertSandboxPersistWidgetDefs::ColumnID_CheckBoxLabel)
						.FixedWidth(SConcertSandboxPersistWidgetDefs::CheckBoxColumnWidth)
						[
							SNew(SCheckBox)
							.IsChecked(this, &SConcertSandboxPersistWidget::GetToggleSelectedState)
							.OnCheckStateChanged(this, &SConcertSandboxPersistWidget::OnToggleSelectedCheckBox)
						]
						+ SHeaderRow::Column(SConcertSandboxPersistWidgetDefs::ColumnID_IconLabel)
						.SortMode(this, &SConcertSandboxPersistWidget::GetColumnSortMode, SConcertSandboxPersistWidgetDefs::ColumnID_IconLabel)
						.OnSort(this, &SConcertSandboxPersistWidget::OnColumnSortModeChanged)
						.FixedWidth(SConcertSandboxPersistWidgetDefs::CheckBoxColumnWidth)
						[
							SNew(SSpacer)
						]
						+ SHeaderRow::Column(SConcertSandboxPersistWidgetDefs::ColumnID_FileLabel)
						.DefaultLabel(LOCTEXT("FileColumnLabel", "File"))
						.SortMode(this, &SConcertSandboxPersistWidget::GetColumnSortMode, SConcertSandboxPersistWidgetDefs::ColumnID_FileLabel)
						.OnSort(this, &SConcertSandboxPersistWidget::OnColumnSortModeChanged)
						.FillWidth(7.0f)
					)
					.SelectionMode(ESelectionMode::None)
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(5, 5, 5, 0))
			[
				SNew( SBorder)
				.Visibility(this, &SConcertSandboxPersistWidget::IsWarningPanelVisible)
				.Padding(5)
				[
					SNew( SErrorText )
					.ErrorText(LOCTEXT("ChangeListDescWarning", "Changelist description is required to submit") )
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5)
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)
				+SWrapBox::Slot()
				.Padding(0.0f, 0.0f, 16.0f, 0.0f)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged( this, &SConcertSandboxPersistWidget::OnCheckStateChanged_SubmitToSourceControl)
					.IsChecked( this, &SConcertSandboxPersistWidget::GetSubmitToSourceControl)
					.IsEnabled( this, &SConcertSandboxPersistWidget::CanSubmitToSourceControl)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SubmitToSourceControl", "Submit to Revision Control"))
					]
				]
				+ SWrapBox::Slot()
				.Padding(0.0f, 0.0f, 16.0f, 0.0f)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SConcertSandboxPersistWidget::OnCheckStateChanged_KeepCheckedOut)
					.IsChecked(this, &SConcertSandboxPersistWidget::GetKeepCheckedOut)
					.IsEnabled(this, &SConcertSandboxPersistWidget::CanCheckOut)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("KeepFilesCheckedOut", "Keep Files Checked Out"))
					]
				]
				+ SWrapBox::Slot()
				.Padding(0.0f, 0.0f, 16.0f, 0.0f)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SConcertSandboxPersistWidget::OnCheckStateChanged_MakeWritable)
					.IsChecked(this, &SConcertSandboxPersistWidget::GetMakeWritable)
					.ToolTipText(LOCTEXT("WarningMakeWritable", "Warning this may prevent accurate check-ins with revision control."))
					.IsEnabled_Lambda([this]() { return !CanCheckOut();} )
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MakeFilesWritable", "Make Files Writable"))
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(0.0f,0.0f,0.0f,5.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.IsEnabled(this, &SConcertSandboxPersistWidget::IsOKEnabled)
					.Text(this, &SConcertSandboxPersistWidget::GetOKButtonText )
					.OnClicked(this, &SConcertSandboxPersistWidget::OKClicked)
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text( LOCTEXT("CancelButton", "Cancel") )
					.OnClicked(this, &SConcertSandboxPersistWidget::CancelClicked)
				]
			]
		]
	];
}

FReply SConcertSandboxPersistWidget::OnKeyDown(const FGeometry & MyGeometry, const FKeyEvent & InKeyEvent)
{
	// Pressing escape returns as if the user clicked cancel
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return CancelClicked();
	}

	return FReply::Unhandled();
}

FConcertPersistCommand SConcertSandboxPersistWidget::GetPersistCommand() const
{
	FConcertPersistCommand Cmd;
	Cmd.bShouldMakeFilesWritable = GetMakeWritable() == ECheckBoxState::Checked;
	Cmd.bShouldSubmit = !IsSubmitDescriptionReadOnly();
	Cmd.ChangelistDescription = SubmitDescriptionTextCtrl->GetText();
	Cmd.PackagesToPersist.Reserve(ListViewItems.Num());
	Cmd.FilesToPersist.Reserve(ListViewItems.Num());

	for (const TSharedPtr<FConcertPersistItem>& Item : ListViewItems)
	{
		if (Item->GetCheckBoxState() == ECheckBoxState::Checked)
		{
			Cmd.PackagesToPersist.Add(Item->GetPackageName());
			Cmd.FilesToPersist.Add(Item->GetFilename());
		}
	}
	return Cmd;
}

TSharedRef<ITableRow> SConcertSandboxPersistWidget::OnGenerateRowForList(TSharedPtr<FConcertPersistItem> PersistItemData, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> Row =
		SNew(SConcertSandboxPersistListRow, OwnerTable)
		.Item(PersistItemData)
		.IsEnabled(PersistItemData->IsEnabled());
	return Row;
}

ECheckBoxState SConcertSandboxPersistWidget::GetToggleSelectedState() const
{
	// Default to a Checked state
	ECheckBoxState PendingState = ECheckBoxState::Checked;

	// Iterate through the list of selected items
	for (const auto& Item : ListViewItems)
	{
		if (Item->GetCheckBoxState() == ECheckBoxState::Unchecked)
		{
			// If any item in the list is Unchecked, then represent the entire set of highlighted items as Unchecked,
			// so that the first (user) toggle of ToggleSelectedCheckBox consistently Checks all items
			PendingState = ECheckBoxState::Unchecked;
			break;
		}
	}

	return PendingState;
}

void SConcertSandboxPersistWidget::OnToggleSelectedCheckBox(ECheckBoxState InNewState)
{
	for (const auto& Item : ListViewItems)
	{
		Item->SetCheckBoxState(InNewState);
	}

	ListView->RequestListRefresh();
}

FReply SConcertSandboxPersistWidget::OKClicked()
{
	bDialogConfirmed = true;
	auto ParentWindowPin = ParentWindow.Pin();
	if (ParentWindowPin.IsValid())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SConcertSandboxPersistWidget::CancelClicked()
{
	bDialogConfirmed = false;
	auto ParentWindowPin = ParentWindow.Pin();
	if (ParentWindowPin.IsValid())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

bool SConcertSandboxPersistWidget::IsOKEnabled() const
{
	return IsSubmitDescriptionReadOnly() || !SubmitDescriptionTextCtrl->GetText().IsEmpty();
}

FText SConcertSandboxPersistWidget::GetOKButtonText() const
{
	return IsSubmitDescriptionReadOnly() ? LOCTEXT("OKButtonPersist", "Persist") : LOCTEXT("OKButtonSubmit", "Submit");
}

EVisibility SConcertSandboxPersistWidget::IsWarningPanelVisible() const
{
	return IsOKEnabled() ? EVisibility::Hidden : EVisibility::Visible;
}

bool SConcertSandboxPersistWidget::IsSubmitDescriptionReadOnly() const
{
	return GetSubmitToSourceControl() != ECheckBoxState::Checked;
}

void SConcertSandboxPersistWidget::OnCheckStateChanged_SubmitToSourceControl(ECheckBoxState InState)
{
	SubmitToSourceControl = InState;
	SubmitDescriptionExpandable->SetExpanded_Animated(SubmitToSourceControl == ECheckBoxState::Checked);
}

ECheckBoxState SConcertSandboxPersistWidget::GetSubmitToSourceControl() const
{
	return SubmitToSourceControl;
}

bool SConcertSandboxPersistWidget::CanSubmitToSourceControl() const
{
	return ISourceControlModule::Get().GetProvider().IsEnabled();
}

void SConcertSandboxPersistWidget::OnCheckStateChanged_MakeWritable(ECheckBoxState InState)
{
	MakeWritable = InState;
}

ECheckBoxState SConcertSandboxPersistWidget::GetMakeWritable() const
{
	return MakeWritable;
}

void SConcertSandboxPersistWidget::OnCheckStateChanged_KeepCheckedOut(ECheckBoxState InState)
{
	KeepFilesCheckedOut = InState;
}

ECheckBoxState SConcertSandboxPersistWidget::GetKeepCheckedOut() const
{
	return KeepFilesCheckedOut;
}

bool SConcertSandboxPersistWidget::CanCheckOut() const
{
	return GetSubmitToSourceControl() == ECheckBoxState::Checked && ISourceControlModule::Get().GetProvider().UsesCheckout();
}

EColumnSortMode::Type SConcertSandboxPersistWidget::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void SConcertSandboxPersistWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName & ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;
	RequestSort();
}

void SConcertSandboxPersistWidget::RequestSort()
{
	// Sort the list of root items
	SortTree();
	ListView->RequestListRefresh();
}

void SConcertSandboxPersistWidget::SortTree()
{
	if (SortByColumn == SConcertSandboxPersistWidgetDefs::ColumnID_FileLabel)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			ListViewItems.Sort([](const TSharedPtr<FConcertPersistItem>& A, const TSharedPtr<FConcertPersistItem>& B) {
				return A->GetDisplayName().ToString() < B->GetDisplayName().ToString(); });
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			ListViewItems.Sort([](const TSharedPtr<FConcertPersistItem>& A, const TSharedPtr<FConcertPersistItem>& B) {
				return A->GetDisplayName().ToString() >= B->GetDisplayName().ToString(); });
		}
	}
	else if (SortByColumn == SConcertSandboxPersistWidgetDefs::ColumnID_IconLabel)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			ListViewItems.Sort([](const TSharedPtr<FConcertPersistItem>& A, const TSharedPtr<FConcertPersistItem>& B) {
				return A->GetIconName().ToString() < B->GetIconName().ToString(); });
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			ListViewItems.Sort([](const TSharedPtr<FConcertPersistItem>& A, const TSharedPtr<FConcertPersistItem>& B) {
				return A->GetIconName().ToString() >= B->GetIconName().ToString(); });
		}
	}
}

#undef LOCTEXT_NAMESPACE
