// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "ISourceControlState.h"

class SMultiLineEditableTextBox;
class SExpandableArea;
class FConcertPersistItem;

struct FConcertPersistCommand
{
	TArray<FName> PackagesToPersist;
	TArray<FString> FilesToPersist;
	FText ChangelistDescription;
	bool bShouldSubmit;
	bool bShouldMakeFilesWritable;
};

/**
 * Persist Widget Row Item View Model
 */
class FConcertPersistItem : public TSharedFromThis<FConcertPersistItem>
{
public:
	/** Constructor */
	explicit FConcertPersistItem(FName InPackageName, const FSourceControlStateRef& InItem)
		: Item(InItem)
		, PackageName(MoveTemp(InPackageName))
	{
		CheckBoxState = ECheckBoxState::Checked;
		DisplayName = FText::FromString(Item->GetFilename());
	}

	/** Returns the package name of this persist item */
	FName GetPackageName() const { return PackageName; }

	/** Returns the full path of the item in source control */
	FString GetFilename() const { return Item->GetFilename(); }

	/** Returns the name of the item as displayed in the widget */
	FText GetDisplayName() const { return DisplayName; }

	/** Returns the name of the icon to be used in the list item widget */
	FName GetIconName() const { return Item->GetIcon().GetStyleName(); }
	
	/** Returns the icon to be used in the list item widget */
	FSlateIcon GetIcon() const { return Item->GetIcon(); }

	/** Returns the tooltip text for the icon */
	FText GetIconTooltip() const { return Item->GetDisplayTooltip(); }

	/** Returns the checkbox state of this item */
	ECheckBoxState GetCheckBoxState() const { return CheckBoxState; }

	/** Sets the checkbox state of this item */
	void SetCheckBoxState(ECheckBoxState NewState) { CheckBoxState = NewState; }

	/** true if the item is enabled in the list */
	bool IsEnabled() const { return true; }
private:
	/** Reference to the source control state */
	FSourceControlStateRef Item;

	/** Checkbox state */
	ECheckBoxState CheckBoxState;

	/** Cached name to display in the listview */
	FText DisplayName;

	/** The associated package name for this persist item. */
	FName PackageName;
};

/**
 * Persist sandbox window
 */
class SConcertSandboxPersistWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConcertSandboxPersistWidget) {}
		/** The parent window this widget is hosted in. */
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		/** The file list to display. */
		SLATE_ARGUMENT(TArray<TSharedPtr<FConcertPersistItem>>, Items)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Used to intercept Escape key press, and interpret it as cancel */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Get dialog result */
	bool IsDialogConfirmed() const { return bDialogConfirmed; }

	/** Get the Persist command from the dialog result. */
	FConcertPersistCommand GetPersistCommand() const;

private:
	/** Called by SListView to get a widget corresponding to the supplied item */
	TSharedRef<ITableRow> OnGenerateRowForList(TSharedPtr<FConcertPersistItem> PersistItemData, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * @return the desired toggle state for the ToggleSelectedCheckBox.
	 * Returns Unchecked, unless all of the selected items are Checked.
	 */
	ECheckBoxState GetToggleSelectedState() const;

	/**
	 * Toggles the highlighted items.
	 * If no items are explicitly highlighted, toggles all items in the list.
	 */
	void OnToggleSelectedCheckBox(ECheckBoxState InNewState);

	/** Called when the settings of the dialog are to be accepted*/
	FReply OKClicked();

	/** Called when the settings of the dialog are to be ignored*/
	FReply CancelClicked();

	/** Called to check if the OK button is enabled or not. */
	bool IsOKEnabled() const;

	/** Called to set the text of the OK button. */
	FText GetOKButtonText() const;

	/** Check if the submit warning panel should be visible. */
	EVisibility IsWarningPanelVisible() const;

	/** Check if the submit description should be editable. */
	bool IsSubmitDescriptionReadOnly() const;

	/** Called when the SubmitToSourceControl Checkbox is changed */
	void OnCheckStateChanged_SubmitToSourceControl(ECheckBoxState InState);

	/** Get the current state of the SubmitToSourceControl checkbox  */
	ECheckBoxState GetSubmitToSourceControl() const;

	/** Check if we can actually submit to source control */
	bool CanSubmitToSourceControl() const;

	/** Called when the Keep checked out Checkbox is changed */
	void OnCheckStateChanged_KeepCheckedOut(ECheckBoxState InState);

	/** Get the current state of the Keep Checked Out checkbox  */
	ECheckBoxState GetKeepCheckedOut() const;

	/** Called when the Make Writable checkbox is changed */
	void OnCheckStateChanged_MakeWritable(ECheckBoxState InState);

	/** Get the current state of the Make Writable checkbox  */
	ECheckBoxState GetMakeWritable() const;

	/** Check if Provider can checkout files */
	bool CanCheckOut() const;

	/**
	 * Returns the current column sort mode (ascending or descending) if the ColumnId parameter matches the current
	 * column to be sorted by, otherwise returns EColumnSortMode_None.
	 *
	 * @param	ColumnId	Column ID to query sort mode for.
	 *
	 * @return	The sort mode for the column, or EColumnSortMode_None if it is not known.
	 */
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/**
	 * Callback for SHeaderRow::Column::OnSort, called when the column to sort by is changed.
	 *
	 * @param	ColumnId	The new column to sort by
	 * @param	InSortMode	The sort mode (ascending or descending)
	 */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/**
	 * Requests that the source list data be sorted according to the current sort column and mode,
	 * and refreshes the list view.
	 */
	void RequestSort();

	/**
	 * Sorts the source list data according to the current sort column and mode.
	 */
	void SortTree();

	/** Result confirmation */
	bool bDialogConfirmed;

	/** Collection of objects (Widgets) to display in the List View. */
	TArray<TSharedPtr<FConcertPersistItem>> ListViewItems;

	/** ListBox for selecting which object to consolidate */
	TSharedPtr<SListView<TSharedPtr<FConcertPersistItem>>> ListView;

	/** Pointer to the parent modal window */
	TWeakPtr<SWindow> ParentWindow;

	/** Internal widgets to save having to get in multiple places*/
	TSharedPtr<SExpandableArea> SubmitDescriptionExpandable;
	TSharedPtr<SMultiLineEditableTextBox> SubmitDescriptionTextCtrl;

	/** State of the "Submit to Revision Control" checkbox */
	ECheckBoxState	SubmitToSourceControl;

	/** State of the "Keep Files Checked Out" checkbox */
	ECheckBoxState	KeepFilesCheckedOut;

	/** State of the "Make Writable" checkbox */
	ECheckBoxState	MakeWritable;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;
};
