// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetGetters.h"

#include "ActiveSession/Replication/Client/Multi/SAllClientsView.h"
#include "ActiveSession/Replication/Client/Multi/SMultiClientView.h"
#include "ActiveSession/Replication/Joined/SReplicationJoinedView.h"
#include "ActiveSession/Replication/SReplicationRootWidget.h"
#include "ActiveSession/SActiveSessionRoot.h"
#include "SConcertBrowser.h"

namespace UE::MultiUserClient
{
	TSharedPtr<SActiveSessionRoot> GetActiveSessionWidgetFromBrowser(const SConcertBrowser& Browser)
	{
		return Browser.GetActiveSessionWidget().Pin();
	}
	
	TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor> GetReplicationStreamEditorWidgetFromBrowser(const SConcertBrowser& Browser)
	{
		const TSharedPtr<SActiveSessionRoot> Root = GetActiveSessionWidgetFromBrowser(Browser);
		if (!Root)
		{
			return nullptr;
		}

		const TSharedPtr<Replication::SReplicationJoinedView> ConnectedView = Root->GetReplicationContent()->GetConnectedView().Pin();
		return ConnectedView
			? ConnectedView->GetAllClientsView()->GetClientView()->GetStreamEditor()
			: nullptr;
	}
}
