// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class SConcertBrowser;

namespace UE::ConcertSharedSlate { class IMultiReplicationStreamEditor; }

namespace UE::MultiUserClient
{
	class SActiveSessionRoot;

	DECLARE_DELEGATE_RetVal(TSharedPtr<SConcertBrowser>, FGetConcertBrowserWidget);
	DECLARE_DELEGATE_RetVal(TSharedPtr<SActiveSessionRoot>, FGetActiveSessionWidget);
	DECLARE_DELEGATE_RetVal(TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor>, FGetReplicationStreamEditorWidget);

	/** Traverses the UI tree to get the SActiveSessionRoot. */
	TSharedPtr<SActiveSessionRoot> GetActiveSessionWidgetFromBrowser(const SConcertBrowser& Browser);
	
	/** Traverses the UI tree to get the SActiveSessionRoot. */
	TSharedPtr<ConcertSharedSlate::IMultiReplicationStreamEditor> GetReplicationStreamEditorWidgetFromBrowser(const SConcertBrowser& Browser);
}
