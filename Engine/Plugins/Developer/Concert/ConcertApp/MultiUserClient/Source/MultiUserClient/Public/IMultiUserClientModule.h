// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"



class IConcertSyncClient;

struct FProcHandle;

struct FServerLaunchOverrides
{
	/** The name of the server upon launch */
	FString ServerName;
};

namespace UE::MultiUserClient
{
	class IMultiUserReplication;
}

/**
 * Interface for the Multi-User module.
 */
class IMultiUserClientModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IMultiUserClientModule& Get()
	{
		static const FName ModuleName = "MultiUserClient";
		return FModuleManager::LoadModuleChecked<IMultiUserClientModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "MultiUserClient";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Get the sync client that will performing the MultiUser role
	 * @param InRole The role to create
	 * @return The client
	 */
	virtual TSharedPtr<IConcertSyncClient> GetClient() const = 0;

	/**
	 * Gets the interface for interacting with the replication system that Multi User sets-up.
	 * Ownership remains in MultiUser. You cannot keep any reference to it: Call this function instead.
	 */
	virtual UE::MultiUserClient::IMultiUserReplication* GetReplication() const = 0;

	/**
	 * Invokes the Multi-User browser tab
	 */
	virtual void OpenBrowser() = 0;

	/**
	 * Hot-links to Concert Settings.
	 */
	virtual void OpenSettings() = 0;

	/**
	 * Connect to the default connection setup
	 * @return true if default connection process properly started
	 */
	virtual bool DefaultConnect() = 0;

	/**
	 * Disconnect from the current session if any, but prompt the user about session changes first.
	 * @param bAlwaysAskConfirmation Prompt the user to confirm leaving the session even if they are no changes to persist.
	 * @return true if the session is disconnected, false if the user declined.
	 */
	virtual bool DisconnectSession(bool bAlwaysAskConfirmation = false) = 0;

	/**
	 * Launches a server (if none are running) on the local machine. On success, the server is launched
	 * and its FProcHandle is returned. On failure, an asynchronous notification (banner) is displayed to the user.
	 */
	virtual TOptional<FProcHandle> LaunchConcertServer(TOptional<FServerLaunchOverrides> LaunchOverrides = {}) = 0;

	/**
	 * Shuts down all local servers running. Do nothing if no servers are running.
	 */
	virtual void ShutdownConcertServer() = 0;

	/**
	 * @return true if the Concert server is running on the local machine.
	 */
	virtual bool IsConcertServerRunning() = 0;
};
