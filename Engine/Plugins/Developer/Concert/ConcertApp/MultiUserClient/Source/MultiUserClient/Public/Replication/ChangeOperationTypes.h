// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Messages/ChangeStream.h"

/**
 * This public, high-level Multi-User API for replication  effectively encapsulates the lower-level ConcertSyncCore replication API,
 * @see FConcertReplication_ChangeStream_Request and FConcertReplication_ChangeAuthority_Request.
 *
 * The main intentions behind the encapsulation are:
 *	1. on an architectural level, the users of MU API should not care be made to care about the internal replication protocols
 *	2. on a functional level, MU internally registers a single stream per client. FConcertReplication_ChangeStream_Request, etc. expose stream IDs directly.
 *	We want to protect the API users in case anything about the way MU handles stream IDs changes.
 *
 * These types can (unfortunately) not be exposed for Runtime Blueprints directly in this module because it is only Editor.
 * They are wrapped yet again in MultiUserClientLibrary.
 */
namespace UE::MultiUserClient
{
	/** Result of changing a client's stream */
	enum class EChangeStreamOperationResult : uint8
	{
		/** All changes made or none were requested. */
		Success,
		/** No changes were requested to be made */
		NoChanges,

		// Realistic failures
		
		/** The changes were rejected by the server. */
		Rejected,
		/** Failed because the request timed out. */
		Timeout,
		/** The submission progress was cancelled, e.g. because the target client disconnected. */
		Cancelled,
		
		/** Unlikely. ISubmissionWorkflow validation rejected the change. See log. Could be e.g. that the remote client does not allow remote editing. */
		FailedToSendRequest,

		// Bad API usage
		
		/** No submission took place because the local editor is not in any session. */
		NotInSession,
		/** No submission took place because the client ID was invalid.  */
		UnknownClient,
		/** The request was not made on the game thread. */
		NotOnGameThread,

		/** Not an actual return code. Make sure it's always last. */
		Count
	};

	/** Result of changing a client's authority */
	enum class EChangeAuthorityOperationResult : uint8
	{
		/** All changes made or none were requested. */
		Success,
		/** No changes were requested to be made */
		NoChanges,

		// Realistic failures

		/** The changes were rejected by the server. */
		RejectedFullyOrPartially,
		/** Failed because the request timed out. */
		Timeout,
		/** The authority change was not submitted because the stream change was unsuccessful. */
		CancelledDueToStreamUpdate,
		/** The submission progress was cancelled, e.g. because the target client disconnected. */
		Cancelled,
		
		/** Unlikely. ISubmissionWorkflow validation rejected the change. See log. Could be e.g. that the remote client does not allow remote editing. */
		FailedToSendRequest,

		// Bad API usage
		
		/** No submission took place because the local editor is not in any session. */
		NotInSession,
		/** No submission took place because the client ID was invalid.  */
		UnknownClient,
		/** The request was not made on the game thread. */
		NotOnGameThread,

		/** Not an actual return code. Make sure it's always last. */
		Count
	};
	
	enum class EPropertyChangeType : uint8
	{
		/** Replace all assigned properties with the given properties. */
		Put,
		/** Append the given properties, keeping any preexisting properties. */
		Add,
		/** Remove the given properties. */
		Remove,
		
		/** Not an actual parameter. Make sure it's always last. */
		Count
	};
	
	struct FPropertyChange
	{
		/** The properties of the operation */
		TSet<FConcertPropertyChain> Properties;
		/** How to interpret Properties. */
		EPropertyChangeType ChangeType;
	};

	/** Params for changing a client's stream. */
	struct FChangeStreamRequest
	{
		/** Property changes to make to objects. This can be used to add new objects. */
		TMap<UObject*, FPropertyChange> PropertyChanges;

		/** Objects that should be unregistered (they will also stop replicating if added here) */
		TSet<FSoftObjectPath> ObjectsToRemove;

		/** Changes how often objects are supposed to be replicated. */
		FConcertReplication_ChangeStream_Frequency FrequencyChanges;
	};

	/** Params for changing a client's authority (what they're replicating). */
	struct FChangeAuthorityRequest
	{
		/** Objects that should start replicating. The objects must previously have been registered. */
		TSet<FSoftObjectPath> ObjectsToStartReplicating;

		/** Objects that should stop replicating. */
		TSet<FSoftObjectPath> ObjectToStopReplicating;
	};

	/** Params for changing a client's stream or authority. */
	struct FChangeClientReplicationRequest
	{
		/**
		 * Changes to make to the client's registered objects.
		 * Performed before AuthorityChangeRequest. If it fails, AuthorityChangeRequest will also fail.
		 */
		TOptional<FChangeStreamRequest> StreamChangeRequest;
		
		/**
		 * Changes to make to the objects the client is replicating.
		 * Fails if StreamChangeRequest fails.
		 */
		TOptional<FChangeAuthorityRequest> AuthorityChangeRequest;
	};
	
	enum class EChangeObjectFrequencyErrorCode : uint8
	{
		/** The object for which the frequency was being changed was not registered. */
		UnregisteredStream,
		/** The replication rate parameter was rejected (it cannot be 0). */
		InvalidReplicationRate,

		/** Not an actual parameter. Make sure it's always last. */
		Count
	};

	/** Result about changing frequency. */
	struct FChangeClientStreamFrequencyResponse
	{
		/** Errors encountered for specific objects */
		TMap<FSoftObjectPath, EChangeObjectFrequencyErrorCode> ObjectErrors;
		/** Error for changing the default replication frequency. */
		TOptional<EChangeObjectFrequencyErrorCode> DefaultChangeErrorCode;
	};

	/** Explains why a change to an object in the stream was invalid. */
	enum class EPutObjectErrorCode : uint8
	{
		/** Stream that the object referenced was not registered on the server. */
		UnresolvedStream,
		/**
		 * Either PutObject contained no data to update with (ensure either ClassPath or Properties is set),
		 * or it tried to create a new object with insufficient data (make sure ClassPath and Properties are both specified).
		 */
		MissingData,

		/** Not an actual parameter. Make sure it's always last. */
		Count
	};

	/** Result of processing FChangeStreamRequest. */
	struct FChangeClientStreamResponse
	{
		/**
		 * Gives general information about what happened to this request.
		 * All other fields only make sense if ErrorCode == Success.
		 */
		EChangeStreamOperationResult ErrorCode;
		
		/**
		 * Dynamic authority errors.
		 * The change was rejected because
		 * 1. this client is replicating the object already,
		 * 2. another client is also replicating the object,
		 * 3. this change would cause overlapping properties with the other client.
		 * */
		TMap<FSoftObjectPath, FGuid> AuthorityConflicts;
		
		/** Errors made in the format of the request */
		TMap<FSoftObjectPath, EPutObjectErrorCode> SemanticErrors;

		/** Errors made in the way frequency was changed. */
		FChangeClientStreamFrequencyResponse FrequencyErrors;

		/**
		 * The client attempted to register the Multi-User stream but failed in doing so.
		 * This usually indicates an internal error that you cannot do anything about as API user.
		 */
		bool bFailedStreamCreation = false;
	};

	/** Result of processing FChangeAuthorityRequest */
	struct FChangeClientAuthorityResponse
	{
		/**
		 * Gives general information about what happened to this request.
		 * All other fields only make sense if ErrorCode == Success.
		 */
		EChangeAuthorityOperationResult ErrorCode;
		
		/** Objects the client did not get authority over. */
		TSet<FSoftObjectPath> RejectedObjects;
	};

	/** Result of processing a FChangeClientReplicationRequest. */
	struct FChangeClientReplicationResult
	{
		FChangeClientStreamResponse StreamChangeResult;
		FChangeClientAuthorityResponse AuthorityChangeResult;
	};
}

