// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChangeOperationTypes.h"

#include "Async/Future.h"

namespace UE::MultiUserClient
{
	/**
	 * Exposes events that occur as part of changing a client's stream, authority, or both.
	 * This is the result of IMultiUserReplication::EnqueueChanges.
	 *
	 * IClientChangeOperation wraps ISubmissionOperation, which is internal to Multi-User.
	 * This allows the two interface to change independently.
	 */
	class MULTIUSERCLIENT_API IClientChangeOperation
	{
	public:

		/**
		 * Completes when the operation of changing streams has completed.
		 * @note This can be called at most once; subsequent calls result in an unset future.
		 * @note This future can complete on any thread.
		 */
		virtual TFuture<FChangeClientStreamResponse> OnChangeStream() = 0;
		
		/**
		 * Completes when the operation of changing authority has completed.
		 * @note This can be called at most once; subsequent calls result in an unset future.
		 * @note This future can complete on any thread.
		 */
		virtual TFuture<FChangeClientAuthorityResponse> OnChangeAuthority() = 0;
		
		/**
		 * Completes all sub-operations have completed.
		 * @note This can be called at most once; subsequent calls result in an unset future.
		 * @note This future can complete on any thread.
		 */
		virtual TFuture<FChangeClientReplicationResult> OnOperationCompleted() = 0;
		
		virtual ~IClientChangeOperation() = default;
	};
}
