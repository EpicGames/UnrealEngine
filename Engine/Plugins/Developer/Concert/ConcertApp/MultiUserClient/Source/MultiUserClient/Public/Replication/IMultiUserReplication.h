// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"

enum class EBreakBehavior : uint8;

struct FConcertClientInfo;
struct FConcertStreamFrequencySettings;
struct FConcertObjectReplicationMap;
struct FGuid;
template<typename OptionalType> struct TOptional;

namespace UE::MultiUserClient
{
	class IClientChangeOperation;
	class IOfflineReplicationClient;
	class IReplicationDiscoverer;
	
	struct FChangeClientReplicationRequest;
	
	/** Interface for interacting with Multi-User replication, which uses the Concert replication system. */
	class MULTIUSERCLIENT_API IMultiUserReplication
	{
	public:

		/**
		 * @return Gets the last known server map of objects registered for replication for an online or offline client.
		 * 
		 * This server state is regularly polled whilst the local client state should always be in sync.
		 * @note This function must be called from the game thread.
		 * @warning Do not keep any latent reference to the returned pointer. The pointed to memory can be reallocated when the client disconnects.
		 */
		virtual const FConcertObjectReplicationMap* FindReplicationMapForClient(const FGuid& ClientId) const = 0;

		/**
		 * @return Gets the last known server object replication frequencies for an online or offline .
		 * 
		 * This server state is regularly polled whilst the local client state should always be in sync.
		 * @note This function must be called from the game thread.
		 * @warning Do not keep any latent reference to the returned pointer. The pointed to memory can be reallocated when the client disconnects.
		 */
		virtual const FConcertStreamFrequencySettings* FindReplicationFrequenciesForClient(const FGuid& ClientId) const = 0;
		
		/**
		 * @return Whether the local editor instance thinks the client has authority over the properties it has registered to ObjectPath.
		 * @note This function must be called from the game thread.
		 */
		virtual bool IsReplicatingObject(const FGuid& ClientId, const FSoftObjectPath& ObjectPath) const = 0;

		/**
		 * Register a discoverer.
		 *
		 * It is used to automatically configure UObjects for replication when appropriate:
		 * - When an user adds an object via Add Actor button
		 * - When an UObject is added to the world via a transaction (run on the client machine that adds the UObject)
		 * This function must be called from the game thread.
		 * @note This function must be called from the game thread.
		 */
		virtual void RegisterReplicationDiscoverer(TSharedRef<IReplicationDiscoverer> Discoverer) = 0;
		/**
		 * Unregisters a previously registered discoverer. 
		 * @note This function must be called from the game thread.
		 */
		virtual void RemoveReplicationDiscoverer(const TSharedRef<IReplicationDiscoverer>& Discoverer) = 0;

		/**
		 * Enqueues a request for changing a client's stream and authority.
		 * The request is enqueued with the other requests that Multi-User might have ongoing already (like those triggered by the UI).
		 *
		 * A stream is the mapping of objects to properties.
		 * The authority state specifies which of the registered objects should actually be sending data.
		 * The stream change is requested first and is followed by the authority change.
		 *
		 * @param ClientId The client for which to change authority
		 * @param SubmissionParams Once the request is ready to be sent to the server, this attribute is used to generate the change request
		 * @note This function must be called from the game thread.
		 */
		virtual TSharedRef<IClientChangeOperation> EnqueueChanges(const FGuid& ClientId, TAttribute<FChangeClientReplicationRequest> SubmissionParams) = 0;
		
		/******************** Offline client ********************/
		
		/**
		 * Iterates over every user who was previously connected to the current session but is now offline. 
		 * The callback function is invoked for each offline client with the last endpoint ID the user had.
		 * 
		 * Endpoint IDs are associated with the "same" user by matching equal display names and device names.
		 * 
		 * @param Callback The callback function to invoke for each offline client.
		 * 
		 * @warning The client instance is only guaranteed to be valid for the duration of the call - do not keep any reference to it!
		 * @note This function must be called from the game thread.
		 */
		virtual void ForEachOfflineClient(TFunctionRef<EBreakBehavior(const IOfflineReplicationClient&)> Callback) const = 0;
		
		/**
		 * Finds an offline client by an endpoint Id that was associated with the user in the past.
		 *
		 * @return Whether Callback was invoked.
		 *
		 * @warning The client instance is only guaranteed to be valid for the duration of the call - do not keep any reference to it!
		 * @note This function must be called from the game thread.
		 */
		virtual bool FindOfflineClient(const FGuid& ClientId, TFunctionRef<void(const IOfflineReplicationClient&)> Callback) const = 0;
		
		/******************** Events ********************/
		
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnServerStateChanged, const FGuid& /*EndpointId*/);
		/** @return Delegate that triggers when the given client's known server state has changed. */
		virtual FOnServerStateChanged& OnStreamServerStateChanged() = 0;
		/** @return Delegate that triggers when the given client's known server state has changed. */
		virtual FOnServerStateChanged& OnAuthorityServerStateChanged() = 0;

		DECLARE_MULTICAST_DELEGATE(FOnOfflineClientsChanged);
		/** @return Delegate that triggers when the endpoints considered offline have changed. */
		virtual FOnOfflineClientsChanged& OnOfflineClientsChanged() = 0;

		/** @return Delegates that triggers when the content of an offline client has changed. Not called as part of OnOfflineClientsChanged. */
		virtual FOnServerStateChanged& OnOfflineClientContentChanged() = 0;

		virtual ~IMultiUserReplication() = default;
	};
}

