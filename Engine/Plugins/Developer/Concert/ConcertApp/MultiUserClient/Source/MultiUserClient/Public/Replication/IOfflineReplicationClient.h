// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FConcertClientInfo;
struct FConcertBaseStreamInfo;
struct FGuid;

namespace UE::MultiUserClient
{
	/** Interface for querying information about an offline replication client. */
	class IOfflineReplicationClient
	{
	public:

		/** @return The client info this client had when they were last connected. */
		virtual const FConcertClientInfo& GetClientInfo() const = 0;
		/** @return The endpoint id this client had when they were last connected. */
		virtual const FGuid& GetLastAssociatedEndpoint() const = 0;
		/** @return The stream content that is expected to be reclaimed by this client when rejoining the session.  */
		virtual const FConcertBaseStreamInfo& GetPredictedStream() const = 0;

		virtual ~IOfflineReplicationClient() = default;
	};
}
