// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

class UObject;
struct FConcertPropertyChain;

namespace UE::MultiUserClient
{
	class IReplicationDiscoveryContext;

	struct FReplicationDiscoveryParams
	{
		/** The object that was added to the stream and for which to discover additional settings. */
		UObject& ExtendedObject;
		/** The endpoint ID of the client to whose stream the object is registered */
		const FGuid& EndpointId;
		/** Mutate replication settings using this context object. */
		IReplicationDiscoveryContext& Context;
	};
	
	/**
	 * An interface for setting up default replication settings for an object.
	 *
	 * This is used when Multi-User determines that an object should be automatically owned by a client and e.g. wants to set up a common workflow.
	 * For example, if you have a specific actor type for which a user usually wants to replicate the position & rotation, you can use this to set up
	 * the workflow automatically for the user.
	 *
	 * Situations where this is executed:
	 * - When a user adds the object to a replication stream using the "Add Actor" button
	 * - When the user adds an actor to the world while in a session, they might automatically want to replicate it (TODO UE-202194)
	 *
	 * This interface is similar to IStreamExtensionContext but the interfaces serve different purposes.
	 * IStreamExtensionContext concerns itself with extending general streams. IReplicationDiscoveryContext is specific to Multi User.
	 * For example, this interface exposes endpoint IDs for which does not make sense for IStreamExtensionContext.
	 */
	class MULTIUSERCLIENT_API IReplicationDiscoverer
	{
	public:

		/**
		 * Finds replication settings given the ExtendedObject.
		 * Using Context, you can assign properties or add additional objects.
		 */
		virtual void DiscoverReplicationSettings(const FReplicationDiscoveryParams& Params) = 0;

		virtual ~IReplicationDiscoverer() = default;
	};
}
