// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

class UObject;
struct FConcertPropertyChain;

namespace UE::MultiUserClient
{
	/**
	 * An interface for discovering replication settings for an object.
	 *
	 * This interface is very similar to IStreamExtensionContext but the interfaces serve different purposes.
	 * IStreamExtensionContext concerns itself with extending general streams. IReplicationDiscoveryContext is specific to Multi User.
	 * 
	 * @see IReplicationDiscoverer
	 */
	class MULTIUSERCLIENT_API IReplicationDiscoveryContext
	{
	public:

		/**
		 * Extends the stream with the given object, if not yet added, and adds the property to it.
		 * If Object was not yet in the stream, IStreamExtender::ExtendStream will be called for Object.
		 * 
		 * @param Object Object to add, if not yet present, and assign a property to
		 * @param PropertyChain Property chain to associate with the object
		 */
		virtual void AddPropertyTo(UObject& Object, FConcertPropertyChain PropertyChain) = 0;

		/**
		 * Adds an additional object.
		 * 
		 * If Object was not yet in the stream, IStreamExtender::ExtendStream will be called for Object.
		 * 
		 * @param Object The additional object to add
		 */
		virtual void AddAdditionalObject(UObject& Object) = 0;

		virtual ~IReplicationDiscoveryContext() = default;
	};
}
