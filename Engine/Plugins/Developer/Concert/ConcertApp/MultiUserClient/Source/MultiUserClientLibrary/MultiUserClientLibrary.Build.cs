// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MultiUserClientLibrary : ModuleRules
	{
		public MultiUserClientLibrary(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"Core",
					"CoreUObject",
					"Engine",
					
					// Concert
					"ConcertReplicationScripting",
				}
			);

			if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
			{
				PrivateDefinitions.Add("WITH_CONCERT=1");

				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"Concert",
						"ConcertClient",
						"ConcertSyncCore",
						"ConcertTransport",
						"MultiUserClient"
					}
				);

				PrivateIncludePathModuleNames.AddRange(
					new string[]
					{
						"Concert",
						"ConcertSyncCore",
						"ConcertSyncClient"
					}
				);
			}
			else
			{
				PrivateDefinitions.Add("WITH_CONCERT=0");
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"Concert",
						"ConcertTransport"
					}
				);
			}
		}
	}
}
