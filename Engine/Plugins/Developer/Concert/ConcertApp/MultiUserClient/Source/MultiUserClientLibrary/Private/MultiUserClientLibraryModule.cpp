// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/CoreRedirects.h"

#include "Modules/ModuleManager.h"

class FMultiUserClientLibraryModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		TArray<FCoreRedirect> Redirects;

		Redirects.Emplace(ECoreRedirectFlags::Type_Package, TEXT("/Script/ConcertSyncClientLibrary"), TEXT("/Script/MultiUserClientLibrary"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Struct, TEXT("ConcertSyncClientInfo"), TEXT("/Script/MultiUserClientLibrary.MultiUserClientInfo"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Class, TEXT("ConcertSyncClientStatics"), TEXT("/Script/MultiUserClientLibrary.MultiUserClientStatics"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.GetLocalConcertClientInfo"), TEXT("GetLocalMultiUserClientInfo"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.GetConcertClientInfoByName"), TEXT("GetMultiUserClientInfoByName"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.GetRemoteConcertClientInfos"), TEXT("GetRemoteMultiUserClientInfos"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.GetConcertConnectionStatus"), TEXT("GetMultiUserConnectionStatus"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.ConcertJumpToPresence"), TEXT("JumpToMultiUserPresence"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.SetPresenceEnabled"), TEXT("SetMultiUserPresenceEnabled"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.SetPresenceVisibility"), TEXT("SetMultiUserPresenceVisibility"));
		Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("MultiUserClientStatics.PersistSessionChanges"), TEXT("PersistMultiUserSessionChanges"));

		FCoreRedirects::AddRedirectList(Redirects, TEXT("MultiUserClientLibrary"));
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FMultiUserClientLibraryModule, MultiUserClientLibrary);
