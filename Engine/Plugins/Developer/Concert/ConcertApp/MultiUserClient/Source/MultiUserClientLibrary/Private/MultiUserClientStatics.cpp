// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserClientStatics.h"
#include "Logging/LogMacros.h"

#if WITH_CONCERT
#include "Algo/Find.h"

#include "IMultiUserClientModule.h"
#include "IConcertSyncClient.h"
#include "IConcertClient.h"
#include "IConcertSession.h"
#include "IConcertClientPresenceManager.h"
#include "ConcertSettings.h"
#include "ConcertClientSettings.h"
#include "ConcertMessageData.h"
#include "ConcertSyncSessionDatabase.h"
#include "IConcertClientWorkspace.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogMultiUserClient, Log, All);

#define LOCTEXT_NAMESPACE "MultiUserClientStatics"

#if WITH_CONCERT
namespace UE::MultiUserClientLibrary
{
	FMultiUserClientInfo ConvertClientInfo(const FGuid& ClientEndpointId, const FConcertClientInfo& ClientInfo)
	{
		return FMultiUserClientInfo(ClientEndpointId, ClientInfo);
	}
	
	FMultiUserConnectionError ConvertConnectionError(FConcertConnectionError Error)
	{
		FMultiUserConnectionError MUError;
		MUError.ErrorCode = static_cast<EMultiUserConnectionError>(Error.ErrorCode);
		MUError.ErrorMessage = Error.ErrorText;
		return MUError;
	}

	UConcertClientConfig* ModifyClientConfig(const FMultiUserClientConfig& InClientConfig)
	{
		UConcertClientConfig* ClientConfig = GetMutableDefault<UConcertClientConfig>();
		ClientConfig->DefaultServerURL = InClientConfig.DefaultServerURL;
		ClientConfig->DefaultSessionName = InClientConfig.DefaultSessionName;
		ClientConfig->DefaultSessionToRestore = InClientConfig.DefaultSessionToRestore;
		ClientConfig->SourceControlSettings.ValidationMode = static_cast<EConcertSourceValidationMode>(InClientConfig.ValidationMode);
		return ClientConfig;
	}

	EMultiUserConnectionStatus ConvertConnectionStatus(EConcertConnectionStatus ConnectionStatus)
	{
		return static_cast<EMultiUserConnectionStatus>(ConnectionStatus);
	}

	FMultiUserSessionInfo ConvertSessionInfo(const FConcertSessionInfo& InSessionInfo, const FConcertServerInfo& InServerInfo)
	{
		FMultiUserSessionInfo ReturnVal;
		ReturnVal.SessionName = InSessionInfo.SessionName;
		ReturnVal.ServerName = InServerInfo.ServerName;
		ReturnVal.EndpointName = InServerInfo.InstanceInfo.InstanceName;
		ReturnVal.ServerEndpointId = InSessionInfo.ServerInstanceId;
		ReturnVal.bValid = true;
		return ReturnVal;
	}

	EMultiUserClientStatus ConvertClientStatus(EConcertClientStatus Status)
	{
		static_assert(static_cast<int32>(EConcertClientStatus::Count) == 3, "Update this location when modifying EConcertClientStatus");
		static_assert(static_cast<int32>(EConcertClientStatus::Connected) == static_cast<int32>(EMultiUserClientStatus::Connected));
		static_assert(static_cast<int32>(EConcertClientStatus::Disconnected) == static_cast<int32>(EMultiUserClientStatus::Disconnected));
		static_assert(static_cast<int32>(EConcertClientStatus::Updated) == static_cast<int32>(EMultiUserClientStatus::Updated));
		return static_cast<EMultiUserClientStatus>(Status);
	}
} // namespace UE::MultiUserClientLibrary
#endif

#if WITH_CONCERT
FMultiUserClientInfo::FMultiUserClientInfo(const FConcertSessionClientInfo& ClientInfo)
	: FMultiUserClientInfo(ClientInfo.ClientEndpointId, ClientInfo.ClientInfo)
{}

FMultiUserClientInfo::FMultiUserClientInfo(const FGuid& ClientId, const FConcertClientInfo& ClientInfo)
	: ClientEndpointId(ClientId)
	, DisplayName(ClientInfo.DisplayName)
	, AvatarColor(ClientInfo.AvatarColor)
	, Tags(ClientInfo.Tags)
{}
#endif

UMultiUserClientStatics::UMultiUserClientStatics(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMultiUserClientStatics::SetMultiUserPresenceEnabled(const bool IsEnabled)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient();
		if (ConcertSyncClient && ConcertSyncClient->GetPresenceManager())
		{
			ConcertSyncClient->GetPresenceManager()->SetPresenceEnabled(IsEnabled);
		}
	}
#endif
}

void UMultiUserClientStatics::SetMultiUserPresenceVisibility(const FString& Name, bool Visibility, bool PropagateToAll)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient();
		if (ConcertSyncClient && ConcertSyncClient->GetPresenceManager())
		{
			ConcertSyncClient->GetPresenceManager()->SetPresenceVisibility(Name, Visibility, PropagateToAll);
		}
	}
#endif
}

void UMultiUserClientStatics::SetMultiUserPresenceVisibilityById(const FGuid& ClientEndpointId, bool Visibility, bool PropagateToAll /*= false*/)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient();
		if (ConcertSyncClient && ConcertSyncClient->GetPresenceManager())
		{
			ConcertSyncClient->GetPresenceManager()->SetPresenceVisibility(ClientEndpointId, Visibility, PropagateToAll);
		}
	}
#endif
}

FTransform UMultiUserClientStatics::GetMultiUserPresenceTransform(const FGuid& ClientEndpointId)
{
	FTransform PresenceTransform;
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient();
		IConcertClientPresenceManager* PresenceManager = ConcertSyncClient ? ConcertSyncClient->GetPresenceManager() : nullptr;
		if (PresenceManager)
		{
			PresenceTransform = PresenceManager->GetPresenceTransform(ClientEndpointId);
		}
	}
#endif
	return PresenceTransform;
}

void UMultiUserClientStatics::JumpToMultiUserPresence(const FString& OtherUserName, FTransform TransformOffset)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();

			const TSharedPtr<IConcertClientSession> ClientSession = ConcertClient->GetCurrentSession();
			FGuid OtherClientId;

			if (ClientSession.IsValid())
			{
				const TArray<FConcertSessionClientInfo> SessionClients = ClientSession->GetSessionClients();
				for (const FConcertSessionClientInfo& SessionClient : SessionClients)
				{
					if (SessionClient.ClientInfo.DisplayName == OtherUserName)
					{
						OtherClientId = SessionClient.ClientEndpointId;
					}
				}
			}

			if (OtherClientId.IsValid() && ConcertSyncClient->GetPresenceManager())
			{
				ConcertSyncClient->GetPresenceManager()->InitiateJumpToPresence(OtherClientId, TransformOffset);
			}
		}
	}
#endif
}

void UMultiUserClientStatics::UpdateWorkspaceModifiedPackages()
{
	PersistMultiUserSessionChanges();
}

void UMultiUserClientStatics::PersistMultiUserSessionChanges()
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			ConcertSyncClient->PersistAllSessionChanges();
		}
	}
#endif
}

void UMultiUserClientStatics::PersistSpecifiedPackages(const TArray<FName>& PackagesToPersist)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			if (TSharedPtr<IConcertClientWorkspace> Workspace = ConcertSyncClient->GetWorkspace())
			{
				ConcertSyncClient->PersistSpecificChanges({PackagesToPersist});
			}
		}
	}
#endif
}

TArray<FName> UMultiUserClientStatics::GatherSessionChanges(bool bIgnorePersisted)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			if (TSharedPtr<IConcertClientWorkspace> Workspace = ConcertSyncClient->GetWorkspace())
			{
				return Workspace->GatherSessionChanges();
			}
		}
	}
#endif
	return {};
}

UMultiUserClientSyncDatabase* UMultiUserClientStatics::GetConcertSyncDatabase()
{
#if WITH_CONCERT
	static bool bRunOnce = []()
	{
		UE::ConcertSyncCore::SyncDatabase::GetOnPackageSavedDelegate().AddLambda([](const FName& PackageName)
		{
			UMultiUserClientSyncDatabase* Database = GetMutableDefault<UMultiUserClientSyncDatabase>();
			check(Database);
			// Re-broadcast message to blueprints.
			Database->OnPackageSaved.Broadcast(PackageName);
		});
		return true;
	}();

	UMultiUserClientSyncDatabase* SyncDatabase = GetMutableDefault<UMultiUserClientSyncDatabase>();
	if (bRunOnce && SyncDatabase)
	{
		return SyncDatabase;
	}
#endif
	return nullptr;
}

FMultiUserClientInfo UMultiUserClientStatics::GetLocalMultiUserClientInfo()
{
	FMultiUserClientInfo ClientInfo;
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			TSharedPtr<IConcertClientSession> ClientSession = ConcertClient->GetCurrentSession();

			FGuid LocalClientEndpointId = ClientSession ? ClientSession->GetSessionClientEndpointId() : FGuid();
			const FConcertClientInfo& LocalClientInfo = ClientSession ? ClientSession->GetLocalClientInfo() : ConcertClient->GetClientInfo();
			ClientInfo = FMultiUserClientInfo(LocalClientEndpointId, LocalClientInfo);
		}
	}
#endif
	return ClientInfo;
}

FMultiUserSessionInfo UMultiUserClientStatics::GetMultiUserSessionInfo()
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			TSharedPtr<IConcertClientSession> ClientSession = ConcertClient->GetCurrentSession();
			if (ClientSession)
			{
				const FConcertSessionInfo& SessionInfo = ClientSession->GetSessionInfo();
				TArray<FConcertServerInfo> Servers = ConcertClient->GetKnownServers();
				FConcertServerInfo* ServerInfo = Algo::FindByPredicate(Servers, [&SessionInfo](const FConcertServerInfo& InServerInfo)
				{
					return InServerInfo.InstanceInfo.InstanceId == SessionInfo.ServerInstanceId;
				});
				if (ServerInfo)
				{
					return UE::MultiUserClientLibrary::ConvertSessionInfo(SessionInfo, *ServerInfo);
				}
			}
		}
	}
#endif
	return {};
}

bool UMultiUserClientStatics::GetMultiUserClientInfoByName(const FString& ClientName, FMultiUserClientInfo& ClientInfo)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			const TSharedPtr<IConcertClientSession> ClientSession = ConcertClient->GetCurrentSession();

			// We return the first match by name that we find. We expect the user to avoid name collisions in the user names. 
			// @todo: We can change this behavior once Concert has unique client IDs that persist across sessions.
			const FConcertClientInfo& LocalClientInfo = ClientSession ? ClientSession->GetLocalClientInfo() : ConcertClient->GetClientInfo();
			if (ClientName == LocalClientInfo.DisplayName)
			{
				ClientInfo = FMultiUserClientInfo(ClientSession ? ClientSession->GetSessionClientEndpointId() : FGuid(), LocalClientInfo);
				return true;
			}

			if (ClientSession.IsValid())
			{
				const TArray<FConcertSessionClientInfo> SessionClients = ClientSession->GetSessionClients();
				for (const FConcertSessionClientInfo& SessionClient : SessionClients)
				{
					if (SessionClient.ClientInfo.DisplayName == ClientName)
					{
						ClientInfo = FMultiUserClientInfo(SessionClient.ClientEndpointId, SessionClient.ClientInfo);
						return true;
					}
				}
			}
		}
	}
#endif
	return false;
}

bool UMultiUserClientStatics::GetRemoteMultiUserClientInfos(TArray<FMultiUserClientInfo>& ClientInfos)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();

			const TSharedPtr<IConcertClientSession> ClientSession = ConcertClient->GetCurrentSession();
			if (ClientSession.IsValid())
			{
				const TArray<FConcertSessionClientInfo> SessionClients = ClientSession->GetSessionClients();
				for (const FConcertSessionClientInfo& SessionClient : SessionClients)
				{
					ClientInfos.Add(FMultiUserClientInfo(SessionClient.ClientEndpointId, SessionClient.ClientInfo));
				}

				return ClientInfos.Num() > 0;
			}
		}
	}
#endif
	return false;
}

bool UMultiUserClientStatics::ConfigureMultiUserClient(const FMultiUserClientConfig& ClientConfig)
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			ConcertSyncClient->GetConcertClient()->Configure(UE::MultiUserClientLibrary::ModifyClientConfig(ClientConfig));
			return true;
		}
	}
#endif
	return false;
}

bool UMultiUserClientStatics::StartMultiUserDefaultConnection()
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		return IMultiUserClientModule::Get().DefaultConnect();
	}
#endif
	return false;
}

FMultiUserConnectionError UMultiUserClientStatics::GetLastMultiUserConnectionError()
{
	FMultiUserConnectionError LastError;
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			LastError = UE::MultiUserClientLibrary::ConvertConnectionError(ConcertSyncClient->GetConcertClient()->GetLastConnectionError());
		}
	}
#endif
	return LastError;
}

EMultiUserConnectionStatus UMultiUserClientStatics::GetMultiUserConnectionStatusDetail()
{
#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();

			const TSharedPtr<IConcertClientSession> ClientSession = ConcertClient->GetCurrentSession();
			if (ClientSession.IsValid())
			{
				return UE::MultiUserClientLibrary::ConvertConnectionStatus(ClientSession->GetConnectionStatus());
			}
		}
	}
#endif
	return EMultiUserConnectionStatus::Disconnected;;
}

bool UMultiUserClientStatics::GetMultiUserConnectionStatus()
{
	return GetMultiUserConnectionStatusDetail() == EMultiUserConnectionStatus::Connected;
}



#undef LOCTEXT_NAMESPACE
