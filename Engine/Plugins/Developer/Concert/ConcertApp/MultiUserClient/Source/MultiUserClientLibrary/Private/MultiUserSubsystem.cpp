// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserSubsystem.h"

#include "Delegates/IDelegateInstance.h"
#include "Engine/Engine.h"
#include "MultiUserClientStatics.h"
#include "Templates/UniquePtr.h"

#if WITH_CONCERT
#include "ConcertTransportMessages.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSession.h"
#include "IConcertSyncClientModule.h"
#include "ConcertLogGlobal.h"
#include "MultiUserSubsystemMessages.h"
#endif


#if WITH_CONCERT
namespace UE::MultiUserSubsystem::Private
{
struct FConcertManager
{
	FConcertManager()
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			ConcertClient->OnSessionStartup().AddRaw(this, &FConcertManager::Register);
			ConcertClient->OnSessionShutdown().AddRaw(this, &FConcertManager::Unregister);

			if (TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
			{
				Register(ConcertClientSession.ToSharedRef());
			}

			ConcertClient->OnSessionConnectionChanged().AddRaw(this, &FConcertManager::OnSessionConnectionChanged);
		}
	}

	~FConcertManager()
	{
		if (IConcertSyncClientModule::IsAvailable())
		{
			Unregister();
			if (const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
			{
				IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
				ConcertClient->OnSessionStartup().RemoveAll(this);
				ConcertClient->OnSessionShutdown().RemoveAll(this);
			}
		}
	}

	void ConnectToSession()
	{
		if (GEngine)
		{
			UMultiUserSubsystem* SubSystem = GEngine->GetEngineSubsystem<UMultiUserSubsystem>();
			if (SubSystem)
			{
				SubSystem->OnSessionConnected.Broadcast();
			}
		}
	}


	void DisconnectFromSession()
	{
		if (GEngine)
		{
			UMultiUserSubsystem* SubSystem = GEngine->GetEngineSubsystem<UMultiUserSubsystem>();
			if (SubSystem)
			{
				SubSystem->OnSessionDisconnected.Broadcast();
			}
		}
	}

	void OnSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus ConnectionStatus)
	{
		if (ConnectionStatus == EConcertConnectionStatus::Connected)
		{
			DisconnectFromSession();
			ConnectToSession();
		}
		else if (ConnectionStatus == EConcertConnectionStatus::Disconnecting)
		{
			DisconnectFromSession();
		}
	}

	/**
	 * Register a new multi-user session.
	 */
	void Register(TSharedRef<IConcertClientSession> InSession)
	{
		WeakSession = InSession;
		
		InSession->RegisterCustomEventHandler<FConcertBlueprintEvent>(this, &FConcertManager::HandleBlueprintEvent);
		InSession->OnSessionClientChanged().AddRaw(this, &FConcertManager::OnSessionClientChanged);
	}

	/**
	 * Unregister a multi-user session (if one exists)
	 */
	void Unregister()
	{
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			Session->UnregisterCustomEventHandler<FConcertBlueprintEvent>(this);
			Session->OnSessionClientChanged().RemoveAll(this);
		}

		WeakSession.Reset();
	}

	/**
	 * Unregister call that is invoked by Multi-user when it is shutdown.
	 */
	void Unregister(TSharedRef<IConcertClientSession> InSession)
	{
		Unregister();
	}

	void HandleBlueprintEvent(const FConcertSessionContext& Context, const FConcertBlueprintEvent& InEvent)
	{
		check(GEngine);
		UMultiUserSubsystem* SubSystem = GEngine->GetEngineSubsystem<UMultiUserSubsystem>();
		check(SubSystem);

		FStructOnScope StructOnScope;
		if (InEvent.Data.GetPayload(StructOnScope))
		{
			SubSystem->DispatchEvent(InEvent);
		}
	}
	
	void OnSessionClientChanged(IConcertClientSession& Session, EConcertClientStatus Status, const FConcertSessionClientInfo& ClientInfo)
	{
		check(GEngine);
		UMultiUserSubsystem* SubSystem = GEngine->GetEngineSubsystem<UMultiUserSubsystem>();
		check(SubSystem);
		
		SubSystem->OnSessionClientChanged.Broadcast(
			MultiUserClientLibrary::ConvertClientStatus(Status),
			FMultiUserClientInfo(ClientInfo.ClientEndpointId, ClientInfo.ClientInfo)
			);
	}

	/** Weak pointer to the client session with which to send events. May be null or stale. */
	TWeakPtr<IConcertClientSession> WeakSession;
};

TUniquePtr<FConcertManager> MakeConcertManager()
{
	return MakeUnique<FConcertManager>();
}

static TUniquePtr<FConcertManager> ConcertManager;

} // namespace UE::MultiUserSubsystem::Private
#endif

void UMultiUserSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
#if WITH_CONCERT
	using namespace UE::MultiUserSubsystem::Private;
	ConcertManager = MakeConcertManager();
#endif
}

void UMultiUserSubsystem::Deinitialize()
{
#if WITH_CONCERT
	UE::MultiUserSubsystem::Private::ConcertManager.Reset();
#endif
}

void UMultiUserSubsystem::K2_SendCustomEvent(const int32& Message)
{
	// This will never be called, the exec version below will be hit instead
	check(0);
}


DEFINE_FUNCTION(UMultiUserSubsystem::execK2_SendCustomEvent)
{
#if WITH_CONCERT
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* MessagePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	if (ensure((StructProp != nullptr) && (StructProp->Struct != nullptr) && (MessagePtr != nullptr)))
	{
		TSharedPtr<IConcertClientSession> Session = UE::MultiUserSubsystem::Private::ConcertManager->WeakSession.Pin();
		FConcertBlueprintEvent OutEvent;
		OutEvent.Data.SetPayload(StructProp->Struct, MessagePtr);

		P_THIS->SendCustomEvent(OutEvent);
	}
#endif
}

void UMultiUserSubsystem::K2_ExtractEventData(FMultiUserBlueprintEventData& EventData, int32& StructOut)
{
	// This will never be called, the exec version below will be hit instead
	check(0);
}

DEFINE_FUNCTION(UMultiUserSubsystem::execK2_ExtractEventData)
{
#if WITH_CONCERT
	P_GET_STRUCT_REF(FMultiUserBlueprintEventData, EventData);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* StructOutPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	if (StructOutPtr && StructProp && StructProp->Struct)
	{
		if (EventData.SharedEventDataPtr.IsValid())
		{
			EventData.SharedEventDataPtr->Data.GetPayload(StructProp->Struct, StructOutPtr);
		}
	}
#endif
}

bool UMultiUserSubsystem::IsConnectedToSession() const
{
#if WITH_CONCERT
	using namespace UE::MultiUserSubsystem::Private;
	if (ConcertManager.IsValid())
	{
		return ConcertManager->WeakSession.IsValid();
	}
#endif
	return false;
}

bool UMultiUserSubsystem::GetLocalClientId(FGuid& OutClientId) const
{
#if WITH_CONCERT
	using namespace UE::MultiUserSubsystem::Private;
	if (ConcertManager.IsValid() && ConcertManager->WeakSession.IsValid())
	{
		const TSharedPtr<IConcertClientSession> SessionPin = ConcertManager->WeakSession.Pin();
		OutClientId = SessionPin->GetSessionClientEndpointId();
		return true;
	}
#endif
	return false;
}

bool UMultiUserSubsystem::GetRemoteClientIds(TArray<FGuid>& OutRemoteClientIds)
{
#if WITH_CONCERT
	using namespace UE::MultiUserSubsystem::Private;
	if (ConcertManager.IsValid() && ConcertManager->WeakSession.IsValid())
	{
		const TSharedPtr<IConcertClientSession> SessionPin = ConcertManager->WeakSession.Pin();
		OutRemoteClientIds = SessionPin->GetSessionClientEndpointIds();
		OutRemoteClientIds.RemoveSingle(SessionPin->GetSessionClientEndpointId());
		return true;
	}
#endif
	return false;
}

void UMultiUserSubsystem::SendCustomEvent(const FConcertBlueprintEvent& EventData)
{
#if WITH_CONCERT
	using namespace UE::MultiUserSubsystem::Private;
	if (!ConcertManager.IsValid())
	{
		UE_LOGF(LogConcert, Error, "No concert manager initialized. Unable to send Blueprint RPC over Multi-user.");
		return;
	}

	if (!ConcertManager->WeakSession.IsValid())
	{
		UE_LOGF(LogConcert, Warning, "Attempting to send a Multi-user custom event when not in a multi-user session. ");
		return;
	}

	TSharedPtr<IConcertClientSession> Session = ConcertManager->WeakSession.Pin();
	Session->SendCustomEvent(EventData, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
#endif
}

bool UMultiUserSubsystem::RegisterCustomEventHandler(const UStruct* EventType, FCustomEventHandler InEventHandler)
{
#if WITH_CONCERT
	FName EventTypeName = *EventType->GetPathName();
	FCustomEventHandler* Handler = CustomEventHandlerTable.Find(EventTypeName);
	if (Handler != nullptr)
	{
		UE_LOGF(LogConcert, Error, "Multi-user Subsystem: A custom event handler for %ls already exists and a new handler cannot be registered.", *EventTypeName.ToString());
		return false;
	}
	UE_LOGF(LogConcert, Display, "Multi-user Subsystem: Registered custom event handler for %ls messages.", *EventTypeName.ToString());
	CustomEventHandlerTable.Emplace(EventTypeName, MoveTemp(InEventHandler));
	return true;
#else
	return false;
#endif
}

bool UMultiUserSubsystem::UnregisterCustomEventHandler(const UStruct* EventType)
{
#if WITH_CONCERT
	FName EventTypeName = *EventType->GetPathName();
	FCustomEventHandler* Handler = CustomEventHandlerTable.Find(EventTypeName);
	if (Handler == nullptr)
	{
		return false;
	}
	UE_LOGF(LogConcert, Display, "Multi-user Subsystem: Unregistered custom event handler for %ls messages.", *EventTypeName.ToString());
	CustomEventHandlerTable.Remove(EventTypeName);

	return true;
#else
	return false;
#endif
}

void UMultiUserSubsystem::DispatchEvent(const FConcertBlueprintEvent& InEvent)
{
#if WITH_CONCERT
	FCustomEventHandler* Handler = CustomEventHandlerTable.Find(InEvent.Data.PayloadTypeName);
	if (Handler)
	{
		FMultiUserBlueprintEventData BPEvent;
		BPEvent.SharedEventDataPtr = MakeShared<FConcertBlueprintEvent>(InEvent);
		Handler->ExecuteIfBound(BPEvent);
	}
#endif
}
