// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"

#include "MultiUserSubsystemMessages.generated.h"

USTRUCT()
struct FConcertBlueprintEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertSessionSerializedPayload Data;
};
