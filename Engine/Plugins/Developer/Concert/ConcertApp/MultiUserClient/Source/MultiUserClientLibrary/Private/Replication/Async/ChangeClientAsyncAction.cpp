// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Async/ChangeClientAsyncAction.h"

#if WITH_CONCERT
#include "IMultiUserClientModule.h"
#include "Replication/IClientChangeOperation.h"
#include "Replication/IMultiUserReplication.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#endif

UChangeClientAsyncAction* UChangeClientAsyncAction::ChangeClient(const FGuid& ClientId, FMultiUserChangeClientReplicationRequest Request)
{
	UChangeClientAsyncAction* AsyncAction = NewObject<UChangeClientAsyncAction>();
	AsyncAction->ClientId = ClientId;
	AsyncAction->ReplicationRequest = MoveTemp(Request);
	return AsyncAction;
}

void UChangeClientAsyncAction::Activate()
{
	Super::Activate();
	
#if WITH_CONCERT
	UE::MultiUserClient::IMultiUserReplication* MultiUserModule = IMultiUserClientModule::IsAvailable()
		? IMultiUserClientModule::Get().GetReplication()
		: nullptr;
	if (MultiUserModule)
	{
		MultiUserModule->EnqueueChanges(ClientId, UE::MultiUserClientLibrary::Transform(MoveTemp(ReplicationRequest)))
			->OnOperationCompleted().Next([WeakThis = TWeakObjectPtr<UChangeClientAsyncAction>(this)](UE::MultiUserClient::FChangeClientReplicationResult&& Result)
			{
				if (IsInGameThread())
				{
					// As per documentation, IsValid is unsafe to call outside of the game thread
					if (WeakThis.IsValid())
					{
						WeakThis->OnCompleted.Broadcast(UE::MultiUserClientLibrary::Transform(MoveTemp(Result)));
					}
				}
				else
				{
					// Result is on stack... do not reference in latent operation.
					UE::MultiUserClient::FChangeClientReplicationResult ResultCopy = MoveTemp(Result);
					AsyncTask(ENamedThreads::GameThread, [WeakThis, Result = MoveTemp(ResultCopy)]() mutable
					{
						if (WeakThis.IsValid())
						{
							WeakThis->OnCompleted.Broadcast(UE::MultiUserClientLibrary::Transform(MoveTemp(Result)));
						}
					});
				}
			});
	}
	else
#endif
	{
		OnCompleted.Broadcast({ { EMultiUserChangeStreamOperationResult::NotAvailable }, { EMultiUserChangeAuthorityOperationResult::NotAvailable } });
	}
}
