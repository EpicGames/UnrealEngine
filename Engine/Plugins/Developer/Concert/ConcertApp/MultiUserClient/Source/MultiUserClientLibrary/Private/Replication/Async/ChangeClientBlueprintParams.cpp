// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Async/ChangeClientBlueprintParams.h"

namespace UE::MultiUserClientLibrary
{
#if WITH_CONCERT
	EMultiUserChangeStreamOperationResult Transform(MultiUserClient::EChangeStreamOperationResult Data)
	{
		static_assert(static_cast<int32>(MultiUserClient::EChangeStreamOperationResult::Count) == 9, "Update EMultiUserChangeStreamOperationResult to have the equivalent enum entry you added to EChangeStreamOperationResult");
		return static_cast<EMultiUserChangeStreamOperationResult>(Data);
	}
	EMultiUserChangeAuthorityOperationResult Transform(MultiUserClient::EChangeAuthorityOperationResult Data)
	{
		static_assert(static_cast<int32>(MultiUserClient::EChangeAuthorityOperationResult::Count) == 10, "Update EMultiUserChangeStreamOperationResult to have the equivalent enum entry you added to EChangeStreamOperationResult");
		return static_cast<EMultiUserChangeAuthorityOperationResult>(Data);
	}

	EMultiUserChangeFrequencyErrorCode Transform(MultiUserClient::EChangeObjectFrequencyErrorCode Data)
	{
		static_assert(static_cast<int32>(EMultiUserChangeFrequencyErrorCode::UnregisteredStream) == static_cast<int32>(MultiUserClient::EChangeObjectFrequencyErrorCode::UnregisteredStream), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EMultiUserChangeFrequencyErrorCode::InvalidReplicationRate) == static_cast<int32>(MultiUserClient::EChangeObjectFrequencyErrorCode::InvalidReplicationRate), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EMultiUserChangeFrequencyErrorCode::Count) == static_cast<int32>(MultiUserClient::EChangeObjectFrequencyErrorCode::Count), "Update this code when making modification to the enums");
		return static_cast<EMultiUserChangeFrequencyErrorCode>(Data);
	}
	
	EMultiUserPutObjectErrorCode Transform(MultiUserClient::EPutObjectErrorCode Data)
	{
		static_assert(static_cast<int32>(EMultiUserPutObjectErrorCode::UnresolvedStream) == static_cast<int32>(MultiUserClient::EPutObjectErrorCode::UnresolvedStream), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EMultiUserPutObjectErrorCode::MissingData) == static_cast<int32>(MultiUserClient::EPutObjectErrorCode::MissingData), "Update this code when making modification to the enums");
		static_assert(static_cast<int32>(EMultiUserPutObjectErrorCode::Count) == static_cast<int32>(MultiUserClient::EPutObjectErrorCode::Count), "Update this code when making modification to the enums");
		return static_cast<EMultiUserPutObjectErrorCode>(Data);
	}

	FMultiUserChangeClientStreamResponse Transform(MultiUserClient::FChangeClientStreamResponse Data)
	{
		FMultiUserChangeClientStreamResponse Response { Transform(Data.ErrorCode), MoveTemp(Data.AuthorityConflicts) };

		for (const TPair<FSoftObjectPath, MultiUserClient::EPutObjectErrorCode>& Pair : Data.SemanticErrors)
		{
			Response.SemanticErrors.Add(Pair.Key, Transform(Pair.Value));
		}
		
		for (const TPair<FSoftObjectPath, MultiUserClient::EChangeObjectFrequencyErrorCode>& Pair : Data.FrequencyErrors.ObjectErrors)
		{
			Response.FrequencyErrors.ObjectErrors.Add(Pair.Key, Transform(Pair.Value));
		}
		
		if (Data.FrequencyErrors.DefaultChangeErrorCode)
		{
			Response.FrequencyErrors.DefaultChangeErrorCode = Transform(*Data.FrequencyErrors.DefaultChangeErrorCode);
		}
			
		Response.bFailedStreamCreation = Data.bFailedStreamCreation;
		return Response;
	}
	FMultiUserChangeClientAuthorityResponse Transform(MultiUserClient::FChangeClientAuthorityResponse Data)
	{
		return FMultiUserChangeClientAuthorityResponse{ Transform(Data.ErrorCode), MoveTemp(Data.RejectedObjects) };
	}
	FMultiUserChangeClientReplicationResult Transform(MultiUserClient::FChangeClientReplicationResult Data)
	{
		return { Transform(Data.StreamChangeResult), Transform(Data.AuthorityChangeResult) };
	}
	
	MultiUserClient::EPropertyChangeType Transform(EMultiUserPropertyChangeType Data)
	{
		static_assert(static_cast<int32>(MultiUserClient::EPropertyChangeType::Count) == 3, "Update this location if you changed the enum");
		return static_cast<MultiUserClient::EPropertyChangeType>(Data);
	}
	MultiUserClient::FPropertyChange Transform(FMultiUserPropertyChange Data)
	{
		MultiUserClient::FPropertyChange Result { .ChangeType =  Transform(MoveTemp(Data.ChangeType)) };
		
		if (!Data.Properties.IsEmpty())
		{
			Result.Properties.Reserve(Data.Properties.Num());
			for (FConcertPropertyChainWrapper& Wrapper : Data.Properties)
			{
				Result.Properties.Emplace(MoveTemp(Wrapper.PropertyChain));
			}
		}
		
		return Result;
	}
	
	EConcertObjectReplicationMode Transform(EMultiUserObjectReplicationMode Data)
	{
		static_assert(static_cast<int32>(EConcertObjectReplicationMode::Count) == 2, "Update this transform operation and its asserts.");
		static_assert(static_cast<int32>(EConcertObjectReplicationMode::Realtime) == static_cast<int32>(EMultiUserObjectReplicationMode::Realtime));
		static_assert(static_cast<int32>(EConcertObjectReplicationMode::SpecifiedRate) == static_cast<int32>(EMultiUserObjectReplicationMode::SpecifiedRate));
		return static_cast<EConcertObjectReplicationMode>(Data);
	}

	EMultiUserObjectReplicationMode Transform(EConcertObjectReplicationMode Data)
	{
		static_assert(static_cast<int32>(EConcertObjectReplicationMode::Count) == 2, "Update this transform operation and its asserts.");
		static_assert(static_cast<int32>(EConcertObjectReplicationMode::Realtime) == static_cast<int32>(EMultiUserObjectReplicationMode::Realtime));
		static_assert(static_cast<int32>(EConcertObjectReplicationMode::SpecifiedRate) == static_cast<int32>(EMultiUserObjectReplicationMode::SpecifiedRate));
		return static_cast<EMultiUserObjectReplicationMode>(Data);
	}

	FConcertObjectReplicationSettings Transform(const FMultiUserObjectReplicationSettings& Data)
	{
		return { Transform(Data.Mode), Data.ReplicationRate };
	}
	
	FMultiUserObjectReplicationSettings Transform(const FConcertObjectReplicationSettings& Data)
	{
		return { Transform(Data.ReplicationMode), Data.ReplicationRate };
	}
	
	MultiUserClient::FChangeStreamRequest Transform(FMultiUserChangeStreamRequest Data)
	{
		MultiUserClient::FChangeStreamRequest Result { .ObjectsToRemove = MoveTemp(Data.ObjectsToRemove) };
		
		if (!Data.PropertyChanges.IsEmpty())
		{
			Result.PropertyChanges.Reserve(Data.PropertyChanges.Num());
			for (TPair<TObjectPtr<UObject>, FMultiUserPropertyChange>& Change : Data.PropertyChanges)
			{
				Result.PropertyChanges.Emplace(Change.Key, Transform(MoveTemp(Change.Value)));
			}
		}

		FMultiUserFrequencyChangeRequest& InFrequencyChanges = Data.FrequencyChanges;
		FConcertReplication_ChangeStream_Frequency& OutFrequencyChanges = Result.FrequencyChanges;
		if (!InFrequencyChanges.IsEmpty())
		{
			OutFrequencyChanges.Flags = InFrequencyChanges.bChangeDefaults ? EConcertReplicationChangeFrequencyFlags::SetDefaults : EConcertReplicationChangeFrequencyFlags::None;
			OutFrequencyChanges.NewDefaults = Transform(InFrequencyChanges.NewDefaults);
			OutFrequencyChanges.OverridesToRemove = MoveTemp(InFrequencyChanges.OverridesToRemove);
			Algo::Transform(InFrequencyChanges.OverridesToAdd, OutFrequencyChanges.OverridesToAdd, [](const TPair<FSoftObjectPath, FMultiUserObjectReplicationSettings>& Pair)
			{
				return TPair<FSoftObjectPath, FConcertObjectReplicationSettings>{ Pair.Key, Transform(Pair.Value) };
			});
		}
		
		return Result;
	}
	MultiUserClient::FChangeAuthorityRequest Transform(FMultiUserChangeAuthorityRequest Data)
	{
		return { MoveTemp(Data.ObjectsToStartReplicating), MoveTemp(Data.ObjectToStopReplicating) };
	}
	MultiUserClient::FChangeClientReplicationRequest Transform(FMultiUserChangeClientReplicationRequest Data)
	{
		return {
			Data.StreamChangeRequest.IsEmpty() ? TOptional<MultiUserClient::FChangeStreamRequest>{} : Transform(MoveTemp(Data.StreamChangeRequest)),
			Data.AuthorityChangeRequest.IsEmpty() ? TOptional<MultiUserClient::FChangeAuthorityRequest>{} : Transform(MoveTemp(Data.AuthorityChangeRequest)),
		};
	}
#endif
}