// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserReplicationRegistrationContextImpl.h"

#if WITH_CONCERT
#include "ConcertPropertyChainWrapper.h"
#include "Replication/IReplicationDiscoveryContext.h"
#include "Replication/Data/ConcertPropertySelection.h"
#endif

void UMultiUserReplicationRegistrationContextImpl::AddPropertiesToObject(UObject* Object, const TArray<FConcertPropertyChainWrapper>& PropertiesToAdd)
{
#if WITH_CONCERT
	if (Object && ensure(NativeContext))
	{
		for (const FConcertPropertyChainWrapper& Wrapper : PropertiesToAdd)
		{
			FConcertPropertyChain Property = Wrapper.PropertyChain;
			NativeContext->AddPropertyTo(*Object, MoveTemp(Property));
		}
	}
#endif
}

void UMultiUserReplicationRegistrationContextImpl::AddAdditionalObject(UObject* Object)
{
#if WITH_CONCERT
	if (Object && ensure(NativeContext))
	{
		NativeContext->AddAdditionalObject(*Object);
	}
#endif
}
