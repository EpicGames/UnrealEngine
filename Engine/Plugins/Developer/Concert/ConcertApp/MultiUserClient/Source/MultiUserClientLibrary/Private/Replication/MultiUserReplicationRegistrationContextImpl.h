// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IMultiUserReplicationRegistrationContext.h"
#include "UObject/Object.h"
#include "MultiUserReplicationRegistrationContextImpl.generated.h"

namespace UE::MultiUserClient
{
	class IReplicationDiscoveryContext;
}

/**
 * Instantiated once and passed to IMultiUserReplicationRegistration.
 * Needed because TScriptInterface, which is passed via FMultiUserReplicationRegistrationParams, requires an UObject to bind to.
 * 
 * @see FUObjectAdapterReplicationDiscoverer
 */
UCLASS()
class UMultiUserReplicationRegistrationContextImpl
	: public UObject
	, public IMultiUserReplicationRegistrationContext
{
	GENERATED_BODY()
public:

	/** Set by FUObjectAdapterReplicationDiscoverer for the duration of the registation */
	UE::MultiUserClient::IReplicationDiscoveryContext* NativeContext = nullptr;

	//~ Begin IMultiUserReplicationRegistrationContext Interface
	UFUNCTION(BlueprintCallable, Category = "Multi-user")
	virtual void AddPropertiesToObject(UObject* Object, const TArray<FConcertPropertyChainWrapper>& PropertiesToAdd) override;
	UFUNCTION(BlueprintCallable, Category = "Multi-user")
	virtual void AddAdditionalObject(UObject* Object) override;
	//~ End IMultiUserReplicationRegistrationContext Interface
};
