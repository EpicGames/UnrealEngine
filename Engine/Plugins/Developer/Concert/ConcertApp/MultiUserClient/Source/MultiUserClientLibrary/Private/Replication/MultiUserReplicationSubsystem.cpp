// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/MultiUserReplicationSubsystem.h"

#include "Replication/Async/ChangeClientBlueprintParams.h"

#if WITH_CONCERT
#include "Data/MultiUserClientDisplayInfo.h"
#include "IMultiUserClientModule.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Data/ReplicationFrequencySettings.h"
#include "Replication/IMultiUserReplication.h"
#include "Replication/IOfflineReplicationClient.h"
#include "UObjectAdapterReplicationDiscoverer.h"

#include "Algo/Transform.h"
#endif

bool UMultiUserReplicationSubsystem::IsReplicatingObject(const FGuid& ClientId, const FSoftObjectPath& ObjectPath) const
{
#if WITH_CONCERT
	const UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		return ReplicationInterface->IsReplicatingObject(ClientId, ObjectPath);
	}
#endif
	return false;
}

bool UMultiUserReplicationSubsystem::GetObjectReplicationFrequency(const FGuid& ClientId, const FSoftObjectPath& ObjectPath, FMultiUserObjectReplicationSettings& OutFrequency)
{
#if WITH_CONCERT
	const UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		const FConcertObjectReplicationMap* ObjectMap = ReplicationInterface->FindReplicationMapForClient(ClientId);
		if (!ObjectMap || !ObjectMap->HasProperties(ObjectPath))
		{
			return false;
		}
		
		const FConcertStreamFrequencySettings* FrequencySettings = ReplicationInterface->FindReplicationFrequenciesForClient(ClientId);
		if (FrequencySettings)
		{
			OutFrequency = UE::MultiUserClientLibrary::Transform(FrequencySettings->GetSettingsFor(ObjectPath));
			return true;
		}
	}
#endif
	return false;
}

TArray<FConcertPropertyChainWrapper> UMultiUserReplicationSubsystem::GetPropertiesRegisteredToObject(const FGuid& ClientId, const FSoftObjectPath& ObjectPath) const
{
#if WITH_CONCERT  
	const UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		const FConcertObjectReplicationMap* ObjectMap = ReplicationInterface->FindReplicationMapForClient(ClientId);
		if (!ObjectMap)
		{
			return {};
		}

		const FConcertReplicatedObjectInfo* ObjectInfo = ObjectMap->ReplicatedObjects.Find(ObjectPath);
		if (!ObjectInfo)
		{
			return {};
		}
		
		TArray<FConcertPropertyChainWrapper> Result;
		Algo::Transform(ObjectInfo->PropertySelection.ReplicatedProperties, Result, [](const FConcertPropertyChain& PropertyChain)
		{
			return FConcertPropertyChainWrapper{ PropertyChain }; 
		});
		return Result;
	}
#endif
	return {};
}

TArray<FSoftObjectPath> UMultiUserReplicationSubsystem::GetRegisteredObjects(const FGuid& ClientId) const
{
#if WITH_CONCERT
	const UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		const FConcertObjectReplicationMap* ObjectMap = ReplicationInterface->FindReplicationMapForClient(ClientId);
		if (!ObjectMap)
		{
			return {};
		}
		
		TArray<FSoftObjectPath> Result;
		ObjectMap->ReplicatedObjects.GenerateKeyArray(Result);
		return Result;
	}
#endif
	return {};
}

TArray<FSoftObjectPath> UMultiUserReplicationSubsystem::GetReplicatedObjects(const FGuid& ClientId) const
{
#if WITH_CONCERT
	const UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		const FConcertObjectReplicationMap* ObjectMap = ReplicationInterface->FindReplicationMapForClient(ClientId);
		if (!ObjectMap)
		{
			return {};
		}

		TArray<FSoftObjectPath> Result;
		for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : ObjectMap->ReplicatedObjects)
		{
			if (ReplicationInterface->IsReplicatingObject(ClientId, Pair.Key))
			{
				Result.Add(Pair.Key);
			}
		}
		return Result;
	}
#endif
	return {};
}

TArray<FGuid> UMultiUserReplicationSubsystem::GetOwningOfflineClients(const FSoftObjectPath& ObjectPath) const
{
	TArray<FGuid> Result;
	
#if WITH_CONCERT
	UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		ReplicationInterface->ForEachOfflineClient([&ObjectPath, &Result](const UE::MultiUserClient::IOfflineReplicationClient& Client)
		{
			if (Client.GetPredictedStream().ReplicationMap.HasProperties(ObjectPath))
			{
				Result.Emplace(Client.GetLastAssociatedEndpoint());
			}
			return EBreakBehavior::Continue;
		});
	}
#endif

	return Result;
}

TArray<FGuid> UMultiUserReplicationSubsystem::GetOfflineClientIds() const
{
	TArray<FGuid> Clients;
	
#if WITH_CONCERT
	UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		ReplicationInterface->ForEachOfflineClient([&Clients](const UE::MultiUserClient::IOfflineReplicationClient& Client)
		{
			Clients.Add(Client.GetLastAssociatedEndpoint());
			return EBreakBehavior::Continue;
		});
	}
#endif
	
	return Clients;
}

void UMultiUserReplicationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);  

#if WITH_CONCERT
	UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
	if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
	{
		UObjectAdapter = MakeShared<UE::MultiUserClientLibrary::FUObjectAdapterReplicationDiscoverer>();
		ReplicationInterface->RegisterReplicationDiscoverer(UObjectAdapter.ToSharedRef());

		ReplicationInterface->OnStreamServerStateChanged().AddUObject(this, &UMultiUserReplicationSubsystem::BroadcastStreamsChanged);
		ReplicationInterface->OnAuthorityServerStateChanged().AddUObject(this, &UMultiUserReplicationSubsystem::BroadcastAuthorityChanged);
		ReplicationInterface->OnOfflineClientsChanged().AddUObject(this, &UMultiUserReplicationSubsystem::BroadcastOfflineClientsChanged);
		ReplicationInterface->OnOfflineClientContentChanged().AddUObject(this, &UMultiUserReplicationSubsystem::BroadcastOfflineClientContentChanged);
	}
#endif
}

void UMultiUserReplicationSubsystem::Deinitialize()
{
	Super::Deinitialize();

#if WITH_CONCERT
	if (IMultiUserClientModule::IsAvailable())
	{
		UE::MultiUserClient::IMultiUserReplication* ReplicationInterface = IMultiUserClientModule::Get().GetReplication();
		if (ensureMsgf(ReplicationInterface, TEXT("We expected it to always be valid.")))
		{
			ReplicationInterface->RemoveReplicationDiscoverer(UObjectAdapter.ToSharedRef());
			UObjectAdapter.Reset();
			
			ReplicationInterface->OnStreamServerStateChanged().RemoveAll(this);
			ReplicationInterface->OnAuthorityServerStateChanged().RemoveAll(this);
			ReplicationInterface->OnOfflineClientsChanged().RemoveAll(this);
		}
	}
#endif
}
