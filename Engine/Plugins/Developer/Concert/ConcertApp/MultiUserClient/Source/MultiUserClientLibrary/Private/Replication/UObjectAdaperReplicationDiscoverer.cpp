// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_CONCERT
#include "UObjectAdapterReplicationDiscoverer.h"

#include "Replication/IMultiUserReplicationRegistration.h"

#include "CoreGlobals.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Package.h"

namespace UE::MultiUserClientLibrary
{
	FUObjectAdapterReplicationDiscoverer::FUObjectAdapterReplicationDiscoverer()
	{
		Context = NewObject<UMultiUserReplicationRegistrationContextImpl>(GetTransientPackage(), NAME_None, RF_Transient);
	}

	void FUObjectAdapterReplicationDiscoverer::DiscoverReplicationSettings(const MultiUserClient::FReplicationDiscoveryParams& Params)
	{
		if (Params.ExtendedObject.Implements<UMultiUserReplicationRegistration>()
			&& ensureMsgf(Context, TEXT("Was it GC'ed even though it is not supposed to have been?")))
		{
			// Auto-generated Execute_DiscoverReplicationSettings checks whether the UFunction is marked call in editor, which is most likely isn't
			// hence we need to allow script execution outside of PIE temporarily.
			const TGuardValue ScriptExecutionGuard(GAllowActorScriptExecutionInEditor, true);
			// No technical reason to reset but let's just follow good practices and not keep it dangling
			const TGuardValue NativeContextGuard(Context->NativeContext, &Params.Context);
			
			IMultiUserReplicationRegistration::Execute_DiscoverReplicationSettings(&Params.ExtendedObject, { Params.EndpointId, Context });
		}
	}
	
	void FUObjectAdapterReplicationDiscoverer::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(Context);
	}

	FString FUObjectAdapterReplicationDiscoverer::GetReferencerName() const
	{
		return TEXT("FUObjectAdapterReplicationDiscoverer");
	}
}
#endif