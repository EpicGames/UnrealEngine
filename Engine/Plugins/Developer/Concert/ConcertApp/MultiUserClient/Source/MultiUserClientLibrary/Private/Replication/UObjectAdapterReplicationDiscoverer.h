// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_CONCERT

#include "MultiUserReplicationRegistrationContextImpl.h"
#include "Replication/IReplicationDiscoverer.h"
#include "UObject/GCObject.h"

namespace UE::MultiUserClientLibrary
{
	/**
	 * This discovery implementation checks whether the target UObject implements IConcertReplicationRegistration, and calls the equivalent functions on it.
	 * @see IMultiUserReplicationRegistration
	 */
	class FUObjectAdapterReplicationDiscoverer
		: public MultiUserClient::IReplicationDiscoverer
		, public FGCObject
	{
	public:

		FUObjectAdapterReplicationDiscoverer();

		//~ Begin IReplicationDiscovery Interface
		virtual void DiscoverReplicationSettings(const MultiUserClient::FReplicationDiscoveryParams& Params) override;
		//~ End IReplicationDiscovery Interface

		//~ Begin FGCObject Interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;
		//~ End FGCObject Interface

	private:

		/** Passed to IMultiUserReplicationRegistration. We set the IReplicationDiscoveryContext for every call of DiscoverReplicationSettings.  */
		TObjectPtr<UMultiUserReplicationRegistrationContextImpl> Context;
	};
}
#endif
