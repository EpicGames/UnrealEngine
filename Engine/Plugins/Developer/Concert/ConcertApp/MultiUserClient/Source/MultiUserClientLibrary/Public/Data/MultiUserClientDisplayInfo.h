// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ConcertMessageData.h"

#include "MultiUserClientDisplayInfo.generated.h"

/** Blueprint copy of FConcertClientInfo. Can describe offline clients, too. */
USTRUCT(BlueprintType)
struct FMultiUserClientDisplayInfo
{
	GENERATED_BODY()

	/** Holds the display name of the user that owns this instance. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Client Info")
	FString DisplayName;

	FMultiUserClientDisplayInfo() = default;
	explicit FMultiUserClientDisplayInfo(const FConcertClientInfo& ClientInfo)
		: DisplayName(ClientInfo.DisplayName)
	{}
};