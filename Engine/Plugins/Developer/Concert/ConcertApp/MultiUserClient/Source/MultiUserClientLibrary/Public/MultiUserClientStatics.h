// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MultiUserClientStatics.generated.h"

#if WITH_CONCERT
class UConcertClientConfig;
enum class EConcertClientStatus : uint8;
enum class EConcertConnectionStatus : uint8;
struct FConcertServerInfo;
struct FConcertSessionClientInfo;
struct FConcertSessionInfo;
struct FConcertClientInfo;
struct FConcertConnectionError;
#endif

/** Delegate that is invoked when a package is saved. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPackageSavedSignature, FName, PackageName);

/**
 * BP copy of FConcertSessionClientInfo
 * Holds info on a client connected through multi-user
 */
USTRUCT(BlueprintType)
struct FMultiUserClientInfo
{
	GENERATED_BODY()

	/** Holds the display name of the user that owns this instance. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Client Info")
	FGuid ClientEndpointId;

	/** Holds the display name of the user that owns this instance. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Client Info")
	FString DisplayName;

	/** Holds the color of the user avatar in a session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Client Info")
	FLinearColor AvatarColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

	/** Holds an array of tags that can be used for grouping and categorizing. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = "Client Info")
	TArray<FName> Tags;

	FMultiUserClientInfo() = default;
#if WITH_CONCERT
	explicit FMultiUserClientInfo(const FConcertSessionClientInfo& ClientInfo);
	explicit FMultiUserClientInfo(const FGuid& ClientId, const FConcertClientInfo& ClientInfo);
#endif
};

/**
 * BP copy of FConcertSessionInfo
 * Holds info on a connected session.
 */
USTRUCT(BlueprintType)
struct FMultiUserSessionInfo
{
	GENERATED_BODY()

	/** Holds the server endpoint id for this session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Session Info")
	FGuid ServerEndpointId;

	/** Holds the session name current connected. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Session Info")
	FString SessionName;

	/** Holds the server name for the connected session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Session Info")
	FString ServerName;

	/** Holds the resolved endpoint name (HostPC) for the connected session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Session Info")
	FString EndpointName;

	/** Indicates if the current session info is valid. If we are connected to a session this is true, otherwise false. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Session Info")
	bool bValid = false;
};

UCLASS()
class UMultiUserClientSyncDatabase : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(VisibleAnywhere, BlueprintAssignable, Category="Save Package Delegate")
	FOnPackageSavedSignature OnPackageSaved;
};

/**
 * Enum of the known Multi-User connection error, their value needs to match the internally returned error code.
 * @see FConcertConnectionError
 */
UENUM(meta=(ScriptName="MultiUserConnectionErrorType"))
enum class EMultiUserConnectionError
{
	None								= 0,
	Canceled							= 1,
	ConnectionAttemptAborted			= 2,
	ServerNotResponding					= 3,
	ServerError							= 4,
	WorkspaceValidationUnknown			= 100,
	SourceControlValidationUnknown		= 110,
	SourceControlValidationCanceled		= 111,
	SourceControlValidationError		= 112,
	DirtyPackageValidationError			= 113,
};

USTRUCT(BlueprintType)
struct FMultiUserConnectionError
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Connection Error")
	EMultiUserConnectionError ErrorCode = EMultiUserConnectionError::None;

	UPROPERTY(BlueprintReadOnly, Category = "Connection Error")
	FText ErrorMessage;
};

UENUM(BlueprintType)
enum class EMultiUserSourceValidationMode : uint8
{
	/** Source control validation will fail on any changes when connecting to a Multi-User Session. */
	Hard = 0,
	/**
	 * Source control validation will warn and prompt on any changes when connecting to a Multi-User session.
	 * In Memory changes will be hot-reloaded.
	 * Source control changes aren't affected but will be stashed/shelved in the future.
	 */
	 Soft,
	 /** Soft validation mode with auto proceed on prompts. */
	 SoftAutoProceed
};

USTRUCT(BlueprintType)
struct FMultiUserClientConfig
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "Client Settings")
	FString DefaultServerURL;

	UPROPERTY(BlueprintReadWrite, Category = "Client Settings")
	FString DefaultSessionName;

	UPROPERTY(BlueprintReadWrite, Category = "Client Settings")
	FString DefaultSessionToRestore;

	UPROPERTY(BlueprintReadWrite, Category = "Revision Control Settings", meta = (Keywords = "Source Control"))
	EMultiUserSourceValidationMode ValidationMode = EMultiUserSourceValidationMode::Hard;
};

/** Connection status for Multi-User client sessions */
UENUM(BlueprintType)
enum class EMultiUserConnectionStatus : uint8
{
	/** Currently establishing connection to the server session */
	Connecting,
	/** Connection established and alive */
	Connected,
	/** Currently severing connection to the server session gracefully */
	Disconnecting,
	/** Disconnected */
	Disconnected,
};

/** Exposes EConcertClientStatus */
UENUM(BlueprintType)
enum class EMultiUserClientStatus : uint8
{
	/** Client connected */
	Connected,
	/** Client disconnected */
	Disconnected,
	/** Client state updated */
	Updated,
};

namespace UE::MultiUserClientLibrary
{
#if WITH_CONCERT
	UE_DEPRECATED(5.5, "Use FMultiUserClientInfo contructor instead.")
	MULTIUSERCLIENTLIBRARY_API FMultiUserClientInfo ConvertClientInfo(const FGuid& ClientEndpointId, const FConcertClientInfo& ClientInfo);
	MULTIUSERCLIENTLIBRARY_API FMultiUserConnectionError ConvertConnectionError(FConcertConnectionError Error);
	MULTIUSERCLIENTLIBRARY_API UConcertClientConfig* ModifyClientConfig(const FMultiUserClientConfig& InClientConfig);
	MULTIUSERCLIENTLIBRARY_API EMultiUserConnectionStatus ConvertConnectionStatus(EConcertConnectionStatus ConnectionStatus);
	MULTIUSERCLIENTLIBRARY_API FMultiUserSessionInfo ConvertSessionInfo(const FConcertSessionInfo& InSessionInfo, const FConcertServerInfo& InServerInfo);
	MULTIUSERCLIENTLIBRARY_API EMultiUserClientStatus ConvertClientStatus(EConcertClientStatus Status);
#endif
}

UCLASS()
class MULTIUSERCLIENTLIBRARY_API UMultiUserClientStatics : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:

	/** Set whether presence is currently enabled and should be shown (unless hidden by other settings) */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Presence", meta=(DevelopmentOnly, DisplayName = "Set Multi-User Presence Enabled"))
	static void SetMultiUserPresenceEnabled(const bool IsEnabled = true);

	/** Set Presence Actor Visibility by display name */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Presence", meta=(DevelopmentOnly, DisplayName = "Set Multi-User Presence Visibility"))
	static void SetMultiUserPresenceVisibility(const FString& Name, bool Visibility, bool PropagateToAll = false);

	/** Set Presence Actor Visibility by client id */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Presence", meta = (DevelopmentOnly, DisplayName = "Set Multi-User Presence Visibility By Id"))
	static void SetMultiUserPresenceVisibilityById(const FGuid& ClientEndpointId, bool Visibility, bool PropagateToAll = false);

	/** Get the Presence Actor transform for the specified client endpoint id or identity if the client isn't found */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Presence", meta = (DevelopmentOnly, DisplayName = "Get Multi-User Presence Transform"))
	static FTransform GetMultiUserPresenceTransform(const FGuid& ClientEndpointId);

	/** Teleport to another Multi-User user's presence. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Presence", meta = (DevelopmentOnly, DisplayName = "Jump to Multi-User Presence"))
	static void JumpToMultiUserPresence(const FString& OtherUserName, FTransform TransformOffset);

	/** Update Multi-User Workspace Modified Packages to be in sync for source control submission. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Revision Control", meta=(DevelopmentOnly, DeprecatedFunction, DeprecationMessage = "UpdateWorkspaceModifiedPackages is deprecated. Please use PersistMultiUserSessionChanges instead.", Keywords = "Source Control"))
	static void UpdateWorkspaceModifiedPackages();

	/** Persist the session changes and prepare the files for source control submission. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Revision Control", meta=(DevelopmentOnly, DisplayName = "Persist Multi-User Session Changes", Keywords = "Source Control"))
	static void PersistMultiUserSessionChanges();

	/** Persist the specified sessions changes using source control. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Source Control", meta=(DevelopmentOnly, DsiplayName = "Persist Multi-User Sessions Changes (By Package name)"))
	static void PersistSpecifiedPackages(const TArray<FName>& PackagesToPersist);

	/** Get the list of packages that have changed since the last persist unless ignore persisted is false.  */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Source Control", meta=(DevelopmentOnly, DsiplayName = "Get List of Packages that have changed."))
	static TArray<FName> GatherSessionChanges(bool IgnorePersisted = true);

	/** Get the proxy object for the sync database. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Source Control", meta=(DevelopmentOnly, DsiplayName = "Get Multi-user Sync Database."))
	static UMultiUserClientSyncDatabase* GetConcertSyncDatabase();

	/** Get the local ClientInfo. Works when not connected to a session. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DisplayName = "Get Local Multi-User Client Info"))
	static FMultiUserClientInfo GetLocalMultiUserClientInfo();

	/** Get the current SessionInfo. Works when not connected to a session, but returns an empty session info. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DisplayName = "Get Multi-User Session Info"))
	static FMultiUserSessionInfo GetMultiUserSessionInfo();

	/** Get the ClientInfo for any Multi-User participant by name. The local user is found even when not connected to a session. Returns false is no client was found. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DisplayName = "Get Multi-User Client Info by Name"))
	static bool GetMultiUserClientInfoByName(const FString& ClientName, FMultiUserClientInfo& ClientInfo);

	/** Get ClientInfos of current Multi-User participants except for the local user. Returns false is no remote clients were found. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DisplayName = "Get Remote Multi-User Client Infos"))
	static bool GetRemoteMultiUserClientInfos(TArray<FMultiUserClientInfo>& ClientInfos);

	/** Configure the Multi-User client. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DisplayName = "Configure Multi-User Client"))
	static bool ConfigureMultiUserClient(const FMultiUserClientConfig& ClientConfig);

	/** Start the Multi-User default connection process. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DisplayName = "Start Multi-User Default Connection"))
	static bool StartMultiUserDefaultConnection();

	/** Get the last Multi-User connection error that happened, if any */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta = (DevelopmentOnly, DisplayName = "Get Last Multi-User Connection Error"))
	static FMultiUserConnectionError GetLastMultiUserConnectionError();

	/** Get Multi-User connection status. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DisplayName = "Get Multi-User Connection Status Detail"))
	static EMultiUserConnectionStatus GetMultiUserConnectionStatusDetail();

	/** Get Multi-User connection status. */
	UFUNCTION(BlueprintCallable, Category = "Multi-User Client", meta=(DevelopmentOnly, DeprecatedFunction, DeprecationMessage = "'Get Multi-User Connection Status' is deprecated. Please use 'Get Multi-User Connection Status Detail' instead.", DisplayName = "Get Multi-User Connection Status"))
	static bool GetMultiUserConnectionStatus();
};
