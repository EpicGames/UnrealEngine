// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/SharedPointer.h"

#include "MultiUserSubsystem.generated.h"

enum class EMultiUserClientStatus : uint8;
struct FConcertSessionContext;
struct FConcertBlueprintEvent;
struct FMultiUserClientInfo;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnSessionConnected);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnSessionDisconnected);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnSessionClientChanged, EMultiUserClientStatus, Status, const FMultiUserClientInfo&, ClientInfo);

USTRUCT(BlueprintType)
struct FMultiUserBlueprintEventData
{
	GENERATED_BODY();

	TSharedPtr<FConcertBlueprintEvent> SharedEventDataPtr;
};

DECLARE_DYNAMIC_DELEGATE_OneParam(FCustomEventHandler, FMultiUserBlueprintEventData, Data);

UCLASS(BlueprintType, Category = "Multi-user", DisplayName = "MultiUserSubsystem")
class MULTIUSERCLIENTLIBRARY_API UMultiUserSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

public:
	
	/** Invoked when the local editor instance has joined a session. */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnSessionConnected OnSessionConnected;

	/** Invoked when the local editor instanced has left a session. */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnSessionDisconnected OnSessionDisconnected;

	/**
	 * Invoked when information about a client changes while the local editor instance is in a session.
	 * For example: other clients joining and leaving the session.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnSessionClientChanged OnSessionClientChanged;

	/**
	 * Returns true if we are currently in a session.
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi-user")
	bool IsConnectedToSession() const;

	/**
	 * If connected to a session, gets the endpoint identifiers of client corresponding to the local editor instance.
	 * @return Whether OutClientId contains a valid identifiers.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user")
	bool GetLocalClientId(FGuid& OutClientId) const;
	/**
	 * If connected to a session, gets the endpoint identifiers of all remote clients.
	 * @return Whether OutRemoteClientIds contains the remote client endpoint identifiers.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user")
	bool GetRemoteClientIds(TArray<FGuid>& OutRemoteClientIds);

	/**
	 * Send a custom event message over multi-user.  If you a not in an active session, this function does nothing and only
	 * reports and warning to the output log.
	 *
	 * @param EventData			The event data to send. This must be the same type of UScriptStruct expected by the receivers on the registered handler.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Multi-user", meta=(CustomStructureParam="EventData", AllowAbstract="false", DisplayName="Send Custom Event"))
	void K2_SendCustomEvent(const int32& EventData);

	DECLARE_FUNCTION(execK2_SendCustomEvent);

	/**
	 * Extract event message data. Given a MultiUserBlueprintEventData extract the message contents and put it into the desired
	 * struct.  The struct must match the target type.
	 *
	 * @param StructOut	   The a reference to the structure that will receive the data.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Multi-user", meta=(CustomStructureParam="StructOut", AllowAbstract="false", DisplayName="Extract Event Data"))
	void K2_ExtractEventData(UPARAM(ref) FMultiUserBlueprintEventData& EventData, UPARAM(ref) int32& StructOut);

	DECLARE_FUNCTION(execK2_ExtractEventData);

	/**
	 * Register an event handler for the given type. Only one handler per type is allowed. You should register your handler
	 * on session connected and remove the handler when the session disconnects.
	 *
	 * @param EventType   The struct type that we support handling.
	 *
	 * @return true if the type handler was accepted.
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi-user")
	bool RegisterCustomEventHandler(const UStruct* EventType, FCustomEventHandler InEventHandler);

	/**
	 * Unregister an event handler for the given type.
	 *
	 * @param EventType   The struct type that we support handling.
	 *
	 * @return true if the type handler was successfully unregistered.
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi-user")
	bool UnregisterCustomEventHandler(const UStruct* EventType);

	// Public interface to send the event to the registered blueprint handler.
	void DispatchEvent(const FConcertBlueprintEvent& InEvent);
private:
	// Table of registered handlers.
	TMap<FName, FCustomEventHandler> CustomEventHandlerTable;

	// Internal send custom event message.
	void SendCustomEvent(const FConcertBlueprintEvent& EventData);
};
