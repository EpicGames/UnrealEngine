// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintAsyncActionBase.h"
#include "ChangeClientBlueprintParams.h"
#include "ChangeClientAsyncAction.generated.h"

/**
 * Enqueues a request for changing a specific client's stream and / or authority.
 * This class exposes IMultiUserReplication::EnqueueChanges.
 */
UCLASS()
class MULTIUSERCLIENTLIBRARY_API UChangeClientAsyncAction : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()
public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChangeOperationCompleted, const FMultiUserChangeClientReplicationResult&, Response);

	/** Event that triggers when the operation completes. */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnChangeOperationCompleted OnCompleted;
	
	/**
	 * Allows changing of a client's stream and authority.
	 *
	 * A stream is a mapping of objects to properties, i.e. registered objects. 
	 * A client must request to take authority over objects they have registered previously.
	 * If there is no conflict with other clients, the server approves the request and the client starts replicating automatically.
	 *
	 * @param ClientId The client for which this request is to be sent; can be the local or a remote client.
	 * @param Request The request specifying a change to stream, authority, or both.
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi-user", meta = (BlueprintInternalUseOnly = "true", Keywords = "Change Client Stream Authority Ownership Replication"))
	static UChangeClientAsyncAction* ChangeClient(const FGuid& ClientId, FMultiUserChangeClientReplicationRequest Request);

	//~ Begin UBlueprintAsyncActionBase Interface
	virtual void Activate() override;
	//~ End UBlueprintAsyncActionBase Interface

private:

	UPROPERTY(Transient)
	FGuid ClientId; 
	
	UPROPERTY(Transient)
	FMultiUserChangeClientReplicationRequest ReplicationRequest;
};
