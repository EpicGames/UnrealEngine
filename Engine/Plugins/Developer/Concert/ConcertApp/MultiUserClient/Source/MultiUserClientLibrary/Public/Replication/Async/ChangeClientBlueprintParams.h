// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertPropertyChainWrapper.h"
#include "UObject/SoftObjectPath.h"

#if WITH_CONCERT
#include "Replication/ChangeOperationTypes.h"
#endif

#include "ChangeClientBlueprintParams.generated.h"

/** Result of changing a client's stream */
UENUM(BlueprintType)
enum class EMultiUserChangeStreamOperationResult : uint8
{
	/** All changes made or none were requested. */
	Success,
	/** No changes were requested to be made */
	NoChanges,

	// Realistic failures
	
	/** The changes were rejected by the server. */
	Rejected,
	/** Failed because the request timed out. */
	Timeout,
	/** The submission progress was cancelled, e.g. because the target client disconnected. */
	Cancelled,
	
	/** Unlikely. ISubmissionWorkflow validation rejected the change. See log. Could be e.g. that the remote client does not allow remote editing. */
	FailedToSendRequest,

	// Bad API usage
	
	/** No submission took place because the local editor is not in any session. */
	NotInSession,
	/** No submission took place because the client ID was invalid.  */
	UnknownClient,
	/** The request was not made on the game thread. */
	NotOnGameThread,
	/** The MultiUserClient module is not available. Usually when you're in a runtime build but the operation can only be run in editor builds. */
	NotAvailable
};

/** Result of changing a client's authority */
UENUM(BlueprintType)
enum class EMultiUserChangeAuthorityOperationResult : uint8
{
	/** All changes made or none were requested. */
	Success,
	/** No changes were requested to be made */
	NoChanges,

	// Realistic failures

	/** Some of the authority changes were rejected by the server. */
	RejectedFullyOrPartially,
	/** Failed because the request timed out. */
	Timeout,
	/** The authority change was not submitted because the stream change was unsuccessful. */
	CancelledDueToStreamUpdate,
	/** The submission progress was cancelled, e.g. because the target client disconnected. */
	Cancelled,
	
	/** Unlikely. ISubmissionWorkflow validation rejected the change. See log. Could be e.g. that the remote client does not allow remote editing. */
	FailedToSendRequest,

	// Bad API usage
	
	/** No submission took place because the local editor is not in any session. */
	NotInSession,
	/** No submission took place because the client ID was invalid.  */
	UnknownClient,
	/** The request was not made on the game thread. */
	NotOnGameThread,
	/** The MultiUserClient module is not available. Usually when you're in a runtime build but the operation can only be run in editor builds. */
	NotAvailable
};

UENUM(BlueprintType)
enum class EMultiUserPropertyChangeType : uint8
{
	/** Replace all assigned properties with the given properties. */
	Put,
	/** Append the given properties, keeping any preexisting properties. */
	Add,
	/** Remove the given properties. */
	Remove
};

USTRUCT(BlueprintType)
struct FMultiUserPropertyChange
{
	GENERATED_BODY()

	/** The properties for the object. See ChangeType for how they are used */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	TArray<FConcertPropertyChainWrapper> Properties;

	/** How you want Properties to be applied to the object. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	EMultiUserPropertyChangeType ChangeType = EMultiUserPropertyChangeType::Add;
};

UENUM(BlueprintType)
enum class EMultiUserObjectReplicationMode : uint8
{
	/** Replicate at the rate specified at FMultiUserObjectReplicationSettings::ReplicationRate */
	SpecifiedRate,
	/** Replicate the object as often as possible: every tick. */
	Realtime
};

/** Frequency settings for a particular object */
USTRUCT(BlueprintType)
struct FMultiUserObjectReplicationSettings
{
	GENERATED_BODY()

	/** Determines whether to send a replication event every tick (Realtime) or at the specified rate (SpecifiedRate). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	EMultiUserObjectReplicationMode Mode = EMultiUserObjectReplicationMode::SpecifiedRate;

	/** If Mode == SpecifiedRate, then replicate this many times per second. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	uint8 ReplicationRate = 30;
};

/**
 * Params for changing how often objects replicate.
 * 
 * All objects default to a global setting that can be changed by setting bChangeDefaults = true and changing NewDefaults to contain the new defaults.
 * Objects can have overrides mutated via OverridesToAdd and OverridesToRemove. Overrides override the default behavior.
 */
USTRUCT(BlueprintType)
struct FMultiUserFrequencyChangeRequest
{
	GENERATED_BODY()

	/** Objects for which to remove overrides. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	TSet<FSoftObjectPath> OverridesToRemove;

	/** Objects for which to add overrides */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	TMap<FSoftObjectPath, FMultiUserObjectReplicationSettings> OverridesToAdd;

	/** Set new frequency defaults for all objects that do not have any overrides. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	FMultiUserObjectReplicationSettings NewDefaults;

	/** Whether to replace the defaults currently registered on the server with the ones specified in NewDefaults. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	bool bChangeDefaults = false;

	bool IsEmpty() const { return OverridesToRemove.IsEmpty() && OverridesToAdd.IsEmpty() && !bChangeDefaults; }
};

/** Params for changing a client's stream. */
USTRUCT(BlueprintType)
struct FMultiUserChangeStreamRequest
{
	GENERATED_BODY()
	
	/** Property changes to make to objects. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	TMap<TObjectPtr<UObject>, FMultiUserPropertyChange> PropertyChanges;

	/** Objects that should be unregistered (they will also stop replicating if added here) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	TSet<FSoftObjectPath> ObjectsToRemove;

	/** Change how often objects are supposed to be replicated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	FMultiUserFrequencyChangeRequest FrequencyChanges;
	
	bool IsEmpty() const { return PropertyChanges.IsEmpty() && ObjectsToRemove.IsEmpty() && FrequencyChanges.IsEmpty(); }
};

/** Params for changing a client's authority (what they're replicating). */
USTRUCT(BlueprintType)
struct FMultiUserChangeAuthorityRequest
{
	GENERATED_BODY()
	
	/** Objects that should start replicating. The objects must previously have been registered. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	TSet<FSoftObjectPath> ObjectsToStartReplicating;

	/** Objects that should stop replicating. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Multi-user")
	TSet<FSoftObjectPath> ObjectToStopReplicating;

	bool IsEmpty() const { return ObjectsToStartReplicating.IsEmpty() && ObjectToStopReplicating.IsEmpty(); }
};

/** Params for changing a client's stream or authority. */
USTRUCT(BlueprintType)
struct FMultiUserChangeClientReplicationRequest
{
	GENERATED_BODY()
	
	/**
	 * Changes to make to the client's registered objects.
	 * Performed before AuthorityChangeRequest. If it fails, AuthorityChangeRequest will also fail.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Multi-user")
	FMultiUserChangeStreamRequest StreamChangeRequest;
	
	/**
	 * Changes to make to the objects the client is replicating.
	 * Fails if StreamChangeRequest fails.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Multi-user")
	FMultiUserChangeAuthorityRequest AuthorityChangeRequest;
};

UENUM(BlueprintType)
enum class EMultiUserChangeFrequencyErrorCode : uint8
{
	/** The object for which the frequency was being changed was not registered. */
	UnregisteredStream,
	/** The replication rate parameter was rejected (it cannot be 0). */
	InvalidReplicationRate,

	/** Not an actual parameter. Make sure it's always last. */
	Count UMETA(Hidden)
};

/** Result about changing frequency. */
USTRUCT(BlueprintType)
struct FChangeClientStreamFrequencyResponse
{
	GENERATED_BODY()
	
	/** Errors encountered for specific objects */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	TMap<FSoftObjectPath, EMultiUserChangeFrequencyErrorCode> ObjectErrors;
	
	/** Error for changing the default replication frequency. */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	TOptional<EMultiUserChangeFrequencyErrorCode> DefaultChangeErrorCode;
};

/** Explains why a change to an object in the stream was invalid. */
UENUM(BlueprintType)
enum class EMultiUserPutObjectErrorCode : uint8
{
	/** Stream that the object referenced was not registered on the server. */
	UnresolvedStream,
	/**
	 * Either PutObject contained no data to update with (ensure either ClassPath or Properties is set),
	 * or it tried to create a new object with insufficient data (make sure ClassPath and Properties are both specified).
	 */
	MissingData,

	/** Not an actual parameter. Make sure it's always last. */
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FMultiUserChangeClientStreamResponse
{
	GENERATED_BODY()
	
	/** The error code of changing streams. */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	EMultiUserChangeStreamOperationResult ErrorCode = EMultiUserChangeStreamOperationResult::Timeout;
	
	/**
	 * Dynamic authority errors.
	 * The change was rejected because
	 * 1. this client is replicating the object already,
	 * 2. another client is also replicating the object,
	 * 3. this change would cause overlapping properties with the other client.
	 * */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	TMap<FSoftObjectPath, FGuid> AuthorityConflicts;
		
	/** Errors made in the format of the request */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	TMap<FSoftObjectPath, EMultiUserPutObjectErrorCode> SemanticErrors;

	/** Errors made in the way frequency was changed. */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	FChangeClientStreamFrequencyResponse FrequencyErrors;

	/**
	 * The client attempted to register the Multi-User stream but failed in doing so.
	 * This usually indicates an internal error that you cannot do anything about as API user.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	bool bFailedStreamCreation = false;
};

USTRUCT(BlueprintType)
struct FMultiUserChangeClientAuthorityResponse
{
	GENERATED_BODY()
	
	/** The error code of changing authority. Fails if StreamChangeResult fails. */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	EMultiUserChangeAuthorityOperationResult ErrorCode = EMultiUserChangeAuthorityOperationResult::Timeout;
	
	/** Objects the client did not get authority over. */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	TSet<FSoftObjectPath> RejectedObjects;
};

/** Result of processing a FChangeClientReplicationRequest. */
USTRUCT(BlueprintType)
struct FMultiUserChangeClientReplicationResult
{
	GENERATED_BODY()

	/**
	 * The result of changing streams.
	 * You can inspect this for why the operation failed.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	FMultiUserChangeClientStreamResponse StreamResponse;

	/**
	 * The result of changing authority. Fails if StreamChangeResult fails.
	 * You can inspect this for why the operation failed.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	FMultiUserChangeClientAuthorityResponse AuthorityResponse;
};

namespace UE::MultiUserClientLibrary
{
#if WITH_CONCERT
	MULTIUSERCLIENTLIBRARY_API EMultiUserChangeStreamOperationResult Transform(MultiUserClient::EChangeStreamOperationResult Data);
	MULTIUSERCLIENTLIBRARY_API EMultiUserChangeAuthorityOperationResult Transform(MultiUserClient::EChangeAuthorityOperationResult Data);
	MULTIUSERCLIENTLIBRARY_API EMultiUserChangeFrequencyErrorCode Transform(MultiUserClient::EChangeObjectFrequencyErrorCode Data);
	MULTIUSERCLIENTLIBRARY_API EMultiUserPutObjectErrorCode Transform(MultiUserClient::EPutObjectErrorCode Data);
	
	MULTIUSERCLIENTLIBRARY_API FMultiUserChangeClientStreamResponse Transform(MultiUserClient::FChangeClientStreamResponse Data);
	MULTIUSERCLIENTLIBRARY_API FMultiUserChangeClientAuthorityResponse Transform(MultiUserClient::FChangeClientAuthorityResponse Data);
	MULTIUSERCLIENTLIBRARY_API FMultiUserChangeClientReplicationResult Transform(MultiUserClient::FChangeClientReplicationResult Data);
	
	MULTIUSERCLIENTLIBRARY_API MultiUserClient::EPropertyChangeType Transform(EMultiUserPropertyChangeType Data);
	MULTIUSERCLIENTLIBRARY_API MultiUserClient::FPropertyChange Transform(FMultiUserPropertyChange Data);

	MULTIUSERCLIENTLIBRARY_API EConcertObjectReplicationMode Transform(EMultiUserObjectReplicationMode Data);
	MULTIUSERCLIENTLIBRARY_API EMultiUserObjectReplicationMode Transform(EConcertObjectReplicationMode Data);
	MULTIUSERCLIENTLIBRARY_API FConcertObjectReplicationSettings Transform(const FMultiUserObjectReplicationSettings& Data);
	MULTIUSERCLIENTLIBRARY_API FMultiUserObjectReplicationSettings Transform(const FConcertObjectReplicationSettings& Data);

	MULTIUSERCLIENTLIBRARY_API MultiUserClient::FChangeStreamRequest Transform(FMultiUserChangeStreamRequest Data);
	MULTIUSERCLIENTLIBRARY_API MultiUserClient::FChangeAuthorityRequest Transform(FMultiUserChangeAuthorityRequest Data);
	MULTIUSERCLIENTLIBRARY_API MultiUserClient::FChangeClientReplicationRequest Transform(FMultiUserChangeClientReplicationRequest Data);
#endif
}