// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "ConcertPropertyChainWrapper.h"
#include "IMultiUserReplicationRegistrationContext.h"
#include "Misc/Guid.h"

#include "IMultiUserReplicationRegistration.generated.h"

struct FConcertPropertyChainWrapper;

UINTERFACE()
class UMultiUserReplicationRegistration : public UInterface
{
	GENERATED_BODY()
};

USTRUCT(BlueprintType)
struct FMultiUserReplicationRegistrationParams
{
	GENERATED_BODY()

	/** The client for which the objects and properties are being registered. */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	FGuid ClientEndpointId;

	/** Uses this to register properties and add additional objects for discovery. */
	UPROPERTY(BlueprintReadOnly, Category = "Multi-user")
	TScriptInterface<IMultiUserReplicationRegistrationContext> Context;
};

/**
 * Exposes IReplicationDiscoverer to Blueprints.
 * 
 * FUObjectAdapterReplicationDiscoverer is used to adapt this interface to the native IReplicationDiscovery interface.
 * @see FUObjectAdapterReplicationDiscoverer
 */
class MULTIUSERCLIENTLIBRARY_API IMultiUserReplicationRegistration
{
	GENERATED_BODY()
public:
	
	/** Registers properties and additional objects that must be replicated. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Multi-user|Replication")
	void DiscoverReplicationSettings(const FMultiUserReplicationRegistrationParams& Params);
};
