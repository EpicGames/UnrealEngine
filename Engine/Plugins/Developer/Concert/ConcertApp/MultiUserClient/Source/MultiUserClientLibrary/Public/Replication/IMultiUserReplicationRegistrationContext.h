// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IMultiUserReplicationRegistrationContext.generated.h"

struct FConcertPropertyChainWrapper;

UINTERFACE(NotBlueprintable)
class UMultiUserReplicationRegistrationContext : public UInterface
{
	GENERATED_BODY()
};

/**
 * Given to IMultiUserReplicationRegistration to register properties and objects.
 * @see IMultiUserReplicationRegistration
 */
class MULTIUSERCLIENTLIBRARY_API IMultiUserReplicationRegistrationContext
{
	GENERATED_BODY()
public:

	/**
	 * Extends the stream with the given object, if not yet added, and adds the properties to it.
	 *
	 * Properties will be automatically discovered for Object, as well.
	 * If the object implements IMultiUserReplicationRegistrationContext, it will be invoked on Object recursively.
	 * 
	 * @param Object Object to add, if not yet present, and assign a property to
	 * @param PropertiesToAdd Property chain to associate with the object
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi-user")
	virtual void AddPropertiesToObject(UObject* Object, const TArray<FConcertPropertyChainWrapper>& PropertiesToAdd) = 0;

	/**
	 * Adds an additional object.
	 * 
	 * Properties will be automatically discovered for Object, as well.
	 * If the object implements IMultiUserReplicationRegistrationContext, it will be invoked on Object recursively.
	 * 
	 * @param Object The additional object to add
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi-user")
	virtual void AddAdditionalObject(UObject* Object) = 0;
};
