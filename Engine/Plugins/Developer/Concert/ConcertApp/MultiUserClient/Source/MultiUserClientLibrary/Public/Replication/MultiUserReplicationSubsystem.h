// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"

#include "ConcertPropertyChainWrapper.h"
#include "Data/MultiUserClientDisplayInfo.h"

#include "MultiUserReplicationSubsystem.generated.h"

struct FMultiUserObjectReplicationSettings;

namespace UE::MultiUserClientLibrary { class FUObjectAdapterReplicationDiscoverer; }

/** Exposes ways to interact with the Multi-user replication system via Blueprints. */
UCLASS()
class MULTIUSERCLIENTLIBRARY_API UMultiUserReplicationSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
public:
	
	// This would be the right place to expose additional MU specific replication functions in the future
	
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnServerStateChanged, const FGuid&, EndpointId);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnOfflineClientsChanged);
	
	/**
	 * @return Whether the client is replicating the object.
	 * @note An object can be registered but not replicated.
	 * @see UMultiUserSubsystem::GetLocalClientId and UMultiUserSubsystem::GetRemoteClientIds
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user")
	bool IsReplicatingObject(const FGuid& ClientId, const FSoftObjectPath& ObjectPath) const;

	/**
	 * @return Whether OutFrequency was modified.
	 * @note An object can be registered but not replicated.
	 * @see UMultiUserSubsystem::GetLocalClientId and UMultiUserSubsystem::GetRemoteClientIds
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user")
	bool GetObjectReplicationFrequency(const FGuid& ClientId, const FSoftObjectPath& ObjectPath, FMultiUserObjectReplicationSettings& OutFrequency);

	/**
	 * @return The properties the client has registered for replication for the object.
	 * @note An object can be registered but not replicated. Use IsReplicatingObject to find out whether the client is replicating the returned properties.
	 * @see UMultiUserSubsystem::GetLocalClientId and UMultiUserSubsystem::GetRemoteClientIds.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user")
	TArray<FConcertPropertyChainWrapper> GetPropertiesRegisteredToObject(const FGuid& ClientId, const FSoftObjectPath& ObjectPath) const;

	/**
	 * Gets the objects the online or offline client has registered with the server.
	 *
	 * Just because an object is returned here, it does not mean that the object is being replicated:
	 * - If ClientId is an offline client, then the object is not being replicated by that client.
	 * - If ClientId is an online client, then the object(s) may only be registered with the server.
	 *
	 * To find out which objects are actually being replicated, use GetReplicatedObjects(), which will always contain GetRegisteredObjects().
	 * 
	 * @param ClientId The client of which to get the registered objects. 
	 * @return The objects the client has
	 * 
	 * @see UMultiUserSubsystem::GetLocalClientId and UMultiUserSubsystem::GetRemoteClientIds
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user")
	TArray<FSoftObjectPath> GetRegisteredObjects(const FGuid& ClientId) const;

	/**
	 * Gets the objects that are currently being replicated by the client.
	 *
	 * @note There is a difference between registered and replicated objects! Objects are registered with the server first and later the client
	 * can attempt to start replicating them. GetReplicatedObjects() will always contain GetRegisteredObjects().
	 * 
	 * @param ClientId The client of which to get the replicated objects. 
	 * @return The objects being replicated by the client
	 *
	 * @see UMultiUserSubsystem::GetLocalClientId and UMultiUserSubsystem::GetRemoteClientIds
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user")
	TArray<FSoftObjectPath> GetReplicatedObjects(const FGuid& ClientId) const;

	/**
	 * A list of offline clients that, upon rejoining a session, will attempt to reclaim properties 
	 * they previously registered for an object, regardless of whether the client left gracefully or due to a crash.
	 * 
	 * By default, when a client disconnects (either gracefully or due to a crash) and later rejoins a session,
	 * the client attempts to re-register the properties it had previously registered for the object.
	 * 
	 * @return A list of client descriptions representing offline clients that will attempt to reclaim properties 
	 * associated with the object when they rejoin.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi-user", meta = (Keywords = "Owning Offline Disconnected  Reclaim Join Rejoin Client Find Stream Registered"))
	TArray<FGuid> GetOwningOfflineClients(const FSoftObjectPath& ObjectPath) const;

	/** @return Whether any offline clients will try to register properties for ObjectPath upon rejoining.*/
	UFUNCTION(BlueprintPure, Category = "Multi-user", meta = (Keywords = "Owning Offline Disconnected  Reclaim Join Rejoin Client Find Stream Registered"))
	bool IsOwnedByOfflineClient(const FSoftObjectPath& ObjectPath) const { return !GetOwningOfflineClients(ObjectPath).IsEmpty(); }

	/** @return The list of offline clients. Each entry is an endpoint ID that a user had in the past. */
	UFUNCTION(BlueprintPure, Category = "Multi-user", meta = (Keywords = "Get Offline Disconnected Clients"))
	TArray<FGuid> GetOfflineClientIds() const;

	//~ Begin USubsystem Interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem Interface

private:
	
	/**
	 * Event triggered when the following changes about a client:
	 * - The registered object to properties bindings
	 * - The registered replication frequency setting of an object
	 */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnServerStateChanged OnClientStreamServerStateChanged;
	
	/** Event triggered a client changes the objects it is replicating. */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnServerStateChanged OnClientAuthorityServerStateChanged;

	/** Event triggered when the local list of offline clients has changed. */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnOfflineClientsChanged OnOfflineClientsChanged;

	/** Event triggered when the content that an offline client will attempt to re-claim changes. */
	UPROPERTY(BlueprintAssignable, Category = "Multi-user")
	FOnServerStateChanged OnOfflineClientContentChanged;

	/**
	 * This is used only for when the adds an object through the Add button in the UI.
	 * 
	 * This allows UObjects, the target being Blueprints, to implement the IConcertReplicationRegistration interface through which MU will use to
	 * auto-add properties when registering an object to a client's replication stream.
	 *
	 * Registered when this subsystem is initialized.
	 */
	TSharedPtr<UE::MultiUserClientLibrary::FUObjectAdapterReplicationDiscoverer> UObjectAdapter;

	void BroadcastStreamsChanged(const FGuid& EndpointId) const { OnClientStreamServerStateChanged.Broadcast(EndpointId); }
	void BroadcastAuthorityChanged(const FGuid& EndpointId) const { OnClientAuthorityServerStateChanged.Broadcast(EndpointId); }
	void BroadcastOfflineClientsChanged() const { OnOfflineClientsChanged.Broadcast(); }
	void BroadcastOfflineClientContentChanged(const FGuid& EndpointId) const { OnOfflineClientContentChanged.Broadcast(EndpointId); }
};
