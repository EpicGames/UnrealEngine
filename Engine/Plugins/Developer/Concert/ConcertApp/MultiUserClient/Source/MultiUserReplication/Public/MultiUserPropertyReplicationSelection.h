// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "MultiUserPropertyReplicationSelection.generated.h"

/**
 * Holds the objects and properties to be replicated.
 *
 * Separate from UMultiUserReplicationStreamAsset so editor UI code editing it can be easily exist independently from
 * UMultiUserReplicationStreamAsset.
 */
UCLASS()
class MULTIUSERREPLICATION_API UMultiUserPropertyReplicationSelection : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY()
	FConcertObjectReplicationMap ReplicationMap;
};
