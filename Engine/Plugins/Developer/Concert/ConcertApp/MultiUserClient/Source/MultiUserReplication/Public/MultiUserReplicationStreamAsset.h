// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Replication/Data/ReplicationStreamDescription.h"
#include "MultiUserPropertyReplicationSelection.h"
#include "MultiUserReplicationStreamAsset.generated.h"


/** Asset describing a stream of replicated data the client sends to the server.  */
UCLASS()
class MULTIUSERREPLICATION_API UMultiUserReplicationStreamAsset : public UObject
{
	GENERATED_BODY()
public:

	/** Unique ID for this stream. CDO has value 0 to not cause issues with delta serialization. */
	UPROPERTY()
	FGuid StreamId;
	
	UPROPERTY(Instanced)
	TObjectPtr<UMultiUserPropertyReplicationSelection> ReplicationList;

	UMultiUserReplicationStreamAsset();

	/** Generates a stream description based on this asset for sending to MU server. */
	FReplicationStreamDescription GenerateDescription() const;
};