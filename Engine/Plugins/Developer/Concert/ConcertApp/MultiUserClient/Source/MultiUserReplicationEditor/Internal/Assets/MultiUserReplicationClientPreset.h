// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Replication/Data/ActorLabelRemapping.h"
#include "Replication/Data/ReplicationStream.h"

#include "MultiUserReplicationClientPreset.generated.h"

/** Stores info about a client's content in a preset. */
USTRUCT()
struct FMultiUserReplicationClientPreset
{
	GENERATED_BODY()

	/** The objects this stream will modify. */
	UPROPERTY()
	FConcertObjectReplicationMap ReplicationMap;
	
	/** The frequency setting the stream has. */
	UPROPERTY()
	FConcertStreamFrequencySettings FrequencySettings;

	/**
	 * For each FSoftObjectPath in ReplicationMap that references an actor this saves its actor label.
	 * 
	 * When the preset is applied, potentially in a new world or different session, this allows rebinding the original FSoftObjectPaths
	 * to FSoftObjectPaths that now point to different objects but that share the same actor label and share the same subobject hierarchy.
	 */
	UPROPERTY()
	FConcertReplicationRemappingData ActorLabelRemappingData;

	/** The FConcertClientInfo::DisplayName of the client. */
	UPROPERTY()
	FString DisplayName;
	/** The FConcertClientInfo::DeviceName of the client. */
	UPROPERTY()
	FString DeviceName;

	FMultiUserReplicationClientPreset() = default;

	FMultiUserReplicationClientPreset(const FString& DisplayName, const FString& DeviceName)
		: DisplayName(DisplayName)
		, DeviceName(DeviceName)
	{}
};
