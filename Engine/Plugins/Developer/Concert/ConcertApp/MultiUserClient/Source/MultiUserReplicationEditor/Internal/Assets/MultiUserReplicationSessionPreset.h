// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Replication/Messages/Muting.h"
#include "MultiUserReplicationSessionPreset.generated.h"

struct FConcertClientInfo;
struct FMultiUserReplicationClientPreset;

USTRUCT()
struct FMultiUserMuteSessionContent
{
	GENERATED_BODY()

	/** The argument to put into FConcertReplication_ChangeMuteState_Request::ObjectsToMute. */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> MutedObjects;

	/** The argument to put into FConcertReplication_ChangeMuteState_Request::ObjectsToUnmute. */
	UPROPERTY()
	TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> UnmutedObjects;

	FMultiUserMuteSessionContent() = default;
	FMultiUserMuteSessionContent(
		TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> MutedObjects,
		TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> UnmutedObjects
		)
		: MutedObjects(MoveTemp(MutedObjects))
		, UnmutedObjects(MoveTemp(UnmutedObjects))
	{}
};

/** Stores per-client replication settings so it can be loaded by a user to quickly set up a session. */
UCLASS()
class MULTIUSERREPLICATIONEDITOR_API UMultiUserReplicationSessionPreset : public UObject
{
	GENERATED_BODY()
public:

	/********** Clients **********/

	/** @return The client preset that matches ClientInfo.DisplayName. If there are multiple, returns the one that matches ClientInfo.DeviceName, as well. */
	const FMultiUserReplicationClientPreset* GetClientContent(const FConcertClientInfo& ClientInfo) const;
	/** @return The client preset that matches both the display and device name. */
	const FMultiUserReplicationClientPreset* GetExactClientContent(const FConcertClientInfo& ClientInfo) const;
	
	/** @return Whether a client that matches ClientInfo.DisplayName. */
	bool ContainsClient(const FConcertClientInfo& ClientInfo) const { return GetClientContent(ClientInfo) != nullptr; }
	/** @return Whether a client that matches both the display and device name is saved in this preset. */
	bool ContainsExactClient(const FConcertClientInfo& ClientInfo) const { return GetExactClientContent(ClientInfo) != nullptr; }
	
	/** Adds a client to the preset if it's not already present. */
	FMultiUserReplicationClientPreset* AddClientIfUnique(const FConcertClientInfo& ClientInfo, const FGuid& StreamId);
	
	const TArray<FMultiUserReplicationClientPreset>& GetClientPresets() const { return ClientPresets; }
	
	/********** Muting **********/
	
	const FMultiUserMuteSessionContent& GetMuteContent() const { return MuteContent; }
	void SetMuteContent(FMultiUserMuteSessionContent Content) { MuteContent = MoveTemp(Content); }
	
private:
	
	UPROPERTY()
	TArray<FMultiUserReplicationClientPreset> ClientPresets;

	UPROPERTY()
	FMultiUserMuteSessionContent MuteContent;
};
