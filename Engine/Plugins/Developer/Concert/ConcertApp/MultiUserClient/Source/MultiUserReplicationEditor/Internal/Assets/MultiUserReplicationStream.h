// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Replication/Data/ReplicationStream.h"
#include "Misc/Attribute.h"
#include "MultiUserReplicationStream.generated.h"

/** Wraps FConcertObjectReplicationMap so its edition can be transacted in the editor and saved in presets. */
UCLASS()
class MULTIUSERREPLICATIONEDITOR_API UMultiUserReplicationStream : public UObject
{
	GENERATED_BODY()
public:

	/** The ID of the stream. Set to 0 for CDO to avoid issues with delta serialization.*/
	UPROPERTY()
	FGuid StreamId;

	/** The objects this stream will modify. */
	UPROPERTY()
	FConcertObjectReplicationMap ReplicationMap;

	UMultiUserReplicationStream();

	/** Util that generates the description of this stream for network requests. */
	FConcertReplicationStream GenerateDescription() const;

	/** Util that returns ReplicationMap. */
	TAttribute<FConcertObjectReplicationMap*> MakeReplicationMapGetterAttribute()
	{
		return TAttribute<FConcertObjectReplicationMap*>::CreateLambda([WeakThis = TWeakObjectPtr<UMultiUserReplicationStream>(this)]()
		{
			return WeakThis.IsValid() ? &WeakThis->ReplicationMap : nullptr;
		});
	}
};
