// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

class ISettingsSection;
struct FAssetCategoryPath;

namespace UE::MultiUserReplicationEditor
{
	class IMultiUserReplicationEditorModule : public IModuleInterface
	{
	public:

		/**
		 * Singleton-like access to this module's interface.  This is just for convenience!
		 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
		 *
		 * @return Returns singleton instance, loading the module on demand if needed
		 */
		static inline IMultiUserReplicationEditorModule& Get()
		{
			static const FName ModuleName = "MultiUserReplicationEditor";
			return FModuleManager::LoadModuleChecked<IMultiUserReplicationEditorModule>(ModuleName);
		}

		/**
		 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
		 *
		 * @return True if the module is loaded and ready to use
		 */
		static inline bool IsAvailable()
		{
			static const FName ModuleName = "MultiUserReplicationEditor";
			return FModuleManager::Get().IsModuleLoaded(ModuleName);
		}
	};
}