// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MultiUserReplicationEditor : ModuleRules
	{
		public MultiUserReplicationEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "MURepEditor";
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"Core",
					"CoreUObject",
					"Engine",
					
					// Concert
					"ConcertClientSharedSlate",
					"ConcertSharedSlate",
					"ConcertSyncCore"
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"AssetDefinition",
					"AssetRegistry",
					"InputCore",
					"Projects",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"ToolWidgets",
					"UnrealEd", 
					
					// Concert
					"Concert",
					"ConcertClient",
					"ConcertSharedSlate",
					"ConcertTransport"
				}
			);
		}
	}
}
