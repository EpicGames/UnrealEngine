// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ReplicationSessionPreset.h"

#include "Assets/MultiUserReplicationSessionPreset.h"
#include "StreamEditor/ReplicationSessionPresetEditor.h"

#define LOCTEXT_NAMESPACE "UAssetDefinition_ReplicationSessionPreset"

FText UAssetDefinition_ReplicationSessionPreset::GetAssetDisplayName() const
{
	return LOCTEXT("AssetName", "Replication Session Preset");
}

FLinearColor UAssetDefinition_ReplicationSessionPreset::GetAssetColor() const
{
	return FLinearColor(4.f, 0.4f, 0.65f);
}

TSoftClassPtr<UObject> UAssetDefinition_ReplicationSessionPreset::GetAssetClass() const
{
	return UMultiUserReplicationSessionPreset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ReplicationSessionPreset::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(LOCTEXT("SessionPreset", "Replication Session Preset")) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_ReplicationSessionPreset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (!AssetEditorSubsystem)
	{
		return EAssetCommandResult::Handled;
	}

	for (UMultiUserReplicationSessionPreset* ClientProfile : OpenArgs.LoadObjects<UMultiUserReplicationSessionPreset>())
	{
		UReplicationSessionPresetEditor* AssetEditor = NewObject<UReplicationSessionPresetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		if (!AssetEditor)
		{
			continue;
		}

		AssetEditor->SetObjectToEdit(ClientProfile);
		AssetEditor->Initialize();
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
