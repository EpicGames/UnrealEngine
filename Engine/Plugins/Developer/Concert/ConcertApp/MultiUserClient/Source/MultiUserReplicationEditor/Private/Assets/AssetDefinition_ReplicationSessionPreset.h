// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_ReplicationSessionPreset.generated.h"

/**
 * 
 */
UCLASS()
class UAssetDefinition_ReplicationSessionPreset : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	
	//~ Begin UAssetDefinition Interface
	virtual FText GetAssetDisplayName() const override; 
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override; 
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	//~ End UAssetDefinition Interface

protected:
	
	//~ Begin UAssetDefinition Interface
	virtual bool CanRegisterStatically() const override
	{
		// TODO UE-196506:
		// The entire replication feature is a MVP created in 5.4.
		// Offline editing is not exposed to protect end-users from relying on it until we figure out workflows & requirements.
		// When implementing offline stream editing adjust the editor this asset definition creates, and remove this.
		return false;
	}
	//~ End UAssetDefinition Interface
};
