// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/MultiUserReplicationSessionPreset.h"

#include "Assets/MultiUserReplicationClientPreset.h"
#include "ConcertLogGlobal.h"
#include "ConcertMessageData.h"

const FMultiUserReplicationClientPreset* UMultiUserReplicationSessionPreset::GetClientContent(const FConcertClientInfo& ClientInfo) const
{
	const auto IsMatch = [&ClientInfo](const FMultiUserReplicationClientPreset& Content)
	{
		return Content.DisplayName == ClientInfo.DisplayName;
	};
	const auto IsPerfectMatch = [&ClientInfo, &IsMatch](const FMultiUserReplicationClientPreset& Content)
	{
		return IsMatch(Content) && Content.DeviceName == ClientInfo.DeviceName;
	};
	
	const FMultiUserReplicationClientPreset* BestMatch = nullptr;
	bool bIsBestMatchPerfect = false;
	for (const FMultiUserReplicationClientPreset& Content : ClientPresets)
	{
		if (IsPerfectMatch(Content))
		{
			UE_CLOGF(bIsBestMatchPerfect, LogConcert, Warning, "Preset %ls contained client (name: %ls, device: %ls) multiple times", *GetPathName(), *ClientInfo.DisplayName, *ClientInfo.DeviceName);
			bIsBestMatchPerfect = true;
			BestMatch = &Content;
		}
		else if (!bIsBestMatchPerfect && IsMatch(Content))
		{
			BestMatch = &Content;
		}
	}
	
	return BestMatch;
}

const FMultiUserReplicationClientPreset* UMultiUserReplicationSessionPreset::GetExactClientContent(const FConcertClientInfo& ClientInfo) const
{
	return ClientPresets.FindByPredicate([&ClientInfo](const FMultiUserReplicationClientPreset& Content)
	{
		return Content.DisplayName == ClientInfo.DisplayName && Content.DeviceName == ClientInfo.DeviceName;
	});
}

FMultiUserReplicationClientPreset* UMultiUserReplicationSessionPreset::AddClientIfUnique(const FConcertClientInfo& ClientInfo, const FGuid& StreamId)
{
	if (ContainsExactClient(ClientInfo))
	{
		return nullptr;
	}
	
	const int32 Index = ClientPresets.Emplace(ClientInfo.DisplayName, ClientInfo.DeviceName);
	return &ClientPresets[Index];
}
