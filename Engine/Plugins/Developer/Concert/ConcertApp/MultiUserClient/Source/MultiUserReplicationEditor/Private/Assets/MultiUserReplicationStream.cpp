// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/MultiUserReplicationStream.h"

#include "Replication/Data/ReplicationStream.h"

UMultiUserReplicationStream::UMultiUserReplicationStream()
{
	// To avoid issues with delta serialization the CDO is always 0
	StreamId = HasAnyFlags(RF_ClassDefaultObject)
		? FGuid{}
		: FGuid::NewGuid();
}

FConcertReplicationStream UMultiUserReplicationStream::GenerateDescription() const
{
	return { StreamId, ReplicationMap };
}
