// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSessionPresetFactory.h"

#include "Assets/MultiUserReplicationSessionPreset.h"

#define LOCTEXT_NAMESPACE "ReplicationSessionPresetFactory"

UReplicationSessionPresetFactory::UReplicationSessionPresetFactory(const FObjectInitializer& ObjectInitializer)
 : Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UMultiUserReplicationSessionPreset::StaticClass();
}

bool UReplicationSessionPresetFactory::CanCreateNew() const
{
	// TODO UE-196506:
	// The entire replication feature is a MVP created in 5.4.
	// Offline editing is not exposed to protect end-users from relying on it until we figure out workflows & requirements.
	return false;
}

FText UReplicationSessionPresetFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Replication Session Preset");
}

bool UReplicationSessionPresetFactory::ConfigureProperties()
{
	return true;
}

UObject* UReplicationSessionPresetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMultiUserReplicationSessionPreset* Asset = NewObject<UMultiUserReplicationSessionPreset>(InParent, Name, Flags);
	// TODO UE-196506:
	ensureMsgf(false, TEXT("Should not be called in MVP"));
	return Asset;
}

#undef LOCTEXT_NAMESPACE