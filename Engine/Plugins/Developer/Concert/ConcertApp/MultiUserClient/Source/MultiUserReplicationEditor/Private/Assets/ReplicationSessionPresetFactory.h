// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "ReplicationSessionPresetFactory.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class UReplicationSessionPresetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual bool CanCreateNew() const override;
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};
