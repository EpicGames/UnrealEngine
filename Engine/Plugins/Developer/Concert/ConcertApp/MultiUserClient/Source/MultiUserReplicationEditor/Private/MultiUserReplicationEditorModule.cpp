// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserReplicationEditorModule.h"

#include "MultiUserReplicationEditorStyle.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FMultiUserReplicationEditorModule"

namespace UE::MultiUserReplicationEditor
{
	void FMultiUserReplicationEditorModule::StartupModule()
	{
		FMultiUserReplicationEditorStyle::Initialize();
	}

	void FMultiUserReplicationEditorModule::ShutdownModule()
	{
		FMultiUserReplicationEditorStyle::Shutdown();
	}
}

IMPLEMENT_MODULE(UE::MultiUserReplicationEditor::FMultiUserReplicationEditorModule, MultiUserReplicationEditor);
#undef LOCTEXT_NAMESPACE