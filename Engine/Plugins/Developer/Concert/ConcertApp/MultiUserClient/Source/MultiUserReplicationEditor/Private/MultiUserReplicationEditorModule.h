// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMultiUserReplicationEditorModule.h"

namespace UE::MultiUserReplicationEditor
{
	class FMultiUserReplicationEditorModule : public IMultiUserReplicationEditorModule
	{
	public:
	
		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface
	};
}
