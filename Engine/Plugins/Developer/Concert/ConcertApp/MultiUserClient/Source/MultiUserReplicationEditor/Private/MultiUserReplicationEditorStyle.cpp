// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserReplicationEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::MultiUserReplicationEditor
{
	TSharedPtr<FSlateStyleSet> FMultiUserReplicationEditorStyle::StyleInstance = nullptr;

	void FMultiUserReplicationEditorStyle::Initialize()
	{
		if (!StyleInstance.IsValid())
		{
			StyleInstance = Create();
			FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
		}
	}

	void FMultiUserReplicationEditorStyle::Shutdown()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
		ensure(StyleInstance.IsUnique());
		StyleInstance.Reset();
	}

	const ISlateStyle& FMultiUserReplicationEditorStyle::Get()
	{
		return *StyleInstance;
	}

	FName FMultiUserReplicationEditorStyle::GetStyleSetName()
	{
		static FName StyleSetName(TEXT("MultiUserReplicationEditor"));
		return StyleSetName;
	}
	
	TSharedRef< FSlateStyleSet > FMultiUserReplicationEditorStyle::Create()
	{
		TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>("MultiUserReplicationEditor");

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("MultiUserClient"));
		if (ensure(Plugin.IsValid()))
		{
			Style->SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
		}
		
		return Style;
	}
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
#undef IMAGE_BRUSH_SVG
