// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::MultiUserReplicationEditor
{
	class FMultiUserReplicationEditorStyle
	{
	public:
	
		static void Initialize();
		static void Shutdown();

		
		static const ISlateStyle& Get();
		static FName GetStyleSetName();

	private:
	
		static TSharedPtr<FSlateStyleSet> StyleInstance;
		static TSharedRef<FSlateStyleSet> Create();
	};
}

