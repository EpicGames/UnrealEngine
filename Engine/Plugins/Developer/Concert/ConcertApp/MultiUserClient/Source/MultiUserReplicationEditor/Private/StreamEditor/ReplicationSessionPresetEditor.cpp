// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSessionPresetEditor.h"

#include "ReplicationSessionPresetEditorToolkit.h"

void UReplicationSessionPresetEditor::SetObjectToEdit(UObject* InObject)
{
	ObjectToEdit = InObject;
}

void UReplicationSessionPresetEditor::GetObjectsToEdit(TArray<UObject*>& OutObjectsToEdit)
{
	OutObjectsToEdit = { ObjectToEdit };
}

TSharedPtr<FBaseAssetToolkit> UReplicationSessionPresetEditor::CreateToolkit()
{
	return MakeShared<UE::MultiUserReplicationEditor::FReplicationSessionPresetEditorToolkit>(this);
}
