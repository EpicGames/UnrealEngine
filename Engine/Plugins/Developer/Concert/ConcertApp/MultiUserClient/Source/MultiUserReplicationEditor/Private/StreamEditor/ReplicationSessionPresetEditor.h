// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"
#include "ReplicationSessionPresetEditor.generated.h"

/**
 * 
 */
UCLASS()
class MULTIUSERREPLICATIONEDITOR_API UReplicationSessionPresetEditor : public UAssetEditor
{
	GENERATED_BODY()
public:

	void SetObjectToEdit(UObject* InObject);

protected:

	//~ Begin UAssetEditor Interface
	virtual void GetObjectsToEdit(TArray<UObject*>& OutObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;
	//~ End UAssetEditor Interface

private:
	
	UPROPERTY(Transient)
	TObjectPtr<UObject> ObjectToEdit;
};
