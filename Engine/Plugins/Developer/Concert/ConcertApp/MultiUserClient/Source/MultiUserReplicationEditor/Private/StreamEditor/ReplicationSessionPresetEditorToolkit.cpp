// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationSessionPresetEditorToolkit.h"

#include "Assets/MultiUserReplicationClientPreset.h"
#include "Assets/MultiUserReplicationSessionPreset.h"
#include "Replication/Editor/Model/PropertySource/SelectPropertyFromUClassModel.h"
#include "Replication/Editor/View/IReplicationStreamEditor.h"
#include "Replication/Editor/Model/ObjectSource/ActorSelectionSourceModel.h"
#include "Replication/ReplicationWidgetFactories.h"

#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Replication/ClientReplicationWidgetFactories.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "FReplicationStreamEditorToolkit"

namespace UE::MultiUserReplicationEditor
{
	const FName FReplicationSessionPresetEditorToolkit::ContentTabId(TEXT("ReplicationStreamAssetEditor_Content"));
	
	FReplicationSessionPresetEditorToolkit::FReplicationSessionPresetEditorToolkit(UAssetEditor* InOwningAssetEditor)
		: FBaseAssetToolkit(InOwningAssetEditor)
	{
		LayoutAppendix = TEXT("ReplicationStreamEditor_v1");
		const FString LayoutString = TEXT("Standalone_Layout_") + LayoutAppendix;
		StandaloneDefaultLayout = FTabManager::NewLayout(FName(LayoutString))
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(1.f)
					->SetHideTabWell(true)
					->AddTab(ContentTabId, ETabState::OpenedTab)
				)
			);
	}

	void FReplicationSessionPresetEditorToolkit::CreateWidgets()
	{
		// Do not call Super because we do not want to create the viewport and details panel from FBaseAssetToolkit
		
	}

	void FReplicationSessionPresetEditorToolkit::SetEditingObject(UObject* InObject)
	{
		// Do not call Super because we do not wish to use the details view from FBaseAssetToolkit
	}

	void FReplicationSessionPresetEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_ReplicationStreamAssetEditor", "Replication Strean Asset Editor"));
		FBaseAssetToolkit::RegisterTabSpawners(InTabManager);

		InTabManager->RegisterTabSpawner(ContentTabId, FOnSpawnTab::CreateSP(this, &FReplicationSessionPresetEditorToolkit::SpawnTab_Content))
			.SetDisplayName(LOCTEXT("ConfigTab", "Config"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	void FReplicationSessionPresetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
		InTabManager->UnregisterTabSpawner(ContentTabId);
	}

	UMultiUserReplicationSessionPreset* FReplicationSessionPresetEditorToolkit::GetEditedStreamAsset() const
	{
		return CastChecked<UMultiUserReplicationSessionPreset>(GetEditingObject());
	}

	TSharedRef<SDockTab> FReplicationSessionPresetEditorToolkit::SpawnTab_Content(const FSpawnTabArgs& SpawnTabArgs)
	{
		using namespace ConcertClientSharedSlate;
		using namespace ConcertSharedSlate;
		return SNew(SDockTab)
			.Label(LOCTEXT("BaseDetailsTitle", "Details"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NotImplemented", "This feature is not implemented"))
			];
	}
}

#undef LOCTEXT_NAMESPACE