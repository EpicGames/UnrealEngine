// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/BaseAssetToolkit.h"

class UMultiUserReplicationSessionPreset;
struct FConcertObjectReplicationMap;

namespace UE::MultiUserReplicationEditor
{
	class SReplicationStreamEditor;

	class FReplicationSessionPresetEditorToolkit : public FBaseAssetToolkit
	{
	public:
		
		static const FName ContentTabId;

		FReplicationSessionPresetEditorToolkit(UAssetEditor* InOwningAssetEditor);

		//~ Begin FAssetEditorToolkit Interface
		virtual void CreateWidgets();
		virtual void SetEditingObject(UObject* InObject) override;
		virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
		virtual FName GetToolkitFName() const override { return TEXT("ReplicationStreamEditorToolkit"); }
		//~ End FAssetEditorToolkit Interface

	private:

		/** Root widget editing the asset */
		TSharedPtr<SReplicationStreamEditor> RootWidget;
		
		UMultiUserReplicationSessionPreset* GetEditedStreamAsset() const;
		
		TSharedRef<SDockTab> SpawnTab_Content(const FSpawnTabArgs& SpawnTabArgs);
	};
}

